{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Motion Design Scene State",
	"Description": "",
	"Category": "Experimental",
	"CreatedBy": "",
	"CreatedByURL": "",
	"DocsURL": "",
	"MarketplaceURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "SceneState",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "SceneStateBinding",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "SceneStateEvent",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "SceneStateTasks",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "SceneStateGameplay",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "SceneStateBlueprint",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		},
		{
			"Name": "SceneStateEventGraph",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		},
		{
			"Name": "SceneStateMachineGraph",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		},
		{
			"Name": "SceneStateTransitionGraph",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		},
		{
			"Name": "SceneStateBlueprintEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "SceneStateEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "SceneStateEventEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "SceneStateMachineEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "SceneStateGameplayEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "PropertyBindingUtils",
			"Enabled": true
		}
	]
}