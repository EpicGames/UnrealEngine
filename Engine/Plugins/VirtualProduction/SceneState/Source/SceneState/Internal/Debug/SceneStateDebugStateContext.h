// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
#include "SceneStateExecutionContextHandle.h"
#include "Templates/SharedPointer.h"

#define UE_API SCENESTATE_API

struct FSceneState;
struct FSceneStateExecutionContext;
struct FSceneStateInstance;

namespace UE::SceneState
{

/** Context for a debugged state */
struct FDebugStateContext
{
	explicit FDebugStateContext(uint16 InStateIndex, const FSceneStateInstance& InStateInstance);

	uint16 GetStateIndex() const
	{
		return StateIndex;
	}

	bool Equals(const FDebugStateContext& InOther) const;

private:
	/** Index of the state that is being debugged */
	uint16 StateIndex;

	/** Identifier for a single state instance */
	uint16 StateInstanceId;
};

} // UE::SceneState

#undef UE_API

#endif
