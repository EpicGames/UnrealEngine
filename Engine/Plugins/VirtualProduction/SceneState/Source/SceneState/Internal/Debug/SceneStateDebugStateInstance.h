// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
#include "SceneStateDebugStateContext.h"
#include "Misc/Optional.h"
#include "SceneStateEnums.h"

#define UE_API SCENESTATE_API

struct FSceneStateInstance;

namespace UE::SceneState
{

/** Holds debug information about the state to visualize as node info popups in the state machine graph */
class FDebugStateInstance
{
	static constexpr double EnteringTime = 0.125;
	static constexpr double ExitingTime = 0.75;

public:
	explicit FDebugStateInstance(const FDebugStateContext& InContext, EExecutionStatus InInitialStatus);

	uint16 GetStateIndex() const;

	/** Returns true if the debug instance matches the given state context information */
	bool operator==(const FDebugStateContext& InOther) const;

	/** Calculate how 'active' a state is. Returns a float value in the range [0, 1] */
	UE_API float GetActivePercentage() const;

	/** Determines whether the state is in a current valid state */
	UE_API bool IsValid() const;

	/**
	 * Returns the elapsed time for the current execution status.
	 * The elapsed time resets every time the execution status changes.
	 */
	UE_API double GetStatusElapsedTime() const;

	EExecutionStatus GetExecutionStatus() const
	{
		return ExecutionStatus;
	}

	/** Sets the execution status of this state debug instance (to match with the actual executed state instance)  */
	void SetExecutionStatus(EExecutionStatus InExecutionStatus);

private:
	/** Context of the executed state */
	FDebugStateContext DebugStateContext;

	/** Current execution status of the state */
	EExecutionStatus ExecutionStatus = EExecutionStatus::NotStarted;

	/** Last time the status was changed */
	double StatusChangeTime = 0.0;
};

} // UE::SceneState

#undef UE_API

#endif // WITH_EDITOR
