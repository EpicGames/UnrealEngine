// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
#include "Containers/Map.h"
#include "SceneStateDebugStateInstance.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakInterfacePtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define UE_API SCENESTATE_API

class USceneStateObject;

namespace UE::SceneState
{
	struct FDebugStateContext;
}

namespace UE::SceneState
{

/** Processes execution events like state status changes. Editor-only */
class FDebugger : public TSharedFromThis<FDebugger>
{
public:
	UE_API ~FDebugger();

	/** Called to attach this debugger to the target scene state object */
	UE_API void AttachTo(USceneStateObject* InObject);

	/** Called to detach this debugger from any existing scene state object */
	void Detach();

	/** Returns whether this debugger is attached to the given object */
	bool IsAttachedTo(USceneStateObject* InObject) const;

	/** Calls the given functor for every state debug instance that matches the state index */
	UE_API void ForEachDebugInstanceOfState(uint16 InStateIndex, TFunctionRef<void(const FDebugStateInstance&)> InFunc);

	/** Called when a state instance has changed status */
	void NotifyStateStatusChanged(const FDebugStateContext& InContext, UE::SceneState::EExecutionStatus InStatus);

private:
	/** Populates debug state instances with currently active states */
	void Initialize();

	/** Removes all state debug instances that are no longer valid */
	void CleanInvalidDebugInstances();

	/** All the relevant debug instances that are or were recently active. */
	TArray<FDebugStateInstance> DebugStateInstances;

	/** The object that this debugger is currently attached to */
	TWeakObjectPtr<USceneStateObject> DebuggedObjectWeak;
};

} // UE::SceneState

#undef UE_API

#endif // WITH_EDITOR
