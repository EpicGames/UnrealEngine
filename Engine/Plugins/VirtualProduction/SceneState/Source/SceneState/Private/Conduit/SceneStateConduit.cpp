// Copyright Epic Games, Inc. All Rights Reserved.

#include "Conduit/SceneStateConduit.h"
#include "Conduit/SceneStateConduitInstance.h"
#include "Conduit/SceneStateConduitLink.h"
#include "SceneState.h"
#include "SceneStateExecutionContext.h"
#include "SceneStateObject.h"
#include "Transition/SceneStateTransition.h"
#include "Transition/SceneStateTransitionEvaluation.h"
#include "Transition/SceneStateTransitionResult.h"

void FSceneStateConduit::Link(const FSceneStateConduitLink& InConduitLink, TNotNull<const UClass*> InOwnerClass)
{
	if (!InConduitLink.ResultPropertyName.IsNone())
	{
		ResultProperty = CastField<FStructProperty>(InOwnerClass->FindPropertyByName(InConduitLink.ResultPropertyName));
		check(ResultProperty);
	}
	else
	{
		ResultProperty = nullptr;
	}

	if (!InConduitLink.EventName.IsNone())
	{
		EvaluationEvent = InOwnerClass->FindFunctionByName(InConduitLink.EventName);
		check(EvaluationEvent);
	}
	else
	{
		EvaluationEvent = nullptr;
	}
}

void FSceneStateConduit::Setup(const FSceneStateExecutionContext& InContext) const
{
	FSceneStateConduitInstance* Instance = InContext.FindOrAddConduitInstance(*this);
	if (!Instance || Instance->bInitialized)
	{
		return;
	}

	Instance->bInitialized = true;

	for (const FSceneStateTransition& Transition : InContext.GetTransitions(*this))
	{
		Transition.Setup(InContext);
	}
}

bool FSceneStateConduit::Evaluate(const UE::SceneState::FTransitionEvaluationParams& InParams) const
{
	// Early return if waiting for tasks to finish and there are still active tasks yet to finish
	if (EnumHasAnyFlags(EvaluationFlags, ESceneStateTransitionEvaluationFlags::WaitForTasksToFinish)
		&& InParams.SourceState.HasPendingTasks(InParams.ExecutionContext))
	{
		return false;
	}

	return ProcessEvaluationEvent(InParams);
}

void FSceneStateConduit::Exit(const FSceneStateExecutionContext& InContext) const
{
	InContext.RemoveConduitInstance(*this);
}

bool FSceneStateConduit::ProcessEvaluationEvent(const UE::SceneState::FTransitionEvaluationParams& InParams) const
{
	if (EnumHasAllFlags(EvaluationFlags, ESceneStateTransitionEvaluationFlags::EvaluationEventAlwaysTrue))
	{
		return true;
	}

	if (!EvaluationEvent || !ResultProperty)
	{
		return false;
	}

	void* FunctionParams = nullptr;

	UObject* RootObject = InParams.ExecutionContext.GetRootObject();
	if (!ensure(RootObject))
	{
		return false;
	}

	RootObject->ProcessEvent(EvaluationEvent, FunctionParams);

	const FSceneStateTransitionResult* Result = ResultProperty->ContainerPtrToValuePtr<FSceneStateTransitionResult>(RootObject);
	check(Result);
	return Result->bCanTransition;
}
