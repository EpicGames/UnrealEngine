// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR
#include "Debug/SceneStateDebugStateContext.h"
#include "SceneStateExecutionContext.h"
#include "SceneStateExecutionContextRegistry.h"

namespace UE::SceneState
{

FDebugStateContext::FDebugStateContext(uint16 InStateIndex, const FSceneStateInstance& InStateInstance)
	: StateIndex(InStateIndex)
	, StateInstanceId(InStateInstance.GetInstanceId())
{
}

bool FDebugStateContext::Equals(const FDebugStateContext& InOther) const
{
	return StateIndex == InOther.StateIndex
		&& StateInstanceId == InOther.StateInstanceId;
}

} // UE::SceneState

#endif // WITH_EDITOR
