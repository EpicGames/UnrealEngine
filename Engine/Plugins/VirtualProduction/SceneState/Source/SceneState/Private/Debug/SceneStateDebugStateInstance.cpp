// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR
#include "Debug/SceneStateDebugStateInstance.h"
#include "Debug/SceneStateDebugStateContext.h"
#include "Misc/App.h"
#include "Misc/AppTime.h"
#include "SceneStateExecutionContext.h"

namespace UE::SceneState
{

FDebugStateInstance::FDebugStateInstance(const FDebugStateContext& InContext, EExecutionStatus InInitialStatus)
	: DebugStateContext(InContext)
{
	SetExecutionStatus(InInitialStatus);
}

uint16 FDebugStateInstance::GetStateIndex() const
{
	return DebugStateContext.GetStateIndex();
}

bool FDebugStateInstance::operator==(const FDebugStateContext& InOther) const
{
	return DebugStateContext.Equals(InOther);
}

float FDebugStateInstance::GetActivePercentage() const
{
	const double StatusElapsedTime = GetStatusElapsedTime();

	const double StatusChangeTotalTime = ExecutionStatus == EExecutionStatus::Running
		? EnteringTime
		: ExitingTime;

	const float Percentage = StatusElapsedTime >= StatusChangeTotalTime
		? 1.f
		: FMath::Clamp(static_cast<float>(StatusElapsedTime / StatusChangeTotalTime), 0.f, 1.f);

	return ExecutionStatus == EExecutionStatus::Running
		? Percentage
		: 1.f - Percentage;
}

bool FDebugStateInstance::IsValid() const
{
	if (ExecutionStatus == EExecutionStatus::Finished)
	{
		return GetStatusElapsedTime() <= ExitingTime;
	}
	return ExecutionStatus == EExecutionStatus::Running;
}

double FDebugStateInstance::GetStatusElapsedTime() const
{
	return FAppTime::GetCurrentTime() - StatusChangeTime;
}

void FDebugStateInstance::SetExecutionStatus(EExecutionStatus InExecutionStatus)
{
	if (ExecutionStatus != InExecutionStatus)
	{
		ExecutionStatus = InExecutionStatus;
		StatusChangeTime = FAppTime::GetCurrentTime();
	}
}

} // UE::SceneState
#endif
