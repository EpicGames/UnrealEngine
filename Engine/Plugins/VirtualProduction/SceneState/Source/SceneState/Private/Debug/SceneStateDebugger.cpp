// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR
#include "Debug/SceneStateDebugger.h"
#include "Debug/SceneStateDebugStateContext.h"
#include "SceneStateExecutionContextRegistry.h"
#include "SceneStateInstance.h"
#include "SceneStateObject.h"

namespace UE::SceneState
{

FDebugger::~FDebugger()
{
	Detach();
}

void FDebugger::AttachTo(USceneStateObject* InObject)
{
	if (DebuggedObjectWeak != InObject)
	{
		Detach();

		DebuggedObjectWeak = InObject;

		if (InObject)
		{
			InObject->SetDebugger(AsWeak());
		}

		Initialize();
	}
}

void FDebugger::Detach()
{
	if (!UObjectInitialized())
	{
		DebuggedObjectWeak.Reset();
		DebugStateInstances.Empty();
		return;
	}

	USceneStateObject* const DebuggedObject = DebuggedObjectWeak.Get();
	DebuggedObjectWeak.Reset(); // reset to prevent loop due to reentry via SetDebugger(nullptr)
	DebugStateInstances.Empty();

	if (DebuggedObject)
	{
		const UE::SceneState::FDebugger* const Debugger = DebuggedObject->GetDebugger().Get();
		if (ensure(!Debugger || Debugger == this))
		{
			DebuggedObject->SetDebugger(nullptr);
		}
	}
}

bool FDebugger::IsAttachedTo(USceneStateObject* InObject) const
{
	return InObject && DebuggedObjectWeak == InObject;
}

void FDebugger::ForEachDebugInstanceOfState(uint16 InStateIndex, TFunctionRef<void(const FDebugStateInstance&)> InFunc)
{
	CleanInvalidDebugInstances();

	for (const FDebugStateInstance& StateDebugInstance : DebugStateInstances)
	{
		if (StateDebugInstance.GetStateIndex() == InStateIndex)
		{
			InFunc(StateDebugInstance);
		}
	}
}

void FDebugger::NotifyStateStatusChanged(const FDebugStateContext& InContext, UE::SceneState::EExecutionStatus InStatus)
{
	CleanInvalidDebugInstances();

	if (FDebugStateInstance* DebugStateInstance = DebugStateInstances.FindByKey(InContext))
	{
		DebugStateInstance->SetExecutionStatus(InStatus);
	}
	else
	{
		// Status is already handled within debug state instance construction
		DebugStateInstances.Emplace(InContext, InStatus);
	}
}

void FDebugger::Initialize()
{
	USceneStateObject* const DebuggedObject = DebuggedObjectWeak.Get();
	if (!DebuggedObject)
	{
		return;
	}

	DebugStateInstances.Empty();

	DebuggedObject->GetContextRegistry()->ForEachExecutionContext(
		[This=this](const FSceneStateExecutionContext& InContext)
		{
			InContext.ForEachStateInstance(
				[&InContext, This](uint16 InStateIndex, const FSceneStateInstance& InStateInstance)
				{
					const FDebugStateContext StateContext(InStateIndex, InStateInstance);
					This->DebugStateInstances.Emplace(StateContext, InStateInstance.GetStatus());
				});
		});
}

void FDebugger::CleanInvalidDebugInstances()
{
	DebugStateInstances.RemoveAll(
		[](const FDebugStateInstance& InDebugInstance)
		{
			return !InDebugInstance.IsValid();
		});
}

} // UE::SceneState

#endif // WITH_EDITOR
