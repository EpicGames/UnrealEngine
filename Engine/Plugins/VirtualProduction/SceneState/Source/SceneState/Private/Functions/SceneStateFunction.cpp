// Copyright Epic Games, Inc. All Rights Reserved.

#include "Functions/SceneStateFunction.h"
#include "SceneStateExecutionContext.h"
#include "SceneStateUtils.h"

#if WITH_EDITOR
const UScriptStruct* FSceneStateFunction::GetFunctionDataType() const
{
	return OnGetFunctionDataType();
}
#endif

void FSceneStateFunction::Setup(const FSceneStateExecutionContext& InContext) const
{
	InContext.SetupFunctionInstances(BindingsBatch);
}

void FSceneStateFunction::Execute(const FSceneStateExecutionContext& InContext, FStructView InFunctionInstance) const
{
	ApplyBindings(InContext, InFunctionInstance);
	OnExecute(InContext, InFunctionInstance);
}

void FSceneStateFunction::Cleanup(const FSceneStateExecutionContext& InContext) const
{
	InContext.RemoveFunctionInstances(BindingsBatch);
}

bool FSceneStateFunction::ApplyBindings(const FSceneStateExecutionContext& InContext, FStructView InFunctionInstance) const
{
	QUICK_SCOPE_CYCLE_COUNTER(SceneStateFunction_ApplyBindings);

	const UE::SceneState::FApplyBatchParams ApplyBatchParams
		{
			.BindingsBatch = BindingsBatch.Get(),
			.TargetDataView = InFunctionInstance,
		};

	return ApplyBatch(InContext, ApplyBatchParams);
}
