// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneState.h"
#include "SceneStateEventHandler.h"
#include "SceneStateEventStream.h"
#include "SceneStateExecutionContext.h"
#include "SceneStateInstance.h"
#include "SceneStateLog.h"
#include "SceneStateMachine.h"
#include "SceneStateObject.h"
#include "SceneStateUtils.h"
#include "StructUtils/InstancedStructContainer.h"
#include "Tasks/SceneStateTask.h"
#include "Tasks/SceneStateTaskInstance.h"
#include "Transition/SceneStateTransition.h"

void FSceneState::Enter(const FSceneStateExecutionContext& InContext) const
{
	using namespace UE::SceneState;

	const FReentryGuard ReentryGuard(ReentryHandle, InContext);
	if (ReentryGuard.IsReentry())
	{
		return;
	}

	// Enter State, add a State Instance if not already present
	FSceneStateInstance& Instance = InContext.FindOrAddStateInstance(*this);
	if (Instance.GetStatus() == EExecutionStatus::Running)
	{
		return;
	}

	UE_LOGF(LogSceneState, Verbose, "State (%ls) receiving enter", GetStateName(InContext).GetData());

	Instance.Setup(*this);
	InContext.SetStateStatus(*this, Instance, EExecutionStatus::Running);
	InContext.SetupFunctionInstances(BindingsBatch);

	// Apply Event Handlers before anything else starts so the Event Data becomes available to this state, substate machines and tasks
	CaptureEvents(InContext);
	AllocateTaskInstances(InContext, InContext.GetTemplateTaskInstances(*this));
	ApplyBindings(InContext, Instance);

	for (const FSceneStateTransition& Transition : InContext.GetTransitions(*this))
	{
		Transition.Setup(InContext);
	}

	for (const FSceneStateMachine& StateMachine : InContext.GetStateMachines(*this))
	{
		StateMachine.Setup(InContext);
	}

	// Setup each Task
	InContext.ForEachTask(*this,
		[&InContext, &Instance](const FSceneStateTask& InTask, uint16 InTaskRelativeIndex)->EIterationResult
		{
			const FStructView TaskInstance = Instance.GetTaskInstances()[InTaskRelativeIndex];
			InTask.Setup(InContext, TaskInstance);
			return EIterationResult::Continue;
		});

	for (const FSceneStateMachine& StateMachine : InContext.GetStateMachines(*this))
	{
		StateMachine.Start(InContext);
	}

	UpdateActiveTasks(InContext, Instance);
}

void FSceneState::Tick(const FSceneStateExecutionContext& InContext, float InDeltaSeconds) const
{
	using namespace UE::SceneState;

	const FReentryGuard ReentryGuard(ReentryHandle, InContext);
	if (ReentryGuard.IsReentry())
	{
		return;
	}

	FSceneStateInstance* Instance = InContext.FindStateInstance(*this);
	if (!Instance || Instance->GetStatus() != EExecutionStatus::Running)
	{
		return;
	}

	UE_LOGF(LogSceneState, VeryVerbose, "State (%ls) receiving tick", GetStateName(InContext).GetData());

	UpdateActiveTasks(InContext, *Instance);

	for (const FSceneStateMachine& StateMachine : InContext.GetStateMachines(*this))
	{
		StateMachine.Tick(InContext, InDeltaSeconds);
	}

	InContext.ForEachTask(*this,
		[&InContext, Instance, InDeltaSeconds](const FSceneStateTask& InTask, uint16 InTaskRelativeIndex)
		{
			const FStructView TaskInstance = Instance->GetTaskInstances()[InTaskRelativeIndex];
			InTask.Tick(InContext, TaskInstance, InDeltaSeconds);
			return EIterationResult::Continue;
		});
}

void FSceneState::Exit(const FSceneStateExecutionContext& InContext) const
{
	using namespace UE::SceneState;

	const FReentryGuard ReentryGuard(ReentryHandle, InContext);
	if (ReentryGuard.IsReentry())
	{
		return;
	}

	FSceneStateInstance* Instance = InContext.FindStateInstance(*this);
	if (!Instance || Instance->GetStatus() != EExecutionStatus::Running)
	{
		return;
	}

	UE_LOGF(LogSceneState, Verbose, "State (%ls) receiving exit", GetStateName(InContext).GetData());

	// Stop State Machines that are still running
	for (const FSceneStateMachine& StateMachine : InContext.GetStateMachines(*this))
	{
		StateMachine.Stop(InContext);
	}

	// Stop tasks that are still running
	InContext.ForEachTask(*this,
		[&InContext, Instance](const FSceneStateTask& InTask, uint16 InTaskRelativeIndex)
		{
			const FStructView TaskInstance = Instance->GetTaskInstances()[InTaskRelativeIndex];
			InTask.Stop(InContext, TaskInstance, ESceneStateTaskStopReason::State);
			return EIterationResult::Continue;
		});

	// Notify transitions of stop
	for (const FSceneStateTransition& Transition : InContext.GetTransitions(*this))
	{
		Transition.Exit(InContext);
	}

	InContext.SetStateStatus(*this, *Instance, EExecutionStatus::Finished);

	InContext.RemoveStateInstance(*this);
	InContext.RemoveFunctionInstances(BindingsBatch);

	ResetCapturedEvents(InContext);
}

const FInstancedPropertyBag& FSceneState::GetParameters() const
{
	return Parameters;
}

const UStruct* FSceneState::GetParametersStruct() const
{
	return Parameters.GetPropertyBagStruct();
}

void FSceneState::UpdateActiveTasks(const FSceneStateExecutionContext& InContext, FSceneStateInstance& InStateInstance) const
{
	using namespace UE::SceneState;

	// Start all the Tasks that haven't started yet and that meet their pre-requisites
	InContext.ForEachTask(*this,
		[&InContext, StartingIndex=TaskRange.Index, &InStateInstance](const FSceneStateTask& InTask, uint16 InTaskRelativeIndex)->EIterationResult
		{
			if (InStateInstance.GetTaskStatus(InTaskRelativeIndex) != EExecutionStatus::NotStarted)
			{
				return EIterationResult::Continue;
			}

			bool bPrerequisitesMet = true;
			for (uint16 PrerequisiteIndex : InContext.GetTaskPrerequisites(InTask))
			{
				// Prerequisite indices are stored relative
				if (InStateInstance.GetTaskStatus(PrerequisiteIndex) != EExecutionStatus::Finished)
				{
					bPrerequisitesMet = false;
					break;
				}
			}

			if (bPrerequisitesMet)
			{
				const FStructView TaskInstance = InStateInstance.GetTaskInstances()[InTaskRelativeIndex];
				InTask.Start(InContext, TaskInstance);
			}

			return EIterationResult::Continue;
		});
}

FStringView FSceneState::GetStateName(const FSceneStateExecutionContext& InContext) const
{
#if WITH_EDITOR
	if (const FSceneStateMetadata* StateMetadata = InContext.GetStateMetadata(*this))
	{
		return StateMetadata->StateName;
	}
#endif
	return FStringView();
}

bool FSceneState::HasPendingTasks(const FSceneStateExecutionContext& InContext) const
{
	using namespace UE::SceneState;

	FSceneStateInstance* const Instance = InContext.FindStateInstance(*this);
	if (!Instance || Instance->GetStatus() != EExecutionStatus::Running)
	{
		return false;
	}

	bool bHasPendingTask = false;

	InContext.ForEachTask(*this,
		[&InContext, &bHasPendingTask, Instance](const FSceneStateTask& InTask, uint16 InTaskRelativeIndex)->EIterationResult
		{
			if (Instance->GetTaskStatus(InTaskRelativeIndex) != EExecutionStatus::Finished)
			{
				bHasPendingTask = true;
				return EIterationResult::Break;
			}
			return EIterationResult::Continue;
		});

	return bHasPendingTask;
}

void FSceneState::AllocateTaskInstances(const FSceneStateExecutionContext& InContext, TConstArrayView<FConstStructView> InTemplateTaskInstances) const
{
	FSceneStateInstance& StateInstance = InContext.FindOrAddStateInstance(*this);

	// Copy the Template data
	StateInstance.GetTaskInstances() = InTemplateTaskInstances;

	// Instance each Template Object into the Instance data
	InstanceTaskObjects(InContext.GetRootObject()
		, UE::SceneState::GetStructViews(StateInstance.GetTaskInstances())
		, InTemplateTaskInstances
		, [](FObjectDuplicationParameters& InParams)->UObject*
		{
			return StaticDuplicateObjectEx(InParams);
		});
}

bool FSceneState::ApplyBindings(const FSceneStateExecutionContext& InContext, FSceneStateInstance& InStateInstance) const
{
	QUICK_SCOPE_CYCLE_COUNTER(SceneState_ApplyBindings);

	const UE::SceneState::FApplyBatchParams ApplyBatchParams
		{
			.BindingsBatch = BindingsBatch.Get(),
			.TargetDataView = InStateInstance.GetParameters().GetMutableValue(),
		};

	return ApplyBatch(InContext, ApplyBatchParams);
}

void FSceneState::InstanceTaskObjects(UObject* InOuter, TConstArrayView<FStructView> InTargets, TConstArrayView<FConstStructView> InSources, TFunctionRef<UObject*(FObjectDuplicationParameters&)> InDuplicationFunc) const
{
	check(InTargets.Num() == InSources.Num());

	for (int32 TaskIndex = 0; TaskIndex < InSources.Num(); ++TaskIndex)
	{
		const FConstStructView Source = InSources[TaskIndex];
		const FStructView Target = InTargets[TaskIndex];

		check(Source.GetScriptStruct() == Target.GetScriptStruct());

		if (!Source.GetScriptStruct())
		{
			continue;
		}

		// Visit all object properties that are instanced references, stepping into structs but not object properties themselves.
		// Using const_cast here as there's no Visit variant taking a const ptr.
		Source.GetScriptStruct()->Visit(const_cast<uint8*>(Source.GetMemory()),
			[&InDuplicationFunc, InOuter, Target](const FPropertyVisitorContext& InContext)->EPropertyVisitorControlFlow
			{
				const FProperty* const Property = InContext.Path.Top().Property;
				if  (!Property)
				{
					return EPropertyVisitorControlFlow::StepOver;
				}

				const FObjectProperty* const ObjectProperty = CastField<FObjectProperty>(Property);
				if (!ObjectProperty)
				{
					return Property->IsA<FObjectPropertyBase>()
						? EPropertyVisitorControlFlow::StepOver // step over for other object property types (e.g. soft/weak).
						: EPropertyVisitorControlFlow::StepInto;
				}

				// TODO: There could be potential cases where a property isn't marked as an instanced reference, that we'd want to duplicate.
				if (!ObjectProperty->HasAnyPropertyFlags(CPF_InstancedReference))
				{
					return EPropertyVisitorControlFlow::StepOver;
				}

				const void* const SourcePropertyMemory = InContext.Data.PropertyData;

				UObject* InstanceObject = nullptr;
				if (UObject* TemplateObject = ObjectProperty->GetObjectPropertyValue(SourcePropertyMemory))
				{
					FObjectDuplicationParameters Parameters = InitStaticDuplicateObjectParams(TemplateObject, InOuter);
					Parameters.DestName = MakeUniqueObjectName(Parameters.DestOuter, Parameters.SourceObject->GetClass(), Parameters.SourceObject->GetFName());
					Parameters.FlagMask = RF_AllFlags & ~RF_DefaultSubObject;
					Parameters.PortFlags |= PPF_DuplicateVerbatim; // Skip resetting text IDs

					InstanceObject = InDuplicationFunc(Parameters);

					UE_LOGF(LogSceneState, Verbose, "Instantiated object '%ls' (Outer: '%ls') from template '%ls' (Outer: '%ls')"
						, *GetNameSafe(InstanceObject)
						, *InOuter->GetName()
						, *TemplateObject->GetName()
						, *GetNameSafe(TemplateObject->GetOuter()));
				}

				void* const TargetPropertyMemory = PropertyVisitorHelpers::ResolveVisitedPath(Target.GetScriptStruct(), Target.GetMemory(), InContext.Path);
				ObjectProperty->SetPropertyValue(TargetPropertyMemory, InstanceObject);
				return EPropertyVisitorControlFlow::StepOver;
			});
	}
}

void FSceneState::CaptureEvents(const FSceneStateExecutionContext& InContext) const
{
	if (USceneStateEventStream* EventStream = InContext.GetEventStream())
	{
		TConstArrayView<FSceneStateEventHandler> EventHandlers = InContext.GetEventHandlers(*this);
		EventStream->CaptureEvents(EventHandlers);
	}
}

void FSceneState::ResetCapturedEvents(const FSceneStateExecutionContext& InContext) const
{
	if (USceneStateEventStream* EventStream = InContext.GetEventStream())
	{
		TConstArrayView<FSceneStateEventHandler> EventHandlers = InContext.GetEventHandlers(*this);
		EventStream->ResetCapturedEvents(EventHandlers);
	}
}
