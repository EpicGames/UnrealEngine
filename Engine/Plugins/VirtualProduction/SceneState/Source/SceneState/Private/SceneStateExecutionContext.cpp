// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateExecutionContext.h"
#include "Functions/SceneStateFunction.h"
#include "PropertyBindingDataView.h"
#include "PropertyBindingTypes.h"
#include "SceneState.h"
#include "SceneStateExecutionContextRegistry.h"
#include "SceneStateLog.h"
#include "SceneStateMachine.h"
#include "SceneStateObject.h"
#include "SceneStateTemplateData.h"
#include "SceneStateUtils.h"
#include "Tasks/SceneStateTask.h"
#include "Tasks/SceneStateTaskBindingExtension.h"
#include "Tasks/SceneStateTaskInstance.h"

#if WITH_EDITOR
#include "Debug/SceneStateDebugStateContext.h"
#include "Debug/SceneStateDebugger.h"
#endif

const FSceneStateExecutionContext FSceneStateExecutionContext::InvalidContext;

FSceneStateExecutionContext::~FSceneStateExecutionContext()
{
	Reset();
}

TWeakPtr<const UE::SceneState::FExecutionContextRegistry> FSceneStateExecutionContext::GetContextRegistry() const
{
	return ContextRegistryWeak;
}

UE::SceneState::FExecutionContextHandle FSceneStateExecutionContext::GetContextHandle() const
{
	return ContextHandle;
}

const FSceneStateBindingCollection& FSceneStateExecutionContext::GetBindingCollection() const
{
	return TemplateData->BindingCollection;
}

const FString& FSceneStateExecutionContext::GetExecutionContextName() const
{
	return ContextName;
}

USceneStateObject* FSceneStateExecutionContext::GetRootObject() const
{
	return RootObject;
}

UObject* FSceneStateExecutionContext::GetContextObject() const
{
	return ContextObjectWeak.Get();
}

USceneStateEventStream* FSceneStateExecutionContext::GetEventStream() const
{
	return EventStreamWeak.Get();
}

void FSceneStateExecutionContext::Setup(const FSceneStateExecutionContext& InParentContext)
{
	const FSetupParams SetupParams
	{
		.TemplateData = InParentContext.TemplateData,
		.RootObject = InParentContext.GetRootObject(),
		.ContextObject = InParentContext.GetContextObject(),
		.EventStream = InParentContext.GetEventStream(),
		.ContextRegistry = InParentContext.ContextRegistryWeak.Pin(),
		.ContextName = InParentContext.GetExecutionContextName(),
	};
	Setup(SetupParams);

	if (SetupParams.ContextRegistry.IsValid())
	{
		ParentHandle = SetupParams.ContextRegistry->FindHandle(InParentContext);
	}
}

void FSceneStateExecutionContext::Setup(TNotNull<USceneStateObject*> InRootState)
{
	const FSetupParams SetupParams
	{
		.TemplateData = InRootState->GetTemplateData(),
		.RootObject = InRootState,
		.ContextObject = InRootState->GetContextObject(),
		.EventStream = InRootState->GetEventStream(),
		.ContextRegistry = InRootState->GetContextRegistry(),
		.ContextName = InRootState->GetContextName(),
	};
	Setup(SetupParams);
}

void FSceneStateExecutionContext::Setup(const FSetupParams& InParams)
{
	Reset();

	TemplateData = InParams.TemplateData;
	RootObject = InParams.RootObject;
	ContextObjectWeak = InParams.ContextObject;
	EventStreamWeak = InParams.EventStream;
	ContextRegistryWeak = InParams.ContextRegistry;
	ContextName = InParams.ContextName;

	if (InParams.ContextRegistry.IsValid())
	{
		ContextHandle = InParams.ContextRegistry->RegisterContext(this);
	}
	else
	{
		ContextHandle = UE::SceneState::FExecutionContextHandle();
	}
}

void FSceneStateExecutionContext::Reset()
{
	TemplateData = nullptr;

	StateInstances.Reset();
	StateMachineInstances.Reset();
	TransitionInstances.Reset();
	ConduitInstances.Reset();
	StateMachineExecutionStack.Reset();
	FunctionInstanceBatches.Reset();
	FunctionInstanceViews.Reset();

	if (TSharedPtr<UE::SceneState::FExecutionContextRegistry> ContextRegistry = ContextRegistryWeak.Pin())
	{
		ContextRegistry->UnregisterContext(this);
	}

	ContextHandle = UE::SceneState::FExecutionContextHandle();
	ParentHandle = UE::SceneState::FExecutionContextHandle();

	ContextRegistryWeak.Reset();
}

FPropertyBindingDataView FSceneStateExecutionContext::FindDataView(const FSceneStateBindingDataHandle& InDataHandle) const
{
	const uint16 DataIndex = InDataHandle.GetDataIndex();
	const uint16 DataSubIndex = InDataHandle.GetDataSubIndex();

	if (InDataHandle.IsExternalDataType())
	{
		// Todo: unsupported external sources
		return FPropertyBindingDataView();
	}

	const ESceneStateDataType DataType = static_cast<ESceneStateDataType>(InDataHandle.GetDataType());

	switch (DataType)
	{
	case ESceneStateDataType::Root:
		return FPropertyBindingDataView(RootObject);

	case ESceneStateDataType::Task:
		return FindTaskInstance(DataIndex);

	case ESceneStateDataType::EventHandler:
		if (USceneStateEventStream* EventStream = GetEventStream())
		{
			if (const FSceneStateEventHandler* EventHandler = GetEventHandler(DataIndex))
			{
				if (FSceneStateEvent* Event = EventStream->FindCapturedEvent(EventHandler->GetHandlerId()))
				{
					return Event->GetDataViewMutable();
				}
			}
		}
		break;

	case ESceneStateDataType::TaskExtension:
		if (const FSceneStateTask* Task = FindTask(DataIndex).GetPtr<const FSceneStateTask>())
		{
			if (const FSceneStateTaskBindingExtension* BindingExtension = Task->GetBindingExtension())
			{
				FStructView DataView;
				if (BindingExtension->FindDataByIndex(FindTaskInstance(DataIndex), DataSubIndex, DataView))
				{
					return DataView;
				}
			}
		}
		break;

	case ESceneStateDataType::Transition:
		if (FSceneStateTransitionInstance* TransitionInstance = FindTransitionInstance(DataIndex))
		{
			return TransitionInstance->Parameters.GetMutableValue();
		}
		break;

	case ESceneStateDataType::StateMachine:
		if (FSceneStateMachineInstance* StateMachineInstance = FindStateMachineInstance(DataIndex))
		{
			return StateMachineInstance->Parameters.GetMutableValue();
		}
		break;

	case ESceneStateDataType::Function:
		return FindFunctionInstance(DataIndex);

	case ESceneStateDataType::State:
		if (FSceneStateInstance* StateInstance = FindStateInstance(DataIndex))
		{
			return StateInstance->GetParameters().GetMutableValue();
		}
		break;
	}

	// The data handle could be exclusively on the parent context, so call FindDataView on the parent too
	if (const FSceneStateExecutionContext* ParentContext = FindParentContext())
	{
		return ParentContext->FindDataView(InDataHandle);
	}

	return FPropertyBindingDataView();
}

void FSceneStateExecutionContext::ForEachTask(const FSceneState& InState, TFunctionRef<UE::SceneState::EIterationResult(const FSceneStateTask&, uint16)> InCallable) const
{
	const FSceneStateRange TaskRange = InState.GetTaskRange();
	if (TaskRange.Count == 0)
	{
		return;
	}

	const FInstancedStructContainer& Tasks = TemplateData->Tasks;

	checkfSlow(Tasks.IsValidIndex(TaskRange.Index) && Tasks.IsValidIndex(TaskRange.GetLastIndex())
		, TEXT("[Context: %s] State Task Range [%d, %d] is out of bounds! (Total Task Count: %d)")
		, *GetExecutionContextName()
		, TaskRange.Index
		, TaskRange.GetLastIndex()
		, Tasks.Num());

	for (uint16 RelativeIndex = 0; RelativeIndex < TaskRange.Count; ++RelativeIndex)
	{
		const uint16 AbsoluteIndex = TaskRange.Index + RelativeIndex;
		const FSceneStateTask& Task = Tasks[AbsoluteIndex].Get<const FSceneStateTask>();
		if (InCallable(Task, RelativeIndex) == UE::SceneState::EIterationResult::Break)
		{
			break;
		}
	}
}

TArray<FConstStructView> FSceneStateExecutionContext::GetTemplateTaskInstances(const FSceneState& InState) const
{
	return UE::SceneState::GetConstStructViews(TemplateData->TaskInstances, InState.GetTaskRange());
}

TConstArrayView<uint16> FSceneStateExecutionContext::GetTaskPrerequisites(const FSceneStateTask& InTask) const
{
	const FSceneStateRange TaskPrerequisiteRange = InTask.GetPrerequisiteRange();
	if (TaskPrerequisiteRange.Count == 0)
	{
		return {};
	}

	const TConstArrayView<uint16> TaskPrerequisitesView = TemplateData->TaskPrerequisites;

	checkfSlow(TaskPrerequisitesView.IsValidIndex(TaskPrerequisiteRange.Index) && TaskPrerequisitesView.IsValidIndex(TaskPrerequisiteRange.GetLastIndex())
		, TEXT("[Context: %s] Task Prerequisite Range [%d, %d] is out of bounds! (Total Prerequisite Count: %d)")
		, *GetExecutionContextName()
		, TaskPrerequisiteRange.Index
		, TaskPrerequisiteRange.GetLastIndex()
		, TaskPrerequisitesView.Num());

	return TaskPrerequisitesView.Slice(TaskPrerequisiteRange.Index, TaskPrerequisiteRange.Count);
}

TConstArrayView<FSceneStateTransition> FSceneStateExecutionContext::GetTransitions(const FSceneState& InState) const
{
	const FSceneStateRange TransitionRange = InState.GetTransitionRange();
	if (TransitionRange.Count == 0)
	{
		return {};
	}

	const TConstArrayView<FSceneStateTransition> TransitionsView = TemplateData->Transitions;

	checkfSlow(TransitionsView.IsValidIndex(TransitionRange.Index) && TransitionsView.IsValidIndex(TransitionRange.GetLastIndex())
		, TEXT("[Context: %s] State Transition Range [%d, %d] is out of bounds! (Total Transition Count: %d)")
		, *GetExecutionContextName()
		, TransitionRange.Index
		, TransitionRange.GetLastIndex()
		, TransitionsView.Num());

	return TransitionsView.Slice(TransitionRange.Index, TransitionRange.Count);
}

TConstArrayView<FSceneStateTransition> FSceneStateExecutionContext::GetTransitions(const FSceneStateConduit& InConduit) const
{
	const FSceneStateRange TransitionRange = InConduit.GetTransitionRange();
	if (TransitionRange.Count == 0)
	{
		return {};
	}

	const TConstArrayView<FSceneStateTransition> TransitionsView = TemplateData->Transitions;

	checkfSlow(TransitionsView.IsValidIndex(TransitionRange.Index) && TransitionsView.IsValidIndex(TransitionRange.GetLastIndex())
		, TEXT("[Context: %s] Conduit Transition Range [%d, %d] is out of bounds! (Total Transition Count: %d)")
		, *GetExecutionContextName()
		, TransitionRange.Index
		, TransitionRange.GetLastIndex()
		, TransitionsView.Num());

	return TransitionsView.Slice(TransitionRange.Index, TransitionRange.Count);
}

FInstancedPropertyBag FSceneStateExecutionContext::GetTemplateTransitionParameter(const FSceneStateTransition& InTransition) const
{
	return TemplateData->TransitionParameters.FindRef(GetTransitionIndex(InTransition));
}

TConstArrayView<FSceneStateMachine> FSceneStateExecutionContext::GetStateMachines(const FSceneState& InState) const
{
	const FSceneStateRange StateMachineRange = InState.GetStateMachineRange();
	if (StateMachineRange.Count == 0)
	{
		return {};
	}

	const TConstArrayView<FSceneStateMachine> StateMachinesView = TemplateData->StateMachines;

	checkfSlow(StateMachinesView.IsValidIndex(StateMachineRange.Index) && StateMachinesView.IsValidIndex(StateMachineRange.GetLastIndex())
		, TEXT("[Context: %s] State State Machine Range [%d, %d] is out of bounds! (Total State Machine Count: %d)")
		, *GetExecutionContextName()
		, StateMachineRange.Index
		, StateMachineRange.GetLastIndex()
		, StateMachinesView.Num());

	return StateMachinesView.Slice(StateMachineRange.Index, StateMachineRange.Count);
}

TConstArrayView<FSceneStateEventHandler> FSceneStateExecutionContext::GetEventHandlers(const FSceneState& InState) const
{
	const FSceneStateRange EventHandlerRange = InState.GetEventHandlerRange();
	if (EventHandlerRange.Count == 0)
	{
		return {};
	}

	const TConstArrayView<FSceneStateEventHandler> EventHandlersView = TemplateData->EventHandlers;

	checkfSlow(EventHandlersView.IsValidIndex(EventHandlerRange.Index) && EventHandlersView.IsValidIndex(EventHandlerRange.GetLastIndex())
		, TEXT("[Context: %s] State Event Handler Range [%d, %d] is out of bounds! (Total Event Handler Count: %d)")
		, *GetExecutionContextName()
		, EventHandlerRange.Index
		, EventHandlerRange.GetLastIndex()
		, EventHandlersView.Num());

	return EventHandlersView.Slice(EventHandlerRange.Index, EventHandlerRange.Count);
}

#if WITH_EDITOR
const FSceneStateMetadata* FSceneStateExecutionContext::GetStateMetadata(const FSceneState& InState) const
{
	const uint16 StateIndex = GetStateIndex(InState);
	const TConstArrayView<FSceneStateMetadata> StateMetadataView = TemplateData->StateMetadata;

	if (!StateMetadataView.IsValidIndex(StateIndex))
	{
		UE_LOGF(LogSceneState, Error, "[Context: %ls] Metadata for State Index %d is not available!"
			"Be sure to have compiled the Scene State Object. Metadata count: %d"
			, *GetExecutionContextName()
			, StateIndex
			, StateMetadataView.Num());
		return nullptr;
	}

	return &StateMetadataView[StateIndex];
}
#endif

const FSceneStateMachine* FSceneStateExecutionContext::GetStateMachine(const FGuid& InStateMachineId) const
{
	return TemplateData->FindStateMachine(InStateMachineId);
}

const FSceneStateMachine* FSceneStateExecutionContext::GetStateMachine(uint16 InAbsoluteIndex) const
{
	const TConstArrayView<FSceneStateMachine> StateMachinesView = TemplateData->StateMachines;

	checkfSlow(StateMachinesView.IsValidIndex(InAbsoluteIndex)
		, TEXT("[Context: %s] Absolute Index %d is out of bounds! (Total State Machine Count: %d)")
		, *GetExecutionContextName()
		, InAbsoluteIndex
		, StateMachinesView.Num());

	return &StateMachinesView[InAbsoluteIndex];
}

const FSceneStateMachine* FSceneStateExecutionContext::GetContextStateMachine() const
{
	if (!StateMachineExecutionStack.IsEmpty())
	{
		return GetStateMachine(StateMachineExecutionStack.Last());
	}
	return nullptr;
}

const FSceneState* FSceneStateExecutionContext::GetActiveState(const FSceneStateMachine& InStateMachine) const
{
	if (const FSceneStateMachineInstance* Instance = FindStateMachineInstance(InStateMachine))
	{
		return GetState(InStateMachine, Instance->ActiveIndex);
	}
	return nullptr;
}

const FSceneState* FSceneStateExecutionContext::GetState(uint16 InAbsoluteIndex) const
{
	const TConstArrayView<FSceneState> StatesView = TemplateData->States;

	checkfSlow(StatesView.IsValidIndex(InAbsoluteIndex)
		, TEXT("[Context: %s] Absolute Index %d is out of bounds! (Total State Count: %d)")
		, *GetExecutionContextName()
		, InAbsoluteIndex
		, StatesView.Num());

	return &StatesView[InAbsoluteIndex];
}

const FSceneStateEventHandler* FSceneStateExecutionContext::GetEventHandler(uint16 InAbsoluteIndex) const
{
	const TConstArrayView<FSceneStateEventHandler> EventHandlers = TemplateData->EventHandlers;

	checkfSlow(EventHandlers.IsValidIndex(InAbsoluteIndex)
		, TEXT("[Context: %s] Absolute Index %d is out of bounds! (Total Event Handlers Count: %d)")
		, *GetExecutionContextName()
		, InAbsoluteIndex
		, EventHandlers.Num());

	return &EventHandlers[InAbsoluteIndex];
}

const FSceneState* FSceneStateExecutionContext::GetState(const FSceneStateMachine& InStateMachine, uint16 InRelativeIndex) const
{
	const FSceneStateRange StateRange = InStateMachine.GetStateRange();
	if (StateRange.Count == 0)
	{
		return nullptr;
	}

	const TConstArrayView<FSceneState> StatesView = TemplateData->States;

	const uint16 AbsoluteIndex = StateRange.Index + InRelativeIndex;

	checkfSlow(StatesView.IsValidIndex(AbsoluteIndex)
		, TEXT("[Context: %s] State Machine Relative Active Index %d from [%d, %d] is out of bounds! (Total State Count: %d)")
		, *GetExecutionContextName()
		, InRelativeIndex
		, StateRange.Index
		, StateRange.GetLastIndex()
		, StatesView.Num());

	return &StatesView[AbsoluteIndex];
}

const FSceneStateConduit* FSceneStateExecutionContext::GetConduit(const FSceneStateMachine& InStateMachine, uint16 InRelativeIndex) const
{
	const FSceneStateRange ConduitRange = InStateMachine.GetConduitRange();
	if (ConduitRange.Count == 0)
	{
		return nullptr;
	}

	const TConstArrayView<FSceneStateConduit> ConduitsView = TemplateData->Conduits;

	const uint16 AbsoluteIndex = ConduitRange.Index + InRelativeIndex;

	checkfSlow(ConduitsView.IsValidIndex(AbsoluteIndex)
		, TEXT("[Context: %s] State machine conduit relative index %d from [%d, %d] is out of bounds! (Total conduit count: %d)")
		, *GetExecutionContextName()
		, InRelativeIndex
		, ConduitRange.Index
		, ConduitRange.GetLastIndex()
		, ConduitsView.Num());

	return &ConduitsView[AbsoluteIndex];
}

FSceneStateInstance& FSceneStateExecutionContext::FindOrAddStateInstance(const FSceneState& InState) const
{
	return StateInstances.FindOrAdd(GetStateIndex(InState));
}

FSceneStateInstance* FSceneStateExecutionContext::FindStateInstance(uint16 InAbsoluteIndex) const
{
	return StateInstances.Find(InAbsoluteIndex);
}

FSceneStateInstance* FSceneStateExecutionContext::FindStateInstance(const FSceneState& InState) const
{
	return FindStateInstance(GetStateIndex(InState));
}

void FSceneStateExecutionContext::RemoveStateInstance(const FSceneState& InState) const
{
	StateInstances.Remove(GetStateIndex(InState));
}

void FSceneStateExecutionContext::ForEachStateInstance(TFunctionRef<void(uint16, const FSceneStateInstance&)> InFunc) const
{
	for (const TPair<uint16, FSceneStateInstance>& StateInstance : StateInstances)
	{
		InFunc(StateInstance.Key, StateInstance.Value);
	}
}

FConstStructView FSceneStateExecutionContext::FindTask(uint16 InAbsoluteIndex) const
{
	if (TemplateData->Tasks.IsValidIndex(InAbsoluteIndex))
	{
		return TemplateData->Tasks[InAbsoluteIndex];
	}
	return {};
}

uint16 FSceneStateExecutionContext::FindTaskRelativeIndex(const FSceneStateTask& InTask) const
{
	const uint16 StateIndex = InTask.GetParentStateIndex();

	const TConstArrayView<FSceneState> StatesView = TemplateData->States;

	checkf(StatesView.IsValidIndex(StateIndex)
		, TEXT("[Context: %s] Task Parent State Index %d is out of bounds! (Total State Count: %d)")
		, *GetExecutionContextName()
		, StateIndex
		, StatesView.Num());

	const FSceneState& State = StatesView[StateIndex];
	return InTask.GetTaskIndex() - State.GetTaskRange().Index;
}

FStructView FSceneStateExecutionContext::FindTaskInstance(const FSceneStateTask& InTask) const
{
	const uint16 StateIndex = InTask.GetParentStateIndex();

	FSceneStateInstance* const StateInstance = FindStateInstance(StateIndex);
	if (!StateInstance)
	{
		return {};
	}

	FInstancedStructContainer& TaskInstanceContainer = StateInstance->GetTaskInstances();

	const uint16 TaskRelativeIndex = FindTaskRelativeIndex(InTask);
	if (!TaskInstanceContainer.IsValidIndex(TaskRelativeIndex))
	{
		return {};
	}

	return TaskInstanceContainer[TaskRelativeIndex];
}

FStructView FSceneStateExecutionContext::FindTaskInstance(uint16 InTaskAbsoluteIndex) const
{
	const FInstancedStructContainer& Tasks = TemplateData->Tasks;
	if (!Tasks.IsValidIndex(InTaskAbsoluteIndex))
	{
		return {};
	}

	const FSceneStateTask& Task = Tasks[InTaskAbsoluteIndex].Get<const FSceneStateTask>();
	return FindTaskInstance(Task);
}

TOptional<UE::SceneState::EExecutionStatus> FSceneStateExecutionContext::GetTaskStatus(const FSceneStateTask& InTask) const
{
	if (FSceneStateInstance* const StateInstance = FindStateInstance(InTask.GetParentStateIndex()))
	{
		return StateInstance->GetTaskStatus(FindTaskRelativeIndex(InTask));
	}
	return {};
}

void FSceneStateExecutionContext::SetTaskStatus(const FSceneStateTask& InTask, UE::SceneState::EExecutionStatus InStatus) const
{
	if (FSceneStateInstance* const StateInstance = FindStateInstance(InTask.GetParentStateIndex()))
	{
		StateInstance->SetTaskStatus(FindTaskRelativeIndex(InTask), InStatus);
	}
}

FSceneStateMachineInstance* FSceneStateExecutionContext::FindOrAddStateMachineInstance(const FSceneStateMachine& InStateMachine) const
{
	return &StateMachineInstances.FindOrAdd(GetStateMachineIndex(InStateMachine));
}

FSceneStateMachineInstance* FSceneStateExecutionContext::FindStateMachineInstance(uint16 InAbsoluteIndex) const
{
	return StateMachineInstances.Find(InAbsoluteIndex);
}

FSceneStateMachineInstance* FSceneStateExecutionContext::FindStateMachineInstance(const FSceneStateMachine& InStateMachine) const
{
	return FindStateMachineInstance(GetStateMachineIndex(InStateMachine));
}

void FSceneStateExecutionContext::RemoveStateMachineInstance(const FSceneStateMachine& InStateMachine) const
{
	StateMachineInstances.Remove(GetStateMachineIndex(InStateMachine));
}

FSceneStateTransitionInstance* FSceneStateExecutionContext::FindOrAddTransitionInstance(const FSceneStateTransition& InTransition) const
{
	return &TransitionInstances.FindOrAdd(GetTransitionIndex(InTransition));
}

FSceneStateTransitionInstance* FSceneStateExecutionContext::FindTransitionInstance(uint16 InAbsoluteIndex) const
{
	return TransitionInstances.Find(InAbsoluteIndex);
}

FSceneStateTransitionInstance* FSceneStateExecutionContext::FindTransitionInstance(const FSceneStateTransition& InTransition) const
{
	return FindTransitionInstance(GetTransitionIndex(InTransition));
}

void FSceneStateExecutionContext::RemoveTransitionInstance(const FSceneStateTransition& InTransition) const
{
	TransitionInstances.Remove(GetTransitionIndex(InTransition));
}

FSceneStateConduitInstance* FSceneStateExecutionContext::FindOrAddConduitInstance(const FSceneStateConduit& InConduit) const
{
	return &ConduitInstances.FindOrAdd(GetConduitIndex(InConduit));
}

void FSceneStateExecutionContext::RemoveConduitInstance(const FSceneStateConduit& InConduit) const
{
	ConduitInstances.Remove(GetConduitIndex(InConduit));	
}

bool FSceneStateExecutionContext::SetupFunctionInstances(FPropertyBindingIndex16 InBindingsBatch) const
{
	if (!InBindingsBatch.IsValid())
	{
		// Normal to not have a valid binding batch.
		return false;
	}

	const FPropertyBindingCopyInfoBatch& CopyInfoBatch = TemplateData->BindingCollection.GetBatch(InBindingsBatch);

	if (!UE::SceneState::IsValidRange(CopyInfoBatch.PropertyFunctionsBegin, CopyInfoBatch.PropertyFunctionsEnd))
	{
		// Binding batch could be valid, but might not have functions bound.
		return false;
	}

	FSceneStateFunctionInstanceBatch& FunctionInstanceBatch = FunctionInstanceBatches.Add(InBindingsBatch);
	FunctionInstanceBatch.FunctionRange = FSceneStateRange::MakeBeginEndRange(CopyInfoBatch.PropertyFunctionsBegin.Get(), CopyInfoBatch.PropertyFunctionsEnd.Get());

	// Initialize the function instances to the templates in the template data
	{
		TArray<FConstStructView> TemplateFunctionInstances;
		TemplateFunctionInstances.Reserve(FunctionInstanceBatch.FunctionRange.Count);

		for (uint16 FunctionIndex = FunctionInstanceBatch.FunctionRange.Index; FunctionIndex <= FunctionInstanceBatch.FunctionRange.GetLastIndex(); ++FunctionIndex)
		{
			TemplateFunctionInstances.Add(TemplateData->FunctionInstances[FunctionIndex]);
		}

		FunctionInstanceBatch.FunctionInstances.Append(TemplateFunctionInstances);
	}

	// Store a map of the absolute indices to the struct views in the batch for quick access from a single index (rather than going through batch index -> relative index)
	for (uint16 FunctionIndex = FunctionInstanceBatch.FunctionRange.Index; FunctionIndex <= FunctionInstanceBatch.FunctionRange.GetLastIndex(); ++FunctionIndex)
	{
		const uint16 RelativeIndex = FunctionIndex - FunctionInstanceBatch.FunctionRange.Index;
		FunctionInstanceViews.Add(FunctionIndex, FunctionInstanceBatch.FunctionInstances[RelativeIndex]);
	}

	// Set up these functions as they could be bound to other functions
	for (uint16 FunctionIndex = FunctionInstanceBatch.FunctionRange.Index; FunctionIndex <= FunctionInstanceBatch.FunctionRange.GetLastIndex(); ++FunctionIndex)
	{
		if (const FSceneStateFunction* Function = TemplateData->Functions[FunctionIndex].GetPtr<const FSceneStateFunction>())
		{
			Function->Setup(*this);
		}
	}

	return true;
}

void FSceneStateExecutionContext::RemoveFunctionInstances(FPropertyBindingIndex16 InBindingsBatch) const
{
	if (!InBindingsBatch.IsValid())
	{
		return;
	}

	FSceneStateRange FunctionRange;
	if (FSceneStateFunctionInstanceBatch* Batch = FunctionInstanceBatches.Find(InBindingsBatch))
	{
		FunctionRange = Batch->FunctionRange;
	}
	else
	{
		return;
	}

	// Clean up the functions used resources (e.g. other function instance data)
	for (uint16 FunctionIndex = FunctionRange.Index; FunctionIndex <= FunctionRange.GetLastIndex(); ++FunctionIndex)
	{
		if (const FSceneStateFunction* Function = TemplateData->Functions[FunctionIndex].GetPtr<const FSceneStateFunction>())
		{
			Function->Cleanup(*this);
		}
	}

	// Remove the allocated instance data for each function
	for (uint16 FunctionIndex = FunctionRange.Index; FunctionIndex <= FunctionRange.GetLastIndex(); ++FunctionIndex)
	{
		FunctionInstanceViews.Remove(FunctionIndex);
	}
	FunctionInstanceBatches.Remove(InBindingsBatch);
}

FConstStructView FSceneStateExecutionContext::FindFunction(uint16 InAbsoluteIndex) const
{
	const FInstancedStructContainer& Functions = TemplateData->Functions;
	if (Functions.IsValidIndex(InAbsoluteIndex))
	{
		return Functions[InAbsoluteIndex];
	}
	return {};
}

FStructView FSceneStateExecutionContext::FindFunctionInstance(uint16 InAbsoluteIndex) const
{
	return FunctionInstanceViews.FindRef(InAbsoluteIndex);
}

void FSceneStateExecutionContext::SetStateStatus(const FSceneState& InState, FSceneStateInstance& InStateInstance, UE::SceneState::EExecutionStatus InStatus) const
{
	if (InStateInstance.GetStatus() == InStatus)
	{
		return;
	}

	InStateInstance.SetStatus(InStatus);

#if WITH_EDITOR
	if (RootObject)
	{
		if (TSharedPtr<UE::SceneState::FDebugger> Debugger = RootObject->GetDebugger())
		{
			const UE::SceneState::FDebugStateContext StateContext(GetStateIndex(InState), InStateInstance);
			Debugger->NotifyStateStatusChanged(StateContext, InStatus);
		}
	}
#endif
}

const FSceneStateExecutionContext* FSceneStateExecutionContext::FindParentContext() const
{
	if (TSharedPtr<UE::SceneState::FExecutionContextRegistry> ContextRegistry = ContextRegistryWeak.Pin())
	{
		return ContextRegistry->FindContext(ParentHandle);
	}
	return nullptr;
}

uint16 FSceneStateExecutionContext::GetStateIndex(const FSceneState& InState) const
{
	TConstArrayView<FSceneState> StatesView = TemplateData->States;
	const int32 Index = static_cast<int32>(&InState - StatesView.GetData());
	checkSlow(StatesView.IsValidIndex(Index) && Index >= 0 && Index <= TNumericLimits<uint16>::Max());
	return static_cast<uint16>(Index);
}

uint16 FSceneStateExecutionContext::GetStateMachineIndex(const FSceneStateMachine& InStateMachine) const
{
	TConstArrayView<FSceneStateMachine> StateMachinesView = TemplateData->StateMachines;
	const int32 Index = static_cast<int32>(&InStateMachine - StateMachinesView.GetData());
	checkSlow(StateMachinesView.IsValidIndex(Index) && Index >= 0 && Index <= TNumericLimits<uint16>::Max());
	return static_cast<uint16>(Index);
}

uint16 FSceneStateExecutionContext::GetTransitionIndex(const FSceneStateTransition& InTransition) const
{
	TConstArrayView<FSceneStateTransition> TransitionsView = TemplateData->Transitions;
	const int32 Index = static_cast<int32>(&InTransition - TransitionsView.GetData());
	checkSlow(TransitionsView.IsValidIndex(Index) && Index >= 0 && Index <= TNumericLimits<uint16>::Max());
	return static_cast<uint16>(Index);
}

uint16 FSceneStateExecutionContext::GetConduitIndex(const FSceneStateConduit& InConduit) const
{
	TConstArrayView<FSceneStateConduit> ConduitsView = TemplateData->Conduits;
	const int32 Index = static_cast<int32>(&InConduit - ConduitsView.GetData());
	checkSlow(ConduitsView.IsValidIndex(Index) && Index >= 0 && Index <= TNumericLimits<uint16>::Max());
	return static_cast<uint16>(Index);
}
