// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateExecutionContextRegistry.h"
#include "SceneStateExecutionContext.h"
#include "SceneStateLog.h"

namespace UE::SceneState
{

FExecutionContextHandle FExecutionContextRegistry::RegisterContext(const FSceneStateExecutionContext* InContext)
{
	check(InContext && !bIterating);

	UE_LOGF(LogSceneState, Verbose, "Registering Context [%p] '%ls'", InContext, *InContext->GetExecutionContextName());

	FExecutionContextHandle Handle;
	Handle.Id = NextHandleId++;

	// In the unlikely scenario that Id is back at 0, redo operation
	if (Handle.Id == 0)
	{
		Handle.Id = NextHandleId++;
	}

	const int32 HandleIndex = Handles.Add(Handle);
	const int32 ContextIndex = Contexts.Add(InContext);
	check(HandleIndex == ContextIndex);
	return Handle;
}

void FExecutionContextRegistry::UnregisterContext(const FSceneStateExecutionContext* InContext)
{
	check(InContext && !bIterating);

	UE_LOGF(LogSceneState, Verbose, "Unregistering Context [%p]", InContext);

	const int32 Index = Contexts.Find(InContext);
	check(Handles.IsValidIndex(Index));
	Contexts.RemoveAt(Index);
	Handles.RemoveAt(Index);
}

const FSceneStateExecutionContext* FExecutionContextRegistry::FindContext(FExecutionContextHandle InHandle) const
{
	const int32 Index = Handles.Find(InHandle);
	if (Index != INDEX_NONE)
	{
		const FSceneStateExecutionContext* Context = Contexts[Index];
		check(Context);
		return Context;
	}
	return nullptr;
}

FExecutionContextHandle FExecutionContextRegistry::FindHandle(const FSceneStateExecutionContext& InContext) const
{
	const int32 Index = Contexts.Find(&InContext);
	if (Index != INDEX_NONE)
	{
		FExecutionContextHandle Handle = Handles[Index];
		check(Handle.IsValid());
		return Handle;
	}
	return FExecutionContextHandle();
}

#if WITH_EDITOR
void FExecutionContextRegistry::ForEachExecutionContext(TFunctionRef<void(const FSceneStateExecutionContext&)> InFunctor)
{
	TGuardValue IterationGuard(bIterating, true);

	for (const FSceneStateExecutionContext* Context : Contexts)
	{
		check(Context);
		InFunctor(*Context);
	}
}
#endif

} // UE::SceneState
