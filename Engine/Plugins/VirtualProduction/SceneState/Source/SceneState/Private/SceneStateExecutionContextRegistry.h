// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "SceneStateExecutionContextHandle.h"
#include "Templates/SharedPointer.h"

struct FSceneStateExecutionContext;

namespace UE::SceneState
{

/** Holds all the contexts setup to the scene state object owning this registry */
class FExecutionContextRegistry : public TSharedFromThis<FExecutionContextRegistry>
{
public:
	/** Called on Context setup to add the given context to the registry */
	FExecutionContextHandle RegisterContext(const FSceneStateExecutionContext* InContext);

	/** Called on Context destructor to remove the given context from the registry */
	void UnregisterContext(const FSceneStateExecutionContext* InContext);

	/** Retrieves the context mapped to a given handle */
	const FSceneStateExecutionContext* FindContext(FExecutionContextHandle InHandle) const;

	/** Retrieves the handle mapped to the given context */
	FExecutionContextHandle FindHandle(const FSceneStateExecutionContext& InContext) const;

#if WITH_EDITOR
	/** Iterates each registered execution context. Used only for editor debug visualization */
	void ForEachExecutionContext(TFunctionRef<void(const FSceneStateExecutionContext&)> InFunctor);
#endif

private:
	/** Unique handles where each handle is mapped to a context at the same index in the context array */
	TArray<FExecutionContextHandle> Handles;

	/** Registered context where each context is mapped to a handle at the same index in the handles array */
	TArray<const FSceneStateExecutionContext*> Contexts;

	/** Id to the next handle to use for this registry */
	uint64 NextHandleId = 1;

	/** Flag to detect when contexts are unexpectedly being added to/removed from the registry while iterating the contexts */
	bool bIterating = false;
};

} // UE::SceneState
