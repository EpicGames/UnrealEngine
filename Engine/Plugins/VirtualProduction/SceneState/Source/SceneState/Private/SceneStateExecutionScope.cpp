// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateExecutionScope.h"
#include "SceneStateExecutionContext.h"

namespace UE::SceneState
{

FExecutionScope::FExecutionScope(const FSceneStateExecutionContext& InContext, const FSceneStateMachine& InStateMachine)
	: ExecutionContext(InContext)
{
	StateMachineIndex = ExecutionContext.GetStateMachineIndex(InStateMachine);
	ExecutionContext.StateMachineExecutionStack.Push(StateMachineIndex);
}

FExecutionScope::~FExecutionScope()
{
	const uint16 RemovedElement = ExecutionContext.StateMachineExecutionStack.Pop(EAllowShrinking::No);
	ensureMsgf(RemovedElement == StateMachineIndex, TEXT("State Machine Execution Stack expected to pop %d but instead popped %d"), StateMachineIndex, RemovedElement);
}

} // UE::SceneState
