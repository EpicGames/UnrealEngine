// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateGeneratedClass.h"
#include "SceneStateTemplateData.h"

#if WITH_EDITOR
#include "StructUtils/UserDefinedStruct.h"
#include "StructUtilsDelegates.h"
#endif

USceneStateGeneratedClass::USceneStateGeneratedClass()
{
#if WITH_EDITOR
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		OnObjectsReinstancedHandle = FCoreUObjectDelegates::OnObjectsReinstanced.AddUObject(this, &USceneStateGeneratedClass::OnObjectsReinstanced);
		OnStructsReinstancedHandle = UE::StructUtils::Delegates::OnUserDefinedStructReinstanced.AddUObject(this, &USceneStateGeneratedClass::OnStructsReinstanced);
	}
#endif
}

void USceneStateGeneratedClass::Link(FArchive& Ar, bool bInRelinkExistingProperties)
{
	Super::Link(Ar, bInRelinkExistingProperties);
	LinkData();
}

void USceneStateGeneratedClass::PurgeClass(bool bInRecompilingOnLoad)
{
	Super::PurgeClass(bInRecompilingOnLoad);
	if (TemplateData)
	{
		TemplateData->Reset();
	}
}

void USceneStateGeneratedClass::PostLoad()
{
	Super::PostLoad();
	LinkData();
	ResolveBindings();
}

void USceneStateGeneratedClass::BeginDestroy()
{
	Super::BeginDestroy();
}

void USceneStateGeneratedClass::LinkData()
{
	if (TemplateData && IsFullClass())
	{
		TemplateData->Link(this);
	}
}

void USceneStateGeneratedClass::ResolveBindings()
{
	if (TemplateData && IsFullClass())
	{
		TemplateData->ResolveBindings(this);
	}
}

bool USceneStateGeneratedClass::IsFullClass() const
{
#if WITH_EDITOR
	FNameBuilder ClassName(GetFName());
	FStringView ClassNameView = ClassName.ToView();
	return !ClassNameView.StartsWith(TEXT("SKEL_")) && !ClassNameView.StartsWith(TEXT("REINST_"));
#else
	return true;
#endif
}

#if WITH_EDITOR
void USceneStateGeneratedClass::OnObjectsReinstanced(const TMap<UObject*, UObject*>& InReplacementMap)
{
	if (TemplateData && IsFullClass())
	{
		TemplateData->OnObjectsReinstanced(this, InReplacementMap);
	}
}

void USceneStateGeneratedClass::OnStructsReinstanced(const UUserDefinedStruct& InStruct)
{
	if (TemplateData && IsFullClass())
	{
		TemplateData->OnStructsReinstanced(this, InStruct);
	}
}
#endif
