// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateInstance.h"

#include "SceneState.h"

namespace UE::SceneState::Private
{
	uint16 GetNextInstanceId()
	{
		static uint16 Id = 0;

		// Starts with 1
		uint16 NewId = ++Id;
		if (NewId == 0)
		{
			NewId = ++Id;
		}
		return NewId; 
	}

} // UE::SceneState::Private

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FSceneStateInstance::FSceneStateInstance()
	: InstanceId(UE::SceneState::Private::GetNextInstanceId())
{
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FSceneStateInstance::Setup(const FSceneState& InState)
{
	using namespace UE::SceneState;
	SetStatus(EExecutionStatus::NotStarted);
	Parameters = InState.GetParameters();
	TaskStatuses.Init(EExecutionStatus::NotStarted, InState.GetTaskRange().Count);
}

UE::SceneState::EExecutionStatus FSceneStateInstance::GetStatus() const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return Status;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FSceneStateInstance::SetStatus(UE::SceneState::EExecutionStatus InStatus)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Status = InStatus;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

uint16 FSceneStateInstance::GetInstanceId() const
{
	return InstanceId;
}

FInstancedPropertyBag& FSceneStateInstance::GetParameters()
{
	return Parameters;
}

FInstancedStructContainer& FSceneStateInstance::GetTaskInstances()
{
	return TaskInstances;
}

UE::SceneState::EExecutionStatus FSceneStateInstance::GetTaskStatus(uint16 InTaskRelativeIndex) const
{
	if (ensure(TaskStatuses.IsValidIndex(InTaskRelativeIndex)))
	{
		return TaskStatuses[InTaskRelativeIndex];
	}
	return UE::SceneState::EExecutionStatus::NotStarted;
}

void FSceneStateInstance::SetTaskStatus(uint16 InTaskRelativeIndex, UE::SceneState::EExecutionStatus InTaskStatus)
{
	if (ensure(TaskStatuses.IsValidIndex(InTaskRelativeIndex)))
	{
		TaskStatuses[InTaskRelativeIndex] = InTaskStatus;
	}
}
