// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateMachine.h"
#include "Conduit/SceneStateConduit.h"
#include "SceneState.h"
#include "SceneStateExecutionContext.h"
#include "SceneStateExecutionScope.h"
#include "SceneStateLog.h"
#include "SceneStateMachineInstance.h"
#include "SceneStateObject.h"
#include "SceneStateUtils.h"
#include "Transition/SceneStateTransition.h"
#include "Transition/SceneStateTransitionEvaluation.h"

void FSceneStateMachine::Setup(const FSceneStateExecutionContext& InContext) const
{
	using namespace UE::SceneState;

	if (!IsValid())
	{
		return;
	}

	// Start State Machine, add a State Machine Instance if not already present
	FSceneStateMachineInstance* Instance = InContext.FindOrAddStateMachineInstance(*this);
	if (!Instance || Instance->Status == EExecutionStatus::Running)
	{
		return;
	}

	InContext.SetupFunctionInstances(BindingsBatch);

	// Initialize Instance
	Instance->ActiveIndex = EntryIndex;
	Instance->Parameters = Parameters;
}

void FSceneStateMachine::Start(const FSceneStateExecutionContext& InContext) const
{
	using namespace UE::SceneState;

	if (!IsValid())
	{
		return;
	}

	// Start State Machine, add a State Machine Instance if not already present
	FSceneStateMachineInstance* Instance = InContext.FindStateMachineInstance(*this);
	if (!Instance || Instance->Status == EExecutionStatus::Running)
	{
		return;
	}

	ApplyBindings(InContext, *Instance);

	const FSceneState* ActiveState = InContext.GetActiveState(*this);
	if (!ActiveState)
	{
		UE_LOGF(LogSceneState, Error, "State Machine could not run because the Active State (Index: %d) is invalid!", Instance->ActiveIndex);
		Stop(InContext);
		return;
	}

	Instance->Status = EExecutionStatus::Running;

	{
		const FExecutionScope ExecutionScope(InContext, *this);
		ActiveState->Enter(InContext);
	}

	EvaluateStateTransitions(InContext, *Instance, *ActiveState);
}

void FSceneStateMachine::Tick(const FSceneStateExecutionContext& InContext, float InDeltaSeconds) const
{
	using namespace UE::SceneState;

	if (!IsValid())
	{
		return;
	}

	FSceneStateMachineInstance* Instance = InContext.FindStateMachineInstance(*this);
	if (!Instance || Instance->Status != EExecutionStatus::Running)
	{
		return;
	}

	const FSceneState* ActiveState = InContext.GetActiveState(*this);
	if (!ActiveState)
	{
		UE_LOGF(LogSceneState, Error, "State Machine could not run because the Active State (Index: %d) is invalid!", Instance->ActiveIndex);
		Stop(InContext);
		return;
	}

	{
		const FExecutionScope ExecutionScope(InContext, *this);
		ActiveState->Tick(InContext, InDeltaSeconds);
	}

	EvaluateStateTransitions(InContext, *Instance, *ActiveState);
}

void FSceneStateMachine::Stop(const FSceneStateExecutionContext& InContext) const
{
	using namespace UE::SceneState;

	if (!IsValid())
	{
		return;
	}

	FSceneStateMachineInstance* Instance = InContext.FindStateMachineInstance(*this);
	if (!Instance || Instance->Status != EExecutionStatus::Running)
	{
		return;
	}

	Instance->Status = EExecutionStatus::Finished;

	StopActiveState(InContext, *Instance);

	InContext.RemoveFunctionInstances(BindingsBatch);
	InContext.RemoveStateMachineInstance(*this);
}

void FSceneStateMachine::EvaluateStateTransitions(const FSceneStateExecutionContext& InContext, FSceneStateMachineInstance& InInstance, const FSceneState& InActiveState) const
{
	using namespace UE::SceneState;

	const FSceneStateTransition* Transition;
	{
		const FTransitionEvaluationParams EvaluationParams
			{
				.ExecutionContext = InContext,
				.StateMachine = *this,
				.StateMachineInstance = InInstance,
				.SourceState = InActiveState
			};

		const FExecutionScope ExecutionScope(InContext, *this);
		Transition = EvaluateTransitions(InContext.GetTransitions(InActiveState), EvaluationParams);
	}

	if (!Transition)
	{
		return;
	}

	const FSceneStateTransitionTarget TransitionTarget = Transition->GetTarget();

	// Conduit transition targets should've been followed until a state/exit was reached
	check(TransitionTarget.Type != ESceneStateTransitionTargetType::Conduit);

	if (TransitionTarget.Type == ESceneStateTransitionTargetType::Exit)
	{
		Stop(InContext);
		return;
	}

	check(TransitionTarget.Type == ESceneStateTransitionTargetType::State);
	StopActiveState(InContext, InInstance);

	InInstance.ActiveIndex = TransitionTarget.Index;
	const FSceneState* NewActiveState = InContext.GetActiveState(*this);
	if (!NewActiveState)
	{
		UE_LOGF(LogSceneState, Error, "State Machine did not transition to a new State! Transition State (Index: %d) is not a valid index!", TransitionTarget.Index);
		Stop(InContext);
		return;
	}

	const FExecutionScope ExecutionScope(InContext, *this);
	NewActiveState->Enter(InContext);
}

const FSceneStateTransition* FSceneStateMachine::EvaluateTransitions(TConstArrayView<FSceneStateTransition> InTransitions, const UE::SceneState::FTransitionEvaluationParams& InEvaluationParams) const
{
	// Return the first transition that succeeds.
	// the transitions are compiled and sorted by priority ahead of time.
	// So it is guaranteed that the first transition that succeeds is the highest priority that will succeed in this pass
	for (const FSceneStateTransition& Transition : InTransitions)
	{
		// Skip targets that will transition back to the active state index
		// This can happen only when there's a conduit that circles back to the active state
		const FSceneStateTransitionTarget Target = Transition.GetTarget();
		if (Target.Type == ESceneStateTransitionTargetType::State && Target.Index == InEvaluationParams.StateMachineInstance.ActiveIndex)
		{
			continue;
		}

		// Check if the transitions conditions are met
		if (!Transition.Evaluate(InEvaluationParams))
		{
			continue;
		}

		// Follow the transition to determine if it would lead to a valid state being selected or state machine exiting.
		if (const FSceneStateTransition* TransitionResult = FollowTransition(Transition, InEvaluationParams))
		{
			return TransitionResult;
		}
	}

	return nullptr;
}

const FSceneStateTransition* FSceneStateMachine::FollowTransition(const FSceneStateTransition& InTransition, const UE::SceneState::FTransitionEvaluationParams& InEvaluationParams) const
{
	const FSceneStateTransitionTarget TransitionTarget = InTransition.GetTarget();

	// Both state and exit types are immediately valid connections. Return the same transition
	if (TransitionTarget.Type != ESceneStateTransitionTargetType::Conduit)
	{
		return &InTransition;
	}

	// Evaluate the conduit, and return null if it doesn't, to give opportunity to other exit transitions to be evaluated
	const FSceneStateConduit* Conduit = InEvaluationParams.ExecutionContext.GetConduit(*this, TransitionTarget.Index);
	if (!Conduit || !Conduit->Evaluate(InEvaluationParams))
	{
		return nullptr;
	}

	// Conduit passed, evaluate the conduit's exit transitions.
	// Can return null to give opportunity to other exit transitions of the state to be evaluated
	return EvaluateTransitions(InEvaluationParams.ExecutionContext.GetTransitions(*Conduit), InEvaluationParams);
}

bool FSceneStateMachine::ApplyBindings(const FSceneStateExecutionContext& InContext, FSceneStateMachineInstance& InStateMachineInstance) const
{
	QUICK_SCOPE_CYCLE_COUNTER(SceneStateMachine_ApplyBindings);

	const UE::SceneState::FApplyBatchParams ApplyBatchParams
		{
			.BindingsBatch = BindingsBatch.Get(),
			.TargetDataView = InStateMachineInstance.Parameters.GetMutableValue(),
		};

	return ApplyBatch(InContext, ApplyBatchParams);
}

void FSceneStateMachine::StopActiveState(const FSceneStateExecutionContext& InContext, FSceneStateMachineInstance& InInstance) const
{
	if (const FSceneState* ActiveState = InContext.GetActiveState(*this))
	{
		const UE::SceneState::FExecutionScope ExecutionScope(InContext, *this);
		ActiveState->Exit(InContext);
	}
}
