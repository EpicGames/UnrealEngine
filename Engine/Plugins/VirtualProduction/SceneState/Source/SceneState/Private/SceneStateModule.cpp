// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "SceneStateLog.h"

DEFINE_LOG_CATEGORY(LogSceneState);

IMPLEMENT_MODULE(FDefaultModuleImpl, SceneState)
