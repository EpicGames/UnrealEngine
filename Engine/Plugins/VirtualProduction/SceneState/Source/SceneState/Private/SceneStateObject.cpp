// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateObject.h"
#include "Debug/SceneStateDebugger.h"
#include "PropertyBindingDataView.h"
#include "SceneStateEvent.h"
#include "SceneStateEventStream.h"
#include "SceneStateExecutionContextRegistry.h"
#include "SceneStateGeneratedClass.h"
#include "SceneStateLog.h"
#include "SceneStatePlayer.h"
#include "SceneStateTemplateData.h"

USceneStateObject::USceneStateObject()
{
	ContextRegistry = MakeShared<UE::SceneState::FExecutionContextRegistry>();

	EventStream = CreateDefaultSubobject<USceneStateEventStream>(TEXT("EventStream"));
}

FString USceneStateObject::GetContextName() const
{
	if (USceneStatePlayer* Player = Cast<USceneStatePlayer>(GetOuter()))
	{
		return Player->GetContextName();
	}
	return FString();
}

UObject* USceneStateObject::GetContextObject() const
{
	if (USceneStatePlayer* Player = Cast<USceneStatePlayer>(GetOuter()))
	{
		return Player->GetContextObject();
	}
	return nullptr;
}

bool USceneStateObject::IsActive() const
{
	if (const FSceneState* RootState = GetRootState())
	{
		const FSceneStateInstance* RootStateInstance = RootExecutionContext.FindStateInstance(*RootState);
		return RootStateInstance && RootStateInstance->GetStatus() == UE::SceneState::EExecutionStatus::Running;
	}
	return false;
}

void USceneStateObject::Setup()
{
	TemplateData = nullptr;

	if (const USceneStateGeneratedClass* GeneratedClass = Cast<USceneStateGeneratedClass>(GetClass()))
	{
		TemplateData = GeneratedClass->GetTemplateData();
	}

	if (!TemplateData)
	{
		UE_LOGF(LogSceneState, Warning, "[%ls] did not find a valid template data object. Scene State will not execute in this instance.", *GetFullName());
		return;
	}

	RootExecutionContext.Setup(this);
}

void USceneStateObject::Enter()
{
	if (const FSceneState* RootState = GetRootState())
	{
		if (EventStream)
		{
			EventStream->Register();
		}

		ReceiveEnter();
		RootState->Enter(RootExecutionContext);
	}
}

void USceneStateObject::Tick(float InDeltaSeconds)
{
	if (const FSceneState* RootState = GetRootState())
	{
		ReceiveTick(InDeltaSeconds);
		RootState->Tick(RootExecutionContext, InDeltaSeconds);
	}
}

void USceneStateObject::Exit()
{
	if (const FSceneState* RootState = GetRootState())
	{
		ReceiveExit();
		RootState->Exit(RootExecutionContext);
	}

	if (EventStream)
	{
		EventStream->Unregister();
	}

	TemplateData = nullptr;
	RootExecutionContext.Reset();

#if WITH_EDITOR
	SetDebugger(nullptr);
#endif
}

TSharedRef<UE::SceneState::FExecutionContextRegistry> USceneStateObject::GetContextRegistry() const
{
	return ContextRegistry.ToSharedRef();
}

#if WITH_EDITOR
TSharedPtr<UE::SceneState::FDebugger> USceneStateObject::GetDebugger() const
{
	return DebuggerWeak.Pin();
}

void USceneStateObject::SetDebugger(const TWeakPtr<UE::SceneState::FDebugger>& InDebuggerWeak)
{
	if (InDebuggerWeak == DebuggerWeak)
	{
		return;
	}

	if (TSharedPtr<UE::SceneState::FDebugger> Debugger = GetDebugger())
	{
		Debugger->Detach();
	}
	DebuggerWeak = InDebuggerWeak;
}
#endif

UWorld* USceneStateObject::GetWorld() const
{
	if (UObject* Context = GetContextObject())
	{
		return Context->GetWorld();
	}
	return nullptr;
}

void USceneStateObject::BeginDestroy()
{
	Super::BeginDestroy();
	TemplateData = nullptr;
    RootExecutionContext.Reset();
}

const FSceneState* USceneStateObject::GetRootState() const
{
	if (TemplateData)
	{
		return TemplateData->GetRootState();
	}
	return nullptr;
}
