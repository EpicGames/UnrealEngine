// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStatePlayer.h"
#include "SceneState.h"
#include "SceneStateMachine.h"
#include "SceneStateObject.h"
#include "SceneStateUtils.h"

void USceneStatePlayer::SetSceneStateClass(TSubclassOf<USceneStateObject> InSceneStateClass)
{
	// Skip object replacement if Root State class is already updated
	if (RootState && RootState->GetClass() == InSceneStateClass)
	{
		ensure(SceneStateClass == RootState->GetClass());
		return;
	}

	UObject* TempState = RootState;

	bool bResult = UE::SceneState::ReplaceObject(TempState, this, InSceneStateClass, TEXT("RootState"), TEXT("SceneStatePlayer"),
		[](UObject* InOldObject)
		{
			if (USceneStateObject* OldSceneState = Cast<USceneStateObject>(InOldObject))
			{
				OldSceneState->Exit();
			}
		});

	if (bResult)
	{
		RootState = Cast<USceneStateObject>(TempState);
		if (RootState)
		{
			SceneStateClass = RootState->GetClass();
			Setup();
		}
		else
		{
			SceneStateClass = nullptr;
		}
	}
}

FString USceneStatePlayer::GetContextName() const
{
	FString ContextName;
	if (OnGetContextName(ContextName))
	{
		return ContextName;
	}
	return GetName();
}

UObject* USceneStatePlayer::GetContextObject() const
{
	UObject* ContextObject;
	if (OnGetContextObject(ContextObject))
	{
		return ContextObject;
	}
	return GetOuter();
}

void USceneStatePlayer::Setup()
{
	if (RootState)
	{
		RootState->Setup();
	}
}

void USceneStatePlayer::Begin()
{
	if (RootState)
	{
		RootState->Enter();
	}
}

void USceneStatePlayer::Tick(float InDeltaTime)
{
	if (RootState)
	{
		RootState->Tick(InDeltaTime);
	}
}

void USceneStatePlayer::End()
{
	if (RootState)
	{
		RootState->Exit();
	}
}

#if WITH_EDITOR
void USceneStatePlayer::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	if (InPropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(USceneStatePlayer, SceneStateClass))
	{
		SetSceneStateClass(SceneStateClass);
	}
}
#endif
