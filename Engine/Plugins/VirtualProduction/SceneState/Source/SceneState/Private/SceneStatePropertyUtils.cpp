// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStatePropertyUtils.h"
#include "PropertyBindingDataView.h"
#include "SceneStateBindingCollection.h"
#include "SceneStateExecutionContext.h"

namespace UE::SceneState
{

namespace Private
{

/** Resolves a property reference without recursing/iterating further if the source of that reference is another property reference */
bool ResolvePropertyInternal(const FSceneStateExecutionContext& InContext, const FSceneStatePropertyReference& InPropertyReference, FResolvePropertyResult& OutResult)
{
	const FSceneStateBindingCollection& BindingCollection = InContext.GetBindingCollection();

	const FSceneStateBindingResolvedReference* ResolvedReference = BindingCollection.FindResolvedReference(InPropertyReference);
	if (!ResolvedReference || !ResolvedReference->SourceLeafProperty)
	{
		return false;
	}

	const FPropertyBindingDataView SourceDataView = InContext.FindDataView(ResolvedReference->SourceDataHandle);
	if (!SourceDataView.IsValid())
	{
		return false;
	}

	uint8* const ValuePtr = BindingCollection.ResolveProperty(*ResolvedReference, SourceDataView);
	if (!ValuePtr)
	{
		return false;
	}

	OutResult.ValuePtr = ValuePtr;
	OutResult.ResolvedReference = ResolvedReference;
	return true;
}

} // UE::SceneState::Private
	
bool ResolveProperty(const FSceneStateExecutionContext& InContext, const FSceneStatePropertyReference& InPropertyReference, FResolvePropertyResult& OutResult)
{
	FResolvePropertyResult Result;
	if (!Private::ResolvePropertyInternal(InContext, InPropertyReference, Result))
	{
		return false;
	}

	constexpr int32 MaxIterations = 100;
	int32 IterationCount = 0;

	// If the Source is a Property Reference, follow the reference chain until a non-property reference is found
	while (IsPropertyReference(Result.ResolvedReference->SourceLeafProperty))
	{
		// Safe max iteration limit hit. Could this be a reference loop?
		if (!ensureMsgf(++IterationCount <= MaxIterations, TEXT("Resolve Property Reference failed as it hit the iteration limit. Could this be a reference loop?")))
		{
			return false;
		}

		const FSceneStatePropertyReference& SourcePropertyReference = *reinterpret_cast<FSceneStatePropertyReference*>(Result.ValuePtr);
		if (!Private::ResolvePropertyInternal(InContext, SourcePropertyReference, Result))
		{
			return false;
		}
	}

	OutResult = MoveTemp(Result);
	return true;
}

} // UE::SceneState
