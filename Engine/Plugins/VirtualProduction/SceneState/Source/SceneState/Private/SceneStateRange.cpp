// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateRange.h"

FSceneStateRange FSceneStateRange::MakeBeginEndRange(uint16 InBegin, uint16 InEnd)
{
	if (InBegin == InvalidIndex || InEnd == InvalidIndex || InBegin >= InEnd)
	{
		return FSceneStateRange();
	}

	FSceneStateRange Range;
	Range.Index = InBegin;
	Range.Count = InEnd - InBegin;
	return Range;
}
