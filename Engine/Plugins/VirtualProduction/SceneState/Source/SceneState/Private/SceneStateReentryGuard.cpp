// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateReentryGuard.h"
#include "Misc/AssertionMacros.h"
#include "SceneStateExecutionContext.h"
#include "SceneStateLog.h"

namespace UE::SceneState
{

FReentryGuard::FReentryGuard(const FReentryHandle& InCurrent, const FSceneStateExecutionContext& InContext)
	: Reference(InCurrent)
	, Original(InCurrent)
{
#if !NO_LOGGING
	if (IsReentry())
	{
		UE_LOGF(LogSceneState, Error, "[%ls] Re-entry detected!", *InContext.GetExecutionContextName());
		FDebug::DumpStackTraceToLog(TEXT("=== FSceneStateReentryGuard::DumpStackTrace(): ==="), ELogVerbosity::Error);
	}
#endif
	Reference.bValue = true;
}

FReentryGuard::~FReentryGuard()
{
	Reference.bValue = Original.bValue;
}

bool FReentryGuard::IsReentry() const
{
	return Original.bValue;
}

} // UE::SceneState
