// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateSchema.h"
#include "Tasks/SceneStateTask.h"

#if WITH_EDITOR
bool USceneStateSchema::IsTaskStructAllowed(TSubScriptStructOf<FSceneStateTask> InTaskStruct) const
{
	return OnIsTaskStructAllowed(InTaskStruct);
}

bool USceneStateSchema::IsTaskAssetAllowed(const FAssetData& InTaskAsset) const
{
	return OnIsTaskAssetAllowed(InTaskAsset);
}
#endif

TSubclassOf<UObject> USceneStateSchema::GetContextObjectClass() const
{
	if (!ContextObjectClass)
	{
		ContextObjectClass = OnGetContextObjectClass();
	}
	return ContextObjectClass;
}
