// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateTemplateData.h"
#include "Misc/EnumerateRange.h"
#include "SceneStateBindingUtils.h"
#include "SceneStateExecutionContext.h"
#include "SceneStateExecutionContextRegistry.h"
#include "SceneStateLog.h"
#include "StructUtils/UserDefinedStruct.h"
#include "Tasks/SceneStateTask.h"
#include "Tasks/SceneStateTaskBindingExtension.h"
#include "Tasks/SceneStateTaskInstance.h"

const FSceneState* USceneStateTemplateData::GetRootState() const
{
	if (States.IsValidIndex(RootStateIndex))
	{
		return &States[RootStateIndex];	
	}
	return nullptr;
}

const FSceneStateMachine* USceneStateTemplateData::FindStateMachine(const FGuid& InStateMachineId) const
{
	const uint16* StateMachineIndex = StateMachineIdToIndex.Find(InStateMachineId);
	if (StateMachineIndex && StateMachines.IsValidIndex(*StateMachineIndex))
	{
		return &StateMachines[*StateMachineIndex];
	}
	return nullptr;
}

const UStruct* USceneStateTemplateData::FindDataStruct(const UClass* InOwnerClass, const FSceneStateBindingDataHandle& InDataHandle)
{
	const uint16 DataIndex = InDataHandle.GetDataIndex();
	const uint16 DataSubIndex = InDataHandle.GetDataSubIndex();

	if (InDataHandle.IsExternalDataType())
	{
		// Todo: unsupported external sources
		return nullptr;
	}

	const ESceneStateDataType DataType = static_cast<ESceneStateDataType>(InDataHandle.GetDataType());

	switch (DataType)
	{
	case ESceneStateDataType::Root:
		return InOwnerClass;

	case ESceneStateDataType::Task:
		return TaskInstances[DataIndex].GetScriptStruct();

	case ESceneStateDataType::EventHandler:
		return EventHandlers[DataIndex].GetEventSchemaHandle().GetEventStruct();

	case ESceneStateDataType::TaskExtension:
		if (const FSceneStateTask* Task = Tasks[DataIndex].GetPtr<const FSceneStateTask>())
		{
			if (const FSceneStateTaskBindingExtension* BindingExtension = Task->GetBindingExtension())
			{
				FStructView DataView;
				if (BindingExtension->FindDataByIndex(TaskInstances[DataIndex], DataSubIndex, DataView))
				{
					return DataView.GetScriptStruct();
				}
			}
		}
		break;

	case ESceneStateDataType::Transition:
		return TransitionParameters[DataIndex].GetPropertyBagStruct();

	case ESceneStateDataType::StateMachine:
		return StateMachines[DataIndex].GetParametersStruct();

	case ESceneStateDataType::Function:
		return FunctionInstances[DataIndex].GetScriptStruct();

	case ESceneStateDataType::State:
		return States[DataIndex].GetParametersStruct();
	}

	return nullptr;
}

void USceneStateTemplateData::ResolveBindings(const UClass* InOwnerClass)
{
	UE::SceneState::PatchBindingCollection(
		{
			.BindingCollection = BindingCollection,
			.FindDataStructFunctor = [This=this, InOwnerClass](const FSceneStateBindingDataHandle& InDataHandle)->const UStruct*
			{
				return This->FindDataStruct(InOwnerClass, InDataHandle);
			}
		});

	// Resolves property paths used by bindings a store property pointers
	if (!BindingCollection.ResolvePaths())
	{
		UE_LOGF(LogSceneState, Warning, "Failed to resolve bindings. Try compiling the template data again.");
	}
}

void USceneStateTemplateData::Link(const UClass* InOwnerClass)
{
	for (TEnumerateRef<FSceneStateTransition> Transition : EnumerateRange(Transitions))
	{
		if (TransitionLinks.IsValidIndex(Transition.GetIndex()))
		{
			Transition->Link(TransitionLinks[Transition.GetIndex()], InOwnerClass);
		}
	}

	for (TEnumerateRef<FSceneStateConduit> Conduit : EnumerateRange(Conduits))
	{
		if (ConduitLinks.IsValidIndex(Conduit.GetIndex()))
		{
			Conduit->Link(ConduitLinks[Conduit.GetIndex()], InOwnerClass);
		}
	}
}

void USceneStateTemplateData::Reset()
{
	Tasks.Reset();
	TaskPrerequisites.Reset();
	TaskInstances.Reset();
	Functions.Reset();
	FunctionInstances.Reset();
	States.Reset();
	Conduits.Reset();
	ConduitLinks.Reset();
	EventHandlers.Reset();
	Transitions.Reset();
	TransitionLinks.Reset();
	TransitionParameters.Reset();
	StateMachines.Reset();
	BindingCollection.Reset();
	StateMachineIdToIndex.Reset();

#if WITH_EDITOR
	TaskMetadata.Reset();
	FunctionMetadata.Reset();
	StateMetadata.Reset();
	TransitionMetadata.Reset();
	StateNodeToIndex.Reset();
	StateMachineGraphToIndex.Reset();
	TaskNodeToIndex.Reset();
#endif
}

#if WITH_EDITOR
const FSceneState* USceneStateTemplateData::FindStateFromNode(FObjectKey InStateNode) const
{
	const uint16* StateIndex = StateNodeToIndex.Find(InStateNode);
	if (StateIndex && States.IsValidIndex(*StateIndex))
	{
		return &States[*StateIndex];
	}
	return nullptr;
}

const FSceneStateTask* USceneStateTemplateData::FindTaskFromNode(FObjectKey InTaskNode) const
{
	const uint16* TaskIndex = TaskNodeToIndex.Find(InTaskNode);
	if (TaskIndex && Tasks.IsValidIndex(*TaskIndex))
	{
		return Tasks[*TaskIndex].GetPtr<const FSceneStateTask>();
	}
	return nullptr;
}

void USceneStateTemplateData::ForEachStateInstance(const TSharedRef<UE::SceneState::FExecutionContextRegistry>& InContextRegistry, FObjectKey InStateNode, TFunctionRef<void(const FSceneStateInstance&)> InFunctor) const
{
}

void USceneStateTemplateData::ForEachTaskStatus(const TSharedRef<UE::SceneState::FExecutionContextRegistry>& InContextRegistry, uint16 InTaskIndex, TFunctionRef<void(UE::SceneState::EExecutionStatus)> InFunctor) const
{
	if (!Tasks.IsValidIndex(InTaskIndex))
	{
		return;
	}

	const FSceneStateTask& Task = Tasks[InTaskIndex].Get<const FSceneStateTask>();
	InContextRegistry->ForEachExecutionContext(
		[&Task, &InFunctor](const FSceneStateExecutionContext& InExecutionContext)
		{
			const TOptional<UE::SceneState::EExecutionStatus> TaskStatus = InExecutionContext.GetTaskStatus(Task);
			if (TaskStatus.IsSet())
			{
				InFunctor(*TaskStatus);
			}
		});
}

void USceneStateTemplateData::ForEachTaskInstance(const TSharedRef<UE::SceneState::FExecutionContextRegistry>& InContextRegistry, FObjectKey InTaskNode, TFunctionRef<void(const FSceneStateTaskInstance&)> InFunctor) const
{
}

void USceneStateTemplateData::OnObjectsReinstanced(const UClass* InOwnerClass, const TMap<UObject*, UObject*>& InReplacementMap)
{
	TSet<const UStruct*> Structs;
	Structs.Reserve(InReplacementMap.Num());

	bool bRequiresResolve = false;

	for (const TPair<UObject*, UObject*>& Pair : InReplacementMap)
	{
		if (const UObject* Replacement = Pair.Value)
		{
			if (Replacement->IsIn(InOwnerClass))
			{
				bRequiresResolve = true;
				break;
			}
			Structs.Add(Replacement->GetClass());
		}
	}

	if (bRequiresResolve || BindingCollection.ContainsAnyStruct(Structs))
	{
		ResolveBindings(InOwnerClass);
	}
}

void USceneStateTemplateData::OnStructsReinstanced(const UClass* InOwnerClass, const UUserDefinedStruct& InStruct)
{
	TSet<const UStruct*> Structs;
	Structs.Add(&InStruct);

	if (BindingCollection.ContainsAnyStruct(Structs))
	{
		ResolveBindings(InOwnerClass);
	}
}
#endif
