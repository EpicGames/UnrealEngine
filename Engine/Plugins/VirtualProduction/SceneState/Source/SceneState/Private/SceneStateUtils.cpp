// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateUtils.h"
#include "Engine/Engine.h"
#include "Functions/SceneStateFunction.h"
#include "PropertyBindingPath.h"
#include "PropertyBindingTypes.h"
#include "SceneStateBindingCollection.h"
#include "SceneStateExecutionContext.h"
#include "SceneStateLog.h"
#include "SceneStateRange.h"
#include "StructUtils/InstancedStructContainer.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/Package.h"
#include "UObject/PropertyVisitor.h"
#include "UObject/UnrealType.h"

namespace UE::SceneState
{

TArray<FStructView> GetStructViews(FInstancedStructContainer& InStructContainer, FSceneStateRange InRange)
{
	if (InRange.Count == 0)
	{
		return {};
	}

	if (!InStructContainer.IsValidIndex(InRange.Index) || !InStructContainer.IsValidIndex(InRange.GetLastIndex()))
	{
		UE_LOGF(LogSceneState, Error, "GetStructViews failed. Range [%d, %d] out of bounds. Struct Container Num: %d"
			, InRange.Index
			, InRange.GetLastIndex()
			, InStructContainer.Num());
		return {};
	}

	TArray<FStructView> StructViews;
	StructViews.Reserve(InRange.Count);

	for (int32 Index = InRange.Index; Index <= InRange.GetLastIndex(); ++Index)
	{
		StructViews.Add(InStructContainer[Index]);
	}

	return StructViews;
}

TArray<FStructView> GetStructViews(FInstancedStructContainer& InStructContainer)
{
	FSceneStateRange Range;
	Range.Index = 0;
	Range.Count = InStructContainer.Num();
	return GetStructViews(InStructContainer, Range);
}

TArray<FConstStructView> GetConstStructViews(const FInstancedStructContainer& InStructContainer, FSceneStateRange InRange)
{
	FInstancedStructContainer& StructContainer = const_cast<FInstancedStructContainer&>(InStructContainer);
	return TArray<FConstStructView>(GetStructViews(StructContainer, InRange));
}

TArray<FConstStructView> GetConstStructViews(const FInstancedStructContainer& InStructContainer)
{
	FSceneStateRange Range;
	Range.Index = 0;
	Range.Count = InStructContainer.Num();
	return GetConstStructViews(InStructContainer, Range);
}

bool IsValidRange(FPropertyBindingIndex16 InBegin, FPropertyBindingIndex16 InEnd)
{
	return InBegin.IsValid() && InEnd.IsValid() && InBegin.Get() < InEnd.Get();
}

bool ApplyBatch(const FSceneStateExecutionContext& InContext, const FApplyBatchParams& InParams)
{
	const FPropertyBindingIndex16 BindingsBatch(InParams.BindingsBatch);
	if (!BindingsBatch.IsValid())
	{
		// Normal behavior if object is not a target of any binding, and so does not have a binding batch.
		return false;
	}

	const FSceneStateBindingCollection& BindingCollection = InContext.GetBindingCollection();

	bool bResult = true;

	const FPropertyBindingCopyInfoBatch& Batch = BindingCollection.GetBatch(BindingsBatch);
	check(InParams.TargetDataView.GetStruct() == Batch.TargetStruct.Get().Struct);

	// If there were valid functions found on setup, execute them now
	if (UE::SceneState::IsValidRange(Batch.PropertyFunctionsBegin, Batch.PropertyFunctionsEnd))
	{
		const FSceneStateRange FunctionRange = FSceneStateRange::MakeBeginEndRange(Batch.PropertyFunctionsBegin.Get(), Batch.PropertyFunctionsEnd.Get());

		for (uint16 FunctionIndex = FunctionRange.Index; FunctionIndex <= FunctionRange.GetLastIndex(); ++FunctionIndex)
		{
			if (const FSceneStateFunction* Function = InContext.FindFunction(FunctionIndex).GetPtr<const FSceneStateFunction>())
			{
				FStructView FunctionInstance = InContext.FindFunctionInstance(FunctionIndex);
				Function->Execute(InContext, FunctionInstance);
			}
		}
	}

	for (const FPropertyBindingCopyInfo& Copy : BindingCollection.GetBatchCopies(Batch))
	{
		const FPropertyBindingDataView SourceView = InContext.FindDataView(Copy.SourceDataHandle.Get<FSceneStateBindingDataHandle>());
		bResult &= BindingCollection.CopyProperty(Copy, SourceView, InParams.TargetDataView);
	}

	return bResult;
}

void* ResolveVisitedPath(const UScriptStruct* InRootObject, void* InRootData, const FPropertyVisitorPath& InPath)
{
	return PropertyVisitorHelpers::ResolveVisitedPath(InRootObject, InRootData, InPath);
}

void DiscardObject(UObject* InObjectToDiscard)
{
	if (InObjectToDiscard)
	{
		UObject* NewOuter = GetTransientPackage();
		FName UniqueName = MakeUniqueObjectName(NewOuter, InObjectToDiscard->GetClass(), *(TEXT("TRASH_") + InObjectToDiscard->GetName()));
		InObjectToDiscard->Rename(*UniqueName.ToString(), NewOuter, REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
		InObjectToDiscard->MarkAsGarbage();
	}
}

UObject* DiscardObject(UObject* InOuter, const TCHAR* InObjectName, TFunctionRef<void(UObject*)> InOnPreDiscardOldObject)
{
	if (UObject* OldObject = StaticFindObject(UObject::StaticClass(), InOuter, InObjectName))
	{
		InOnPreDiscardOldObject(OldObject);
		DiscardObject(OldObject);
	}
	return nullptr;
}

bool ReplaceObject(UObject*& InOutObject
	, UObject* InOuter
	, UClass* InClass
	, const TCHAR* InObjectName
	, const TCHAR* InContextName
	, TFunctionRef<void(UObject*)> InOnPreDiscardOldObject)
{
	if (!InOuter)
	{
		UE_LOGF(LogSceneState, Error, "ReplaceObjectSafe did not take place (Context: %ls). Outer is invalid.", InContextName);
		return false;
	}

	if (!InObjectName)
	{
		UE_LOGF(LogSceneState, Error, "ReplaceObjectSafe did not take place (Context: %ls). Object Name is invalid.", InContextName);
		return false;
	}

	if (InOutObject && InOutObject->GetName() != InObjectName)
	{
		UE_LOGF(LogSceneState, Error, "ReplaceObjectSafe did not take place (Context: %ls). Object Name '%ls' does not match existing object name '%ls'."
			, InContextName
			, InObjectName
			, *InOutObject->GetName());
		return false;
	}

	if (InOutObject && InClass && InOutObject->GetClass() == InClass)
	{
		UE_LOGF(LogSceneState, Log, "ReplaceObjectSafe did not take place (Context: %ls). '%ls' (%p) as is already of class %ls."
			, InContextName
			, *InOutObject->GetName()
			, InOutObject
			, *InClass->GetName());
		return false;
	}

	EObjectFlags MaskedOuterFlags = InOuter->GetMaskedFlags(RF_PropagateToSubObjects);

	UObject* OldObject = DiscardObject(InOuter, InObjectName, InOnPreDiscardOldObject);

	if (InClass)
	{
		InOutObject = NewObject<UObject>(InOuter, InClass, InObjectName, MaskedOuterFlags);

		if (OldObject && GEngine)
		{
			TMap<UObject*, UObject*> ReplacementMap;
			ReplacementMap.Add(OldObject, InOutObject);
			GEngine->NotifyToolsOfObjectReplacement(ReplacementMap);
		}
	}
	else
	{
		InOutObject = nullptr;
	}

	return true;
}

#if WITH_EDITOR
namespace Editor
{

bool StructHasMetaData(const UStruct* InStruct, FName InMetaData, bool bInIncludeSuperStructs)
{
	return InStruct && StructHasAnyMetaData(InStruct, MakeArrayView(&InMetaData, 1), bInIncludeSuperStructs);
}

bool StructHasAnyMetaData(const UStruct* InStruct, TConstArrayView<FName> InMetaData, bool bInIncludeSuperStructs)
{
	for (const UStruct* Struct = InStruct; Struct; Struct = Struct->GetSuperStruct())
	{
		for (const FName& MetaData : InMetaData)
		{
			if (Struct->HasMetaData(MetaData))
			{
				return true;
			}
		}
		if (!bInIncludeSuperStructs)
		{
			return false;
		}
	}
	return false;
}

} // UE::SceneState::Editor
#endif

} // UE::SceneState
