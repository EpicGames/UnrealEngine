// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateVersion.h"
#include "Serialization/CustomVersion.h"

namespace UE::SceneState
{
	const FGuid FVersion::Guid(0x2333E4E3, 0xA940435F, 0x8B481930, 0x906B27F9);
	FCustomVersionRegistration GRegisterSceneStateVersion(FVersion::Guid, FVersion::LatestVersion, TEXT("SceneStateVersion"));

} // UE::SceneState
