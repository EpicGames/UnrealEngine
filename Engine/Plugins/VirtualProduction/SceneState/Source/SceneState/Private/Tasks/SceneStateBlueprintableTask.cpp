// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/SceneStateBlueprintableTask.h"
#include "PropertyBindingDataView.h"
#include "SceneStateEventHandler.h"
#include "SceneStateExecutionContext.h"
#include "SceneStateLog.h"
#include "SceneStateObject.h"
#include "Tasks/SceneStateBlueprintableTaskWrapper.h"
#include "Tasks/SceneStateTask.h"
#include "Tasks/SceneStateTaskInstance.h"

USceneStateObject* USceneStateBlueprintableTask::GetRootState() const
{
	return Cast<USceneStateObject>(GetOuter());
}

const FSceneStateExecutionContext& USceneStateBlueprintableTask::GetExecutionContext() const
{
	if (const FSceneStateExecutionContext* ExecutionContext = TaskExecutionContext.GetExecutionContext())
	{
		return *ExecutionContext;
	}
	return FSceneStateExecutionContext::InvalidContext;
}

UObject* USceneStateBlueprintableTask::GetContextObject() const
{
	if (USceneStateObject* RootState = GetRootState())
	{
		return RootState->GetContextObject();
	}
	return nullptr;
}

USceneStateEventStream* USceneStateBlueprintableTask::GetEventStream() const
{
	if (USceneStateObject* RootState = GetRootState())
	{
		return RootState->GetEventStream();
	}
	return nullptr;
}

void USceneStateBlueprintableTask::FinishTask()
{
	TaskExecutionContext.FinishTask();
}

bool USceneStateBlueprintableTask::FindEventHandlerId(const FSceneStateEventSchemaHandle& InEventSchemaHandle, FGuid& OutHandlerId) const
{
	const FSceneStateTask* Task = TaskExecutionContext.GetTask().GetPtr<const FSceneStateTask>();
	if (!Task || Task->GetParentStateIndex() == FSceneStateRange::InvalidIndex)
	{
		return false;
	}

	const FSceneStateExecutionContext& ExecutionContext = GetExecutionContext();

	const FSceneState* State = ExecutionContext.GetState(Task->GetParentStateIndex());
	if (!State)
	{
		return false;
	}

	for (const FSceneStateEventHandler& EventHandler : ExecutionContext.GetEventHandlers(*State))
	{
		if (EventHandler.GetEventSchemaHandle() == InEventSchemaHandle)
		{
			OutHandlerId = EventHandler.GetHandlerId();
			return true;
		}
	}

	return false;
}

UWorld* USceneStateBlueprintableTask::GetWorld() const
{
	if (UObject* Context = GetContextObject())
	{
		return Context->GetWorld();
	}
	return nullptr;
}
