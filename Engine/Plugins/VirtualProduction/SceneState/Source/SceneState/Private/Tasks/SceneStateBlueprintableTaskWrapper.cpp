// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/SceneStateBlueprintableTaskWrapper.h"
#include "PropertyBindingDataView.h"
#include "SceneStateExecutionContext.h"
#include "SceneStateObject.h"
#include "SceneStateUtils.h"
#include "Tasks/SceneStateBlueprintableTask.h"

FSceneStateBlueprintableTaskWrapper::FSceneStateBlueprintableTaskWrapper()
{
	SetFlags(ESceneStateTaskFlags::Ticks);
}

bool FSceneStateBlueprintableTaskWrapper::SetTaskClass(TSubclassOf<USceneStateBlueprintableTask> InTaskClass)
{
	if (TaskClass == InTaskClass)
	{
		return false;
	}

	TaskClass = InTaskClass;
	return true;
}

#if WITH_EDITOR
const UScriptStruct* FSceneStateBlueprintableTaskWrapper::OnGetTaskInstanceType() const
{
	return FInstanceDataType::StaticStruct();
}

void FSceneStateBlueprintableTaskWrapper::OnBuildTaskInstance(UObject* InOuter, FStructView InTaskInstance) const
{
	if (TaskClass)
	{
		FInstanceDataType& Instance = InTaskInstance.Get<FInstanceDataType>();
		Instance.Task = NewObject<USceneStateBlueprintableTask>(InOuter, TaskClass);
	}
}
#endif

void FSceneStateBlueprintableTaskWrapper::OnSetup(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance) const
{
	FInstanceDataType& Instance = InTaskInstance.Get<FInstanceDataType>();
	if (Instance.Task)
	{
		Instance.Task->TaskExecutionContext = UE::SceneState::FTaskExecutionContext(*this, InContext);
	}
}

void FSceneStateBlueprintableTaskWrapper::OnStart(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance) const
{
	FInstanceDataType& Instance = InTaskInstance.Get<FInstanceDataType>();
	if (Instance.Task)
	{
		Instance.Task->ReceiveStart();
	}
}

void FSceneStateBlueprintableTaskWrapper::OnTick(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance, float InDeltaSeconds) const
{
	FInstanceDataType& Instance = InTaskInstance.Get<FInstanceDataType>();
	if (Instance.Task)
	{
		Instance.Task->ReceiveTick(InDeltaSeconds);
	}
}

void FSceneStateBlueprintableTaskWrapper::OnStop(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance, ESceneStateTaskStopReason InStopReason) const
{
	FInstanceDataType& Instance = InTaskInstance.Get<FInstanceDataType>();
	if (Instance.Task)
	{
		Instance.Task->ReceiveStop(InStopReason);
		Instance.Task = nullptr;
	}
}
