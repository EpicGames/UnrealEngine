// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/SceneStateMachineTask.h"
#include "SceneStateBindingUtils.h"
#include "SceneStateExecutionContext.h"
#include "SceneStateMachine.h"

FSceneStateMachineTask::FSceneStateMachineTask()
{
	SetFlags(ESceneStateTaskFlags::Ticks | ESceneStateTaskFlags::HasBindingExtension);
}

#if WITH_EDITOR
const UScriptStruct* FSceneStateMachineTask::OnGetTaskInstanceType() const
{
	return FInstanceDataType::StaticStruct();
}

void FSceneStateMachineTask::OnBuildTaskInstance(UObject* InOuter, FStructView InTaskInstance) const
{
	FInstanceDataType& Instance = InTaskInstance.Get<FInstanceDataType>();

	const FGuid OldParametersId = Instance.ParametersId;
	Instance.ParametersId = FGuid::NewGuid();

	UE::SceneState::HandleStructIdChanged(*InOuter, OldParametersId, Instance.ParametersId);
}
#endif

const FSceneStateTaskBindingExtension* FSceneStateMachineTask::OnGetBindingExtension() const
{
	return &Binding;
}

void FSceneStateMachineTask::OnStart(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance) const
{
	FInstanceDataType& Instance = InTaskInstance.Get<FInstanceDataType>();

	const FSceneStateMachine* StateMachine = InContext.GetStateMachine(Instance.TargetId);
	if (!StateMachine)
	{
		return;
	}

	Instance.ExecutionContext.Setup(InContext);

	StateMachine->Setup(Instance.ExecutionContext);

	FSceneStateMachineInstance* StateMachineInstance = Instance.ExecutionContext.FindStateMachineInstance(*StateMachine);
	if (!StateMachineInstance)
	{
		Finish(InContext, InTaskInstance);
		return;
	}

	StateMachineInstance->Parameters = Instance.Parameters;
	StateMachine->Start(Instance.ExecutionContext);

	if (IsStateMachineFinished(Instance.ExecutionContext, *StateMachine))
	{
		Finish(InContext, InTaskInstance);
	}
}

void FSceneStateMachineTask::OnTick(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance, float InDeltaSeconds) const
{
	const FInstanceDataType& Instance = InTaskInstance.Get<FInstanceDataType>();

	const FSceneStateMachine* StateMachine = InContext.GetStateMachine(Instance.TargetId);
	if (!StateMachine)
	{
		// Finish task with the context execution that ran this task
		Finish(InContext, InTaskInstance);
		return;
	}

	StateMachine->Tick(Instance.ExecutionContext, InDeltaSeconds);

	if (IsStateMachineFinished(Instance.ExecutionContext, *StateMachine))
	{
		// Finish task with the context execution that ran this task
		Finish(InContext, InTaskInstance);
	}
}

void FSceneStateMachineTask::OnStop(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance, ESceneStateTaskStopReason InStopReason) const
{
	FInstanceDataType& Instance = InTaskInstance.Get<FInstanceDataType>();

	const FSceneStateMachine* StateMachine = InContext.GetStateMachine(Instance.TargetId);
	if (!StateMachine)
	{
		return;
	}

	StateMachine->Stop(Instance.ExecutionContext);
	Instance.ExecutionContext.Reset();
}

bool FSceneStateMachineTask::IsStateMachineFinished(const FSceneStateExecutionContext& InContext, const FSceneStateMachine& InStateMachine) const
{
	const FSceneStateMachineInstance* StateMachineInstance = InContext.FindStateMachineInstance(InStateMachine);
	return !StateMachineInstance || StateMachineInstance->Status == UE::SceneState::EExecutionStatus::Finished;
}
