// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/SceneStateMachineTaskBinding.h"
#include "Tasks/SceneStateMachineTask.h"

#if WITH_EDITOR
void FSceneStateMachineTaskBinding::VisitBindingDescs(FStructView InTaskInstance, TFunctionRef<void(const UE::SceneState::FTaskBindingDesc&)> InFunctor) const
{
	FInstanceDataType& Instance = InTaskInstance.Get<FInstanceDataType>();

	if (Instance.Parameters.IsValid())
	{
		UE::SceneState::FTaskBindingDesc BindingDesc;

		BindingDesc.Id = Instance.ParametersId;
		BindingDesc.Name = TEXT("State Machine Task Parameters");
		BindingDesc.DataView = Instance.Parameters.GetMutableValue();
		BindingDesc.DataIndex = PARAMETERS_DATA_INDEX;

		InFunctor(BindingDesc);
	}
}

void FSceneStateMachineTaskBinding::SetBindingBatch(uint16 InDataIndex, uint16 InBatchIndex)
{
	if (InDataIndex == PARAMETERS_DATA_INDEX)
	{
		ParametersBatchIndex = InBatchIndex;
	}
}

bool FSceneStateMachineTaskBinding::FindDataById(FStructView InTaskInstance, const FGuid& InStructId, FStructView& OutDataView, uint16& OutDataIndex) const
{
	FInstanceDataType& Instance = InTaskInstance.Get<FInstanceDataType>();
	if (Instance.ParametersId == InStructId && Instance.Parameters.IsValid())
	{
		OutDataView = Instance.Parameters.GetMutableValue();
		OutDataIndex = PARAMETERS_DATA_INDEX;
		return true;
	}
	return false;
}
#endif

bool FSceneStateMachineTaskBinding::FindDataByIndex(FStructView InTaskInstance, uint16 InDataIndex, FStructView& OutDataView) const
{
	FInstanceDataType& Instance = InTaskInstance.Get<FInstanceDataType>();

	if (InDataIndex == PARAMETERS_DATA_INDEX && Instance.Parameters.IsValid())
	{
		OutDataView = Instance.Parameters.GetMutableValue();
		return true;
	}

	return false;
}

void FSceneStateMachineTaskBinding::VisitBindingBatches(FStructView InTaskInstance, TFunctionRef<void(uint16, FStructView)> InFunctor) const
{
	FInstanceDataType& Instance = InTaskInstance.Get<FInstanceDataType>();

	if (Instance.Parameters.IsValid())
	{
		InFunctor(ParametersBatchIndex, Instance.Parameters.GetMutableValue());
	}
}
