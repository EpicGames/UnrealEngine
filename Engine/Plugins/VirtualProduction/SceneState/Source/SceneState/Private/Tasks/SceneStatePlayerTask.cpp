// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/SceneStatePlayerTask.h"
#include "PropertyBindingDataView.h"
#include "SceneStateExecutionContext.h"
#include "SceneStateObject.h"
#include "SceneStatePlayer.h"

FSceneStatePlayerTask::FSceneStatePlayerTask()
{
	SetFlags(ESceneStateTaskFlags::Ticks);
}

#if WITH_EDITOR
const UScriptStruct* FSceneStatePlayerTask::OnGetTaskInstanceType() const
{
	return FInstanceDataType::StaticStruct();
}

void FSceneStatePlayerTask::OnBuildTaskInstance(UObject* InOuter, FStructView InTaskInstance) const
{
	FInstanceDataType& Instance = InTaskInstance.Get<FInstanceDataType>();
	Instance.Player = NewObject<USceneStatePlayer>(InOuter);
}
#endif

void FSceneStatePlayerTask::OnSetup(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance) const
{
	FInstanceDataType& Instance = InTaskInstance.Get<FInstanceDataType>();
	if (Instance.Player)
	{
		Instance.Player->Setup();
	}
}

void FSceneStatePlayerTask::OnStart(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance) const
{
	FInstanceDataType& Instance = InTaskInstance.Get<FInstanceDataType>();
	if (Instance.Player)
	{
		Instance.Player->Begin();
	}
}

void FSceneStatePlayerTask::OnTick(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance, float InDeltaSeconds) const
{
	FInstanceDataType& Instance = InTaskInstance.Get<FInstanceDataType>();
	if (Instance.Player)
	{
		Instance.Player->Tick(InDeltaSeconds);
	}
}

void FSceneStatePlayerTask::OnStop(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance, ESceneStateTaskStopReason InStopReason) const
{
	FInstanceDataType& Instance = InTaskInstance.Get<FInstanceDataType>();
	if (Instance.Player)
	{
		Instance.Player->End();
	}
}
