// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/SceneStateTask.h"
#include "SceneStateExecutionContext.h"
#include "SceneStateObject.h"
#include "SceneStateUtils.h"
#include "Tasks/SceneStateTaskBindingExtension.h"
#include "Tasks/SceneStateTaskInstance.h"

namespace UE::SceneState::Private
{

EExecutionStatus GetTaskStatus(ESceneStateTaskStopReason InStopReason)
{
	switch (InStopReason)
	{
	case ESceneStateTaskStopReason::State:
		return EExecutionStatus::NotStarted;

	case ESceneStateTaskStopReason::Finished:
		return EExecutionStatus::Finished;
	}

	checkNoEntry();
	return EExecutionStatus::Finished;
}

} // UE::SceneState::Private

#if WITH_EDITOR
const UScriptStruct* FSceneStateTask::GetTaskInstanceType() const
{
	return OnGetTaskInstanceType();
}

bool FSceneStateTask::SupportsSchema(const USceneStateSchema* InSchema) const
{
	return InSchema && OnSupportsSchema(InSchema);
}

void FSceneStateTask::BuildTaskInstance(UObject* InOuter, FStructView InTaskInstance) const
{
	if (InTaskInstance.IsValid())
	{
		OnBuildTaskInstance(InOuter, InTaskInstance);
	}
}

void FSceneStateTask::PostEditChange(UE::SceneState::FTaskEditChange& InEditChange, FStructView InTaskInstance)
{
	if (InTaskInstance.IsValid())
	{
		OnPostTaskEditChange(InEditChange, InTaskInstance);
	}
}
#endif

const FSceneStateTaskBindingExtension* FSceneStateTask::GetBindingExtension() const
{
	if (EnumHasAllFlags(TaskFlags, ESceneStateTaskFlags::HasBindingExtension))
	{
		return OnGetBindingExtension();
	}
	return nullptr;
}

FStructView FSceneStateTask::FindTaskInstance(const FSceneStateExecutionContext& InContext) const
{
	ensureAlwaysMsgf(false, TEXT("FSceneStateTask::FindTaskInstance is deprecated! Use the task instance provided by the virtual methods."));
	return InContext.FindTaskInstance(*this);
}

void FSceneStateTask::Setup(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance) const
{
	using namespace UE::SceneState;

	QUICK_SCOPE_CYCLE_COUNTER(SceneState_OnSetup);

	if (!InTaskInstance.IsValid())
	{
		return;
	}

	InContext.SetTaskStatus(*this, EExecutionStatus::NotStarted);
	InContext.SetupFunctionInstances(BindingsBatch);

	if (const FSceneStateTaskBindingExtension* BindingExtension = GetBindingExtension())
	{
		BindingExtension->VisitBindingBatches(InTaskInstance,
			[&InContext](uint16 InBatchIndex, FStructView InTargetDataView)
			{
				InContext.SetupFunctionInstances(FPropertyBindingIndex16(InBatchIndex));
			});
	}

	OnSetup(InContext, InTaskInstance);
}

void FSceneStateTask::Start(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance) const
{
	using namespace UE::SceneState;

	QUICK_SCOPE_CYCLE_COUNTER(SceneState_OnStart);

	if (!InTaskInstance.IsValid() || !IsTaskStatus(InContext, EExecutionStatus::NotStarted))
	{
		return;
	}

	InContext.SetTaskStatus(*this, EExecutionStatus::Running);

	ApplyBindings(InContext, InTaskInstance);
	OnStart(InContext, InTaskInstance);
}

void FSceneStateTask::Tick(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance, float InDeltaSeconds) const
{
	using namespace UE::SceneState;

	// Skip task is not set to tick
	if (!EnumHasAnyFlags(TaskFlags, ESceneStateTaskFlags::Ticks))
	{
		return;
	}

	QUICK_SCOPE_CYCLE_COUNTER(SceneState_OnTick);

	if (!InTaskInstance.IsValid() || !IsTaskStatus(InContext, EExecutionStatus::Running))
	{
		return;
	}

	OnTick(InContext, InTaskInstance, InDeltaSeconds);
}

void FSceneStateTask::Stop(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance, ESceneStateTaskStopReason InStopReason) const
{
	using namespace UE::SceneState;

	QUICK_SCOPE_CYCLE_COUNTER(SceneState_OnStop);

	if (!InTaskInstance.IsValid())
	{
		return;
	}

	if (IsTaskStatus(InContext, EExecutionStatus::Running))
	{
		OnStop(InContext, InTaskInstance, InStopReason);
	}

	InContext.RemoveFunctionInstances(BindingsBatch);

	if (const FSceneStateTaskBindingExtension* BindingExtension = GetBindingExtension())
	{
		BindingExtension->VisitBindingBatches(InTaskInstance,
			[&InContext](uint16 InBatchIndex, FStructView InTargetDataView)
			{
				InContext.RemoveFunctionInstances(FPropertyBindingIndex16(InBatchIndex));
			});
	}

	InContext.SetTaskStatus(*this, Private::GetTaskStatus(InStopReason));
}

void FSceneStateTask::Finish(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance) const
{
	Stop(InContext, InTaskInstance, ESceneStateTaskStopReason::Finished);
}

void FSceneStateTask::SetFlags(ESceneStateTaskFlags InFlags)
{
	TaskFlags |= InFlags;
}

void FSceneStateTask::ClearFlags(ESceneStateTaskFlags InFlags)
{
	TaskFlags &= ~InFlags;
}

bool FSceneStateTask::IsTaskStatus(const FSceneStateExecutionContext& InContext, UE::SceneState::EExecutionStatus InExpectedStatus) const
{
	const TOptional<UE::SceneState::EExecutionStatus> TaskStatus = InContext.GetTaskStatus(*this);
	return TaskStatus.IsSet() && *TaskStatus == InExpectedStatus;
}

bool FSceneStateTask::ApplyBindings(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance) const
{
	QUICK_SCOPE_CYCLE_COUNTER(SceneStateTask_ApplyBindings);

	UE::SceneState::FApplyBatchParams ApplyBatchParams
		{
			.BindingsBatch = BindingsBatch.Get(),
			.TargetDataView = InTaskInstance
		};

	bool bResult = ApplyBatch(InContext, ApplyBatchParams);

	if (const FSceneStateTaskBindingExtension* BindingExtension = GetBindingExtension())
	{
		BindingExtension->VisitBindingBatches(InTaskInstance,
			[&bResult, &InContext, &ApplyBatchParams](uint16 InBatchIndex, FStructView InTargetDataView)
			{
				ApplyBatchParams.BindingsBatch = InBatchIndex;
				ApplyBatchParams.TargetDataView = InTargetDataView;
				bResult &= ApplyBatch(InContext, ApplyBatchParams);
			});
	}

	return bResult;
}
