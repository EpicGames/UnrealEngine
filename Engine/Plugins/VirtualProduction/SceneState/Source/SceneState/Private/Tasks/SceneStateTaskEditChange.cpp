// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/SceneStateTaskEditChange.h"

namespace UE::SceneState
{

#if WITH_EDITOR
FTaskEditChange::FTaskEditChange()
	: FPropertyChangedEvent(nullptr)
{
}

bool FTaskEditChange::IsTaskChange() const
{
	return ChangedObject == ETaskObjectType::Task;
}

bool FTaskEditChange::IsTaskInstanceChange() const
{
	return ChangedObject == ETaskObjectType::TaskInstance;
}
#endif // WITH_EDITOR

} // UE::SceneState
