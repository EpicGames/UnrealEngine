// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/SceneStateTaskExecutionContext.h"
#include "SceneStateExecutionContext.h"
#include "SceneStateExecutionContextRegistry.h"
#include "SceneStateLog.h"
#include "Tasks/SceneStateTask.h"
#include "Tasks/SceneStateTaskInstance.h"

namespace UE::SceneState
{

namespace Private
{

uint16 GetStateInstanceId(const FSceneStateTask& InTask, const FSceneStateExecutionContext& InContext)
{
	if (FSceneStateInstance* StateInstance = InContext.FindStateInstance(InTask.GetParentStateIndex()))
	{
		return StateInstance->GetInstanceId();
	}
	return 0;
}

bool IsMatchingInstanceId(const FSceneStateExecutionContext& InContext, uint16 InParentStateIndex, uint16 InStateInstanceId)
{
	if (const FSceneStateInstance* StateInstance = InContext.FindStateInstance(InParentStateIndex))
	{
		return StateInstance->GetInstanceId() == InStateInstanceId;
	}
	return false;
}

} // Private

FTaskExecutionContext::FTaskExecutionContext()
	: TaskIndex(FSceneStateRange::InvalidIndex)
	, StateIndex(FSceneStateRange::InvalidIndex)
	, StateInstanceId(0)
{
}

FTaskExecutionContext::FTaskExecutionContext(const FSceneStateTask& InTask, const FSceneStateExecutionContext& InContext)
	: ContextHandle(InContext.GetContextHandle())
	, ContextRegistryWeak(InContext.GetContextRegistry())
	, TaskIndex(InTask.GetTaskIndex())
	, StateIndex(InTask.GetParentStateIndex())
	, StateInstanceId(Private::GetStateInstanceId(InTask, InContext))
{
}

const FSceneStateExecutionContext* FTaskExecutionContext::GetExecutionContext() const
{
	if (const TSharedPtr<const FExecutionContextRegistry> ContextRegistry = ContextRegistryWeak.Pin())
	{
		return ContextRegistry->FindContext(ContextHandle);
	}
	return nullptr;
}

FConstStructView FTaskExecutionContext::GetTask() const
{
	if (const FSceneStateExecutionContext* ExecutionContext = GetExecutionContext())
	{
		// Ensure this Context is still valid, by ensuring the saved State Instance Id matches the current State Instance id of execution
		if (Private::IsMatchingInstanceId(*ExecutionContext, StateIndex, StateInstanceId))
		{
			return ExecutionContext->FindTask(TaskIndex);
		}
	}
	return FConstStructView();
}

FStructView FTaskExecutionContext::GetTaskInstance() const
{
	if (const FSceneStateExecutionContext* ExecutionContext = GetExecutionContext())
	{
		FStructView TaskInstance = ExecutionContext->FindTaskInstance(TaskIndex);

		// Ensure this Context is still valid, by ensuring the saved State Instance Id matches the current State Instance id of execution
		if (Private::IsMatchingInstanceId(*ExecutionContext, StateIndex, StateInstanceId))
		{
			return TaskInstance;
		}
	}
	return FStructView();
}

void FTaskExecutionContext::FinishTask() const
{
	constexpr const TCHAR* BaseErrorMessage = TEXT("Failed to Finish Task.");

	const FSceneStateExecutionContext* ExecutionContext = GetExecutionContext();
	if (!ExecutionContext)
	{
		UE_LOGF(LogSceneState, Error, "[%ls] %ls Invalid Execution Context !"
			, *GetDebugString()
			, BaseErrorMessage)
		return;
	}

	const FSceneStateTask* Task = ExecutionContext->FindTask(TaskIndex).GetPtr<const FSceneStateTask>();
	if (!Task)
	{
		UE_LOGF(LogSceneState, Error, "[%ls in Context '%ls'] %ls Task not found!"
			, *GetDebugString()
			, BaseErrorMessage
			, *ExecutionContext->GetExecutionContextName())
		return;
	}

	const FStructView TaskInstance = ExecutionContext->FindTaskInstance(TaskIndex);
	if (!TaskInstance.IsValid())
	{
		UE_LOGF(LogSceneState, Error, "[%ls in Context '%ls'] %ls Task Instance not found!"
			, *GetDebugString()
			, *ExecutionContext->GetExecutionContextName()
			, BaseErrorMessage)
		return;
	}

	if (!Private::IsMatchingInstanceId(*ExecutionContext, StateIndex, StateInstanceId))
	{
		UE_LOGF(LogSceneState, Error, "[%ls in Context '%ls'] %ls Task Context outdated!"
			, *GetDebugString()
			, *ExecutionContext->GetExecutionContextName()
			, BaseErrorMessage)
		return;
	}

	Task->Finish(*ExecutionContext, TaskInstance);
}

FString FTaskExecutionContext::GetDebugString() const
{
	return FString::Printf(TEXT("Task Index: %d, State Instance Id: %u"), TaskIndex, StateInstanceId);
}

} // UE::SceneState
