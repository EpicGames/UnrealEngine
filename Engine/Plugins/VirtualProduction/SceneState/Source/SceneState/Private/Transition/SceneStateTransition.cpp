// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transition/SceneStateTransition.h"
#include "Conduit/SceneStateConduit.h"
#include "SceneState.h"
#include "SceneStateEventHandler.h"
#include "SceneStateEventStream.h"
#include "SceneStateExecutionContext.h"
#include "SceneStateMachine.h"
#include "SceneStateObject.h"
#include "SceneStateUtils.h"
#include "Transition/SceneStateTransitionEvaluation.h"
#include "Transition/SceneStateTransitionInstance.h"
#include "Transition/SceneStateTransitionLink.h"
#include "Transition/SceneStateTransitionResult.h"

void FSceneStateTransition::Link(const FSceneStateTransitionLink& InTransitionLink, TNotNull<const UClass*> InOwnerClass)
{
	if (!InTransitionLink.ResultPropertyName.IsNone())
	{
		ResultProperty = CastField<FStructProperty>(InOwnerClass->FindPropertyByName(InTransitionLink.ResultPropertyName));
		check(ResultProperty);
	}
	else
	{
		ResultProperty = nullptr;
	}

	if (!InTransitionLink.EventName.IsNone())
	{
		EvaluationEvent = InOwnerClass->FindFunctionByName(InTransitionLink.EventName);
		check(EvaluationEvent);
	}
	else
	{
		EvaluationEvent = nullptr;
	}
}

void FSceneStateTransition::Setup(const FSceneStateExecutionContext& InContext) const
{
	FSceneStateTransitionInstance* Instance = InContext.FindOrAddTransitionInstance(*this);
	if (!ensure(Instance))
	{
		return;
	}

	InContext.SetupFunctionInstances(BindingsBatch);
	Instance->Parameters = InContext.GetTemplateTransitionParameter(*this);

	// Setup the conduit chaining this transition
	if (const FSceneStateConduit* Conduit = FindConduitTarget(InContext))
	{
		Conduit->Setup(InContext);
	}
}

bool FSceneStateTransition::Evaluate(const UE::SceneState::FTransitionEvaluationParams& InParams) const
{
	// Check if the target state's required events are all present
	if (!ContainsAllRequiredEvents(InParams))
	{
		return false;
	}

	// Early return if waiting for tasks to finish and there are still active tasks yet to finish
	if (EnumHasAnyFlags(EvaluationFlags, ESceneStateTransitionEvaluationFlags::WaitForTasksToFinish)
		&& InParams.SourceState.HasPendingTasks(InParams.ExecutionContext))
	{
		return false;
	}

	return ProcessEvaluationEvent(InParams);
}

void FSceneStateTransition::Exit(const FSceneStateExecutionContext& InContext) const
{
	InContext.RemoveTransitionInstance(*this);
	InContext.RemoveFunctionInstances(BindingsBatch);

	// Exit the conduit chaining this transition
	if (const FSceneStateConduit* Conduit = FindConduitTarget(InContext))
	{
		Conduit->Exit(InContext);
	}
}

bool FSceneStateTransition::ApplyBindings(const FSceneStateExecutionContext& InContext, FSceneStateTransitionInstance& InInstance) const
{
	QUICK_SCOPE_CYCLE_COUNTER(SceneStateTransition_ApplyBindings);

	const UE::SceneState::FApplyBatchParams ApplyBatchParams
		{
			.BindingsBatch = BindingsBatch.Get(),
			.TargetDataView = InInstance.Parameters.GetMutableValue(),
		};

	return ApplyBatch(InContext, ApplyBatchParams);
}

bool FSceneStateTransition::ContainsAllRequiredEvents(const UE::SceneState::FTransitionEvaluationParams& InParams) const
{
	// No required events present for targets that aren't states
	if (Target.Type != ESceneStateTransitionTargetType::State)
	{
		return true;
	}

	USceneStateEventStream* EventStream = InParams.ExecutionContext.GetEventStream();
	if (!EventStream)
	{
		return true;
	}

	const FSceneState* TargetState = InParams.ExecutionContext.GetState(InParams.StateMachine, Target.Index);
	if (!TargetState)
	{
		return true;
	}

	for (const FSceneStateEventHandler& TargetEventHandler : InParams.ExecutionContext.GetEventHandlers(*TargetState))
	{
		// Fail transition condition if a Target Event Handler cannot find a matching Event in the current stream
		if (!EventStream->FindEventBySchema(TargetEventHandler.GetEventSchemaHandle()))
		{
			return false;
		}
	}

	return true;
}

bool FSceneStateTransition::ProcessEvaluationEvent(const UE::SceneState::FTransitionEvaluationParams& InParams) const
{
	if (EnumHasAllFlags(EvaluationFlags, ESceneStateTransitionEvaluationFlags::EvaluationEventAlwaysTrue))
	{
		return true;
	}

	if (!EvaluationEvent || !ResultProperty)
	{
		return false;
	}

	void* FunctionParams = nullptr;

	// The evaluation event could either have 0 parameters (optimized when the transition parameters are not used in the event)
	// or it could be using these parameters, in which case EvalEvent->NumParams must match the number of parameters in the bag
	if (EvaluationEvent->NumParms != 0)
	{
		FSceneStateTransitionInstance* Instance = InParams.ExecutionContext.FindOrAddTransitionInstance(*this);
		if (!ensureMsgf(Instance && Instance->Parameters.IsValid()
			, TEXT("Transition Instance Parameters is not valid. Transition evaluation event expects %d parameters.")
			, EvaluationEvent->NumParms))
		{
			return false;
		}

		if (!ensureMsgf(EvaluationEvent->NumParms == Instance->Parameters.GetNumPropertiesInBag()
			, TEXT("Unexpected parameter mismatch! Event Parameter Count: %d... Transition Parameter Count: %d")
			, EvaluationEvent->NumParms
			, Instance->Parameters.GetNumPropertiesInBag()))
		{
			return false;
		}

		ApplyBindings(InParams.ExecutionContext, *Instance);
		FunctionParams = Instance->Parameters.GetMutableValue().GetMemory();
	}

	UObject* RootObject = InParams.ExecutionContext.GetRootObject();
	if (!ensure(RootObject))
	{
		return false;
	}

	RootObject->ProcessEvent(EvaluationEvent, FunctionParams);

	const FSceneStateTransitionResult* Result = ResultProperty->ContainerPtrToValuePtr<FSceneStateTransitionResult>(RootObject);
	check(Result);
	return Result->bCanTransition;
}

const FSceneStateConduit* FSceneStateTransition::FindConduitTarget(const FSceneStateExecutionContext& InContext) const
{
	if (Target.Type != ESceneStateTransitionTargetType::Conduit)
	{
		// Target isn't a conduit, skip
		return nullptr;
	}

	const FSceneStateMachine* StateMachine = InContext.GetContextStateMachine();
	if (!ensureMsgf(StateMachine, TEXT("Unexpectedly found no context state machine. Is this being directly called outside of a State Machine?")))
	{
		return nullptr;
	}

	const FSceneStateConduit* Conduit = InContext.GetConduit(*StateMachine, Target.Index);
	if (!ensureMsgf(Conduit, TEXT("Unable to find conduit at target index %d. State Machine Conduit count: %d"), Target.Index, StateMachine->GetConduitRange().Count))
	{
		return nullptr;
	}

	return Conduit;
}
