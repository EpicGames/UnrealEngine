// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneStateEnums.h"
#include "SceneStateRange.h"
#include "UObject/ObjectPtr.h"
#include "SceneStateConduit.generated.h"

struct FSceneStateConduitLink;
struct FSceneStateExecutionContext;

namespace UE::SceneState
{
	struct FTransitionEvaluationParams;

	namespace Editor
	{
		class FStateMachineCompiler;
		class FStateMachineConduitCompiler;
	}
}

/** Defines a conduit, a transition indirection */
USTRUCT()
struct FSceneStateConduit
{
	GENERATED_BODY()

	/** Called when the Owner Class is linking to cache the Event function and Result property */
	void Link(const FSceneStateConduitLink& InConduitLink, TNotNull<const UClass*> InOwnerClass);

	/** Called when a state that could evaluate this conduit has entered. Allocates and initializes the conduit instance */
	void Setup(const FSceneStateExecutionContext& InContext) const;

	/** Called to evaluate this conduit. Returns true if the conduit should take place */
	bool Evaluate(const UE::SceneState::FTransitionEvaluationParams& InParams) const;

	/** Called when the State has exited. Cleans up the conduit instance */
	void Exit(const FSceneStateExecutionContext& InContext) const;

	FSceneStateRange GetTransitionRange() const
	{
		return TransitionRange;
	}

private:
	/** Calls the evaluation event (can return early if optimized away via the evaluation flags) */
	bool ProcessEvaluationEvent(const UE::SceneState::FTransitionEvaluationParams& InParams) const;

	/** Index and count of the exit transitions that go out of this conduit and into other targets (states, conduits, exit). */
	UPROPERTY()
	FSceneStateRange TransitionRange;

	/** Transition flags indicating how a transition should be evaluated */
	UPROPERTY()
	ESceneStateTransitionEvaluationFlags EvaluationFlags = ESceneStateTransitionEvaluationFlags::None;

	/** Conduit evaluation event to execute */ 
	UPROPERTY(Transient)
	TObjectPtr<UFunction> EvaluationEvent;

	/** Pointer to the result property of this conduit evaluation */
	FStructProperty* ResultProperty = nullptr;

	friend UE::SceneState::Editor::FStateMachineCompiler;
	friend UE::SceneState::Editor::FStateMachineConduitCompiler;
};
