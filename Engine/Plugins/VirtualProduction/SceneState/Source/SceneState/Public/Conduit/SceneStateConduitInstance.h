// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/PropertyBag.h"
#include "SceneStateConduitInstance.generated.h"

/** Instance data of a conduit */
USTRUCT()
struct FSceneStateConduitInstance
{
	GENERATED_BODY()

	/**
	 * Flag to determine if the conduit instance has already been initialized.
	 * Used as a way to avoid re-setting up all the exit transitions of a conduit.
	 */
	UPROPERTY()
	bool bInitialized = false;
};
