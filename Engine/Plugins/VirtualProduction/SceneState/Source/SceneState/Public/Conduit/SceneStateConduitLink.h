// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"
#include "SceneStateConduitLink.generated.h"

/**
 * Holds information about the conduit that is only used in conduit link time
 * @see FSceneStateConduitLink::Link
 */
USTRUCT()
struct FSceneStateConduitLink
{
	GENERATED_BODY()

	/** Name to lookup and set the Event Function */
	UPROPERTY()
	FName EventName;

	/** Name of the Result Property the Event function sets */
	UPROPERTY()
	FName ResultPropertyName;
};
