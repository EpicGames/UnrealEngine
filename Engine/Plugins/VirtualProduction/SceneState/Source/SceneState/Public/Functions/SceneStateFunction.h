// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyBindingTypes.h"
#include "SceneStateFunction.generated.h"

#define UE_API SCENESTATE_API

struct FSceneStateExecutionContext;
struct FStructView;

namespace UE::SceneState
{
	namespace Editor
	{
		class FBindingCompiler;
	}
}

/**
 * Base struct for all scene state functions.
 * Binding Functions are a layer of logic that execute prior to evaluating owner's bindings.
 *
 * The function's instance data is expected to have its output properties with meta "Output", and these will be hidden in the UI.
 * This property is used to find which bindings the function can be used for.
 *
 * Example:
 *
 *	USTRUCT()
 *	struct FTextToStringFunctionInstance
 *	{
 *		GENERATED_BODY()
 *
 *		UPROPERTY(EditAnywhere, Category="Test")
 *		FText Text;
 *
 *		UPROPERTY(EditAnywhere, Category="Test", meta=(Output))
 *		FString Result;
 *	};
 */
USTRUCT(meta=(Hidden))
struct FSceneStateFunction
{
	GENERATED_BODY()

#if WITH_EDITOR
	/**
	 * Called in-editor to get the function data type
	 * @return the function data struct type
	 */
	 UE_API const UScriptStruct* GetFunctionDataType() const;
#endif

	/** Called when binding owners are setting up and allocating instance data */
	void Setup(const FSceneStateExecutionContext& InContext) const;

	/**
	 * Called right before evaluating bindings for the owner.
	 * @param InContext reference to current execution context.
	 * @param InFunctionInstance data view of the function instance data
	 */
	void Execute(const FSceneStateExecutionContext& InContext, FStructView InFunctionInstance) const;

	/** Called when binding owners are cleaning up their allocated instance data */
	void Cleanup(const FSceneStateExecutionContext& InContext) const;

protected:
#if WITH_EDITOR
	/** Called to retrieve the data type that this function expects*/
	virtual const UScriptStruct* OnGetFunctionDataType() const PURE_VIRTUAL(FSceneStateFunction::OnGetFunctionDataType, return nullptr;);
#endif

	/** Called right before evaluating bindings for the owner. */
	virtual void OnExecute(const FSceneStateExecutionContext& InContext, FStructView InFunctionInstance) const
	{
	}

private:
	/** Applies the bindings batch to the function instance */
	bool ApplyBindings(const FSceneStateExecutionContext& InContext, FStructView InFunctionInstance) const;

	/** Bindings Batch where this function is target */
	UPROPERTY()
	FPropertyBindingIndex16 BindingsBatch;

	friend UE::SceneState::Editor::FBindingCompiler;
};

#if WITH_EDITOR
/** The pure virtual functions in FSceneStateFunction are editor-only. */
template<>
struct TStructOpsTypeTraits<FSceneStateFunction> : TStructOpsTypeTraitsBase2<FSceneStateFunction>
{
	enum
	{
		WithPureVirtual = true,
	};
};
#endif

#undef UE_API
