// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneStateRange.h"
#include "StructUtils/InstancedStructContainer.h"
#include "SceneStateFunctionInstanceBatch.generated.h"

/** Holds all the function instances for a given batch */
USTRUCT()
struct FSceneStateFunctionInstanceBatch
{
	GENERATED_BODY()

	/** The function range (in absolute index) of the functions within the batch */
	UPROPERTY()
	FSceneStateRange FunctionRange;

	/** Container holding all the function instances for a batch in a contiguous block of memory */
	UPROPERTY()
	FInstancedStructContainer FunctionInstances;
};
