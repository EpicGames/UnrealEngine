// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"
#include "SceneStateFunctionMetadata.generated.h"

/** Metadata information about the function. Its data is meant to be available/accessible in editor-only builds */
USTRUCT()
struct FSceneStateFunctionMetadata
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FGuid FunctionId;
#endif
};
