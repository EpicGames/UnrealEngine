// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "ISceneStateSchemaProvider.generated.h"

class USceneStateSchema;

UINTERFACE(MinimalAPI, meta=(CannotImplementInterfaceInBlueprint))
class USceneStateSchemaProvider : public UInterface
{
	GENERATED_BODY()
};

class ISceneStateSchemaProvider
{
	GENERATED_BODY()

public:
	/** Returns the schema used for the scene state */
	virtual TSubclassOf<USceneStateSchema> GetSceneStateSchemaClass() const = 0;
};
