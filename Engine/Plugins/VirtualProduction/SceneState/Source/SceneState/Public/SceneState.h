// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyBindingTypes.h"
#include "SceneStateEnums.h"
#include "SceneStateRange.h"
#include "SceneStateReentryGuard.h"
#include "StructUtils/PropertyBag.h"
#include "SceneState.generated.h"

#define UE_API SCENESTATE_API

struct FConstStructView;
struct FInstancedStructContainer;
struct FSceneStateExecutionContext;
struct FSceneStateInstance;
struct FStructView;

namespace UE::SceneState::Editor
{
	class FBindingCompiler;
	class FBlueprintCompilerContext;
	class FStateMachineCompiler;

} // UE::SceneState

/** Metadata information about the State. Available only in editor */
USTRUCT()
struct FSceneStateMetadata
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FString StateName;
#endif
};

/**
 * Runtime-immutable information of a state.
 * Holds the range of tasks, state machines and transitions belonging to the state.
 * These states are stored in the Scene State Generated Class.
 */
USTRUCT()
struct FSceneState
{
	GENERATED_BODY()

	/** Called the state becomes the active state in its parent state machine */
	UE_API void Enter(const FSceneStateExecutionContext& InContext) const;

	/** Called on every tick while the state is active */
	UE_API void Tick(const FSceneStateExecutionContext& InContext, float InDeltaSeconds) const;

	/** Called when the state is no longer active in its parent state machine */
	UE_API void Exit(const FSceneStateExecutionContext& InContext) const;

	/** Get the template parameters of the state */
	const FInstancedPropertyBag& GetParameters() const;

	/** Get the template parameter struct */
	const UStruct* GetParametersStruct() const;

	FSceneStateRange GetTaskRange() const
	{
		return TaskRange;
	}

	FSceneStateRange GetStateMachineRange() const
	{
		return StateMachineRange;
	}

	FSceneStateRange GetTransitionRange() const
	{
		return TransitionRange;
	}

	FSceneStateRange GetEventHandlerRange() const
	{
		return EventHandlerRange;
	}

	/** Starts pending tasks whose prerequisites are satisfied */
	void UpdateActiveTasks(const FSceneStateExecutionContext& InContext, FSceneStateInstance& InStateInstance) const;

	/** Returns the State Name used for logging. Returns an empty string view if metadata is not available */
	FStringView GetStateName(const FSceneStateExecutionContext& InContext) const;

	/** Returns whether there's any Task pending to finish */
	UE_API bool HasPendingTasks(const FSceneStateExecutionContext& InContext) const;

	/** Called on start to create the required task instances that will run for this state */
	UE_INTERNAL UE_API void AllocateTaskInstances(const FSceneStateExecutionContext& InContext, TConstArrayView<FConstStructView> InTemplateTaskInstances) const;

private:
	/** Applies Bindings to the given state instance */
	bool ApplyBindings(const FSceneStateExecutionContext& InContext, FSceneStateInstance& InStateInstance) const;

	/** Instances the instanced references in source data into the target data using the provided duplication functor */
	UE_INTERNAL UE_API void InstanceTaskObjects(UObject* InOuter
		, TConstArrayView<FStructView> InTargets
		, TConstArrayView<FConstStructView> InSources
		, TFunctionRef<UObject*(FObjectDuplicationParameters&)> InDuplicationFunc) const;

	/** Captures all the events of interest from the handlers this state owns */
	void CaptureEvents(const FSceneStateExecutionContext& InContext) const;

	/** Removes all the event data that was captured by the handlers this state owns */
	void ResetCapturedEvents(const FSceneStateExecutionContext& InContext) const;

	/** Template parameters to use for the state instances */
	UPROPERTY()
	FInstancedPropertyBag Parameters;

	/** Bindings batch where this state is the target */
	UPROPERTY()
	FPropertyBindingIndex16 BindingsBatch;

	/** Index and Count of the tasks that belong to this state. */
	UPROPERTY()
	FSceneStateRange TaskRange;

	/** Index and Count of the sub state machines that belong to this state. */
	UPROPERTY()
	FSceneStateRange StateMachineRange;

	/** Index and count of the exit transitions that go out of this state and into other targets (states, conduits, exit). */
	UPROPERTY()
	FSceneStateRange TransitionRange;

	/** Index and Count of the events that this state and its tasks and its sub state machines will be handling */
	UPROPERTY()
	FSceneStateRange EventHandlerRange;

	/**
	 * Handle to ensure reentry is not hit
	 * @see FSceneStateReentryGuard
	 */
	UE::SceneState::FReentryHandle ReentryHandle;

	friend UE::SceneState::Editor::FBindingCompiler;
	friend UE::SceneState::Editor::FBlueprintCompilerContext;
	friend UE::SceneState::Editor::FStateMachineCompiler;
};

#undef UE_API
