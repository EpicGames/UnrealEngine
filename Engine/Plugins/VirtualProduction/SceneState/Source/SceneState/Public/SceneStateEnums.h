// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "SceneStateEnums.generated.h"

namespace UE::SceneState
{

/** Defines the next actions for the current iteration */
enum class EIterationResult : uint8
{
	Continue,
	Break,
};

/** Defines the possible execution status of a State, Task and State Machine */
enum class EExecutionStatus : uint8
{
	NotStarted,
	Running,
	Finished,
};

/** Describes the different components of a task */
enum class ETaskObjectType : uint8
{
	None,
	/** Object deriving from FSceneStateTask that contains the logic to run */
	Task,
	/** Object deriving from FSceneStateTaskInstance containing the instance data to feed to the task */
	TaskInstance,
};

} // UE::SceneState

/** Defines common task behaviors: whether it should tick, whether it handles custom bindings, etc. */
UENUM()
enum class ESceneStateTaskFlags : uint8
{
	None = 0,

	/** Task will always call OnTick when Active */
	Ticks = 1 << 0,

	/** Task has custom binding extension */
	HasBindingExtension = 1 << 1,
};
ENUM_CLASS_FLAGS(ESceneStateTaskFlags);

/** Defines the possible ways a state machine will run */
UENUM()
enum class ESceneStateMachineRunMode : uint8
{
	/** The state machine will run automatically */
	Auto,

	/** The state machine will run through other means (e.g. State Machine Task) */
	Manual,
};

/** Defines the reasons why a task has stopped */
UENUM(BlueprintType)
enum class ESceneStateTaskStopReason : uint8
{
	/** State ended (transitioning, or end play) so it's forcing all its Active States to stop */
	State,

	/** Task was marked as finished */
	Finished,
};

/** Defines how a transition evaluation should take place */
UENUM()
enum class ESceneStateTransitionEvaluationFlags : uint8
{
	None = 0,

	/** Transition will not evaluate until all the current tasks have finished */
	WaitForTasksToFinish = 1 << 0,

	/**
	 * Transition Evaluation Event will not be processed and will always evaluate to true.
	 * This is set when the transition graph compiler finds that the transition evaluation event will always return true,
	 * so no event is created and this flag is set.
	 */
	EvaluationEventAlwaysTrue = 1 << 1,
};
ENUM_CLASS_FLAGS(ESceneStateTransitionEvaluationFlags);
