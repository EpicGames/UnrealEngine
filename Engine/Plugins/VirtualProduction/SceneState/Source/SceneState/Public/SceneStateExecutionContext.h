// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Conduit/SceneStateConduitInstance.h"
#include "Containers/Array.h"
#include "Functions/SceneStateFunctionInstanceBatch.h"
#include "SceneStateEnums.h"
#include "SceneStateExecutionContextHandle.h"
#include "SceneStateInstance.h"
#include "SceneStateMachineInstance.h"
#include "SceneStatePropertyUtils.h"
#include "StructUtils/InstancedStructContainer.h"
#include "Transition/SceneStateTransitionInstance.h"
#include "UObject/ObjectPtr.h"
#include "SceneStateExecutionContext.generated.h"

#define UE_API SCENESTATE_API

class UClass;
class USceneStateEventStream;
class USceneStateObject;
class USceneStatePlayer;
class USceneStateTemplateData;
struct FPropertyBindingDataView;
struct FPropertyBindingIndex16;
struct FSceneState;
struct FSceneStateBindingCollection;
struct FSceneStateBindingDataHandle;
struct FSceneStateConduit;
struct FSceneStateEventHandler;
struct FSceneStateFunction;
struct FSceneStateMachine;
struct FSceneStateMetadata;
struct FSceneStatePropertyReference;
struct FSceneStateTask;
struct FSceneStateTransition;

namespace UE::SceneState
{
	class FExecutionContextRegistry;
	class FExecutionScope;
}

/**
 * Struct representing an execution. It is used mainly to run a Scene State Object,
 * but is also used in to run the same state machines, state, etc. multiple times.
 * @see USceneStateObject
 * @see FSceneStateMachineTask
 *
 * This struct is the place holding the mutable data during execution, as the data residing in the Scene State Generated Class
 * is immutable during execution.
 *
 * This struct also offers functionality to both get class (immutable) objects (states, state machines, etc)
 * and their instance (mutable) data (state instances, state machine instances, etc).
 *
 * Prior to execution, ensure that the template data is in a valid state,
 * as it is assumed to be valid during execution to avoid re-checking validity.
 * See USceneStateObject as an example.
 */
USTRUCT()
struct FSceneStateExecutionContext
{
	GENERATED_BODY()

	UE_API static const FSceneStateExecutionContext InvalidContext;

	UE_API ~FSceneStateExecutionContext();

	/** Gets the registry that this context is registered to */
	TWeakPtr<const UE::SceneState::FExecutionContextRegistry> GetContextRegistry() const;

	/** Gets the context handle for this execution context, if any */
	UE::SceneState::FExecutionContextHandle GetContextHandle() const;

	const USceneStateTemplateData* GetTemplateData() const
	{
		return TemplateData;
	}

	/** Gets the binding collection owned by the template data */
	const FSceneStateBindingCollection& GetBindingCollection() const;

	/** Returns the Player Debug Name for logging purposes */
	UE_API const FString& GetExecutionContextName() const;

	/** Retrieves the Root Object of this Execution */
	UE_API USceneStateObject* GetRootObject() const;

	/** Retrieves the Context Object for this Execution */
	UE_API UObject* GetContextObject() const;

	/** Gets the Event Stream from the Root State if available */
	UE_API USceneStateEventStream* GetEventStream() const;

	/**
	 * Initialization for a context with a parent context.
	 * @param InParentContext the parent of this execution context
	 */
	UE_API void Setup(const FSceneStateExecutionContext& InParentContext);

	/**
	 * Initialization for the Context.
	 * Through the given root state it pre-allocates the instance data that will be used
	 * @param InRootState the parameters containing the template data and context for this execution
	 */
	UE_API void Setup(TNotNull<USceneStateObject*> InRootState);

	struct FSetupParams
	{
		/** Template data object holding all the state machines, states, etc. */
		const USceneStateTemplateData* TemplateData = nullptr;
		/** Root object instance */
		USceneStateObject* RootObject = nullptr;
		/** Context object for utility purposes (e.g. to get world from context, etc) */
		UObject* ContextObject = nullptr;
		/** Event stream to use for execution */
		USceneStateEventStream* EventStream = nullptr;
		/** Context registry to use */
		TSharedPtr<UE::SceneState::FExecutionContextRegistry> ContextRegistry;
		/** Debug name describing the context */
		FString ContextName;
	};
	/**
	 * Initialization for the Context.
	 * Through the given root state it pre-allocates the instance data that will be used
	 * @param InParams the parameters containing the template data and context for this execution
	 */
	UE_API void Setup(const FSetupParams& InParams);

	/** Called when cleaning up the instances of this execution */
	UE_API void Reset();

	/** Finds the data view that matches the given data handle */
	UE_API FPropertyBindingDataView FindDataView(const FSceneStateBindingDataHandle& InDataHandle) const;

	/** Returns the pointer to the property value address the given reference is mapped to */
	template<typename T = void>
	T* ResolveProperty(const FSceneStatePropertyReference& InPropertyReference) const
	{
		return UE::SceneState::ResolveProperty<T>(*this, InPropertyReference);
	}

	/** Invokes a given callable for each task in the given state */
	void ForEachTask(const FSceneState& InState, TFunctionRef<UE::SceneState::EIterationResult(const FSceneStateTask&, uint16 /*RelativeIndex*/)> InCallable) const;

	/** Returns an array of const views of the Template Task Instances of the given State */
	TArray<FConstStructView> GetTemplateTaskInstances(const FSceneState& InState) const;

	/** Returns a sliced view of the prerequisite task indices for a given task, in relative index based on the owning state's task range */
	TConstArrayView<uint16> GetTaskPrerequisites(const FSceneStateTask& InTask) const;

	/** Returns a sliced view of all the Exit Transitions going out of the given State */
	TConstArrayView<FSceneStateTransition> GetTransitions(const FSceneState& InState) const;

	/** Returns a sliced view of all the Exit Transitions going out of the given Conduit */
	TConstArrayView<FSceneStateTransition> GetTransitions(const FSceneStateConduit& InConduit) const;

	/** Gets the template transition parameter for the given transition */
	FInstancedPropertyBag GetTemplateTransitionParameter(const FSceneStateTransition& InTransition) const;

	/** Returns a sliced view of all the Sub State Machines belonging to the given State */
	TConstArrayView<FSceneStateMachine> GetStateMachines(const FSceneState& InState) const;

	/** Returns a sliced view of all the Event Handlers in the given State */
	TConstArrayView<FSceneStateEventHandler> GetEventHandlers(const FSceneState& InState) const;

#if WITH_EDITOR
	/** Returns the State Metadata of a given state */
	const FSceneStateMetadata* GetStateMetadata(const FSceneState& InState) const;
#endif

	/** Returns the State Machine linked to the given id */
	const FSceneStateMachine* GetStateMachine(const FGuid& InStateMachineId) const;

	/** Returns the State Machine at the given absolute index */
	const FSceneStateMachine* GetStateMachine(uint16 InAbsoluteIndex) const;

	/** Returns the most relevant State Machine in the current scope, if any */
	const FSceneStateMachine* GetContextStateMachine() const;

	/** Returns the currently Active State within this context for the given State Machine */
	const FSceneState* GetActiveState(const FSceneStateMachine& InStateMachine) const;

	/** Returns the State at the given absolute index */
	const FSceneState* GetState(uint16 InAbsoluteIndex) const;

	/** Returns the Event Handler at the given absolute Index */
	const FSceneStateEventHandler* GetEventHandler(uint16 InAbsoluteIndex) const;

	/** Returns the State at the given relative index for the given State Machine */
	const FSceneState* GetState(const FSceneStateMachine& InStateMachine, uint16 InRelativeIndex) const;

	/** Returns the Conduit at the given relative index for the given State Machine */
	const FSceneStateConduit* GetConduit(const FSceneStateMachine& InStateMachine, uint16 InRelativeIndex) const;

	/** Returns the State Instance of this context for the given state, adding a new one if it's not found */
	FSceneStateInstance& FindOrAddStateInstance(const FSceneState& InState) const;

	/** Returns the existing State Instance of this context for the given state absolute index. Null if not found */
	FSceneStateInstance* FindStateInstance(uint16 InAbsoluteIndex) const;

	/** Returns the existing State Instance of this context for the given state. Null if not found */
	FSceneStateInstance* FindStateInstance(const FSceneState& InState) const;

	/** Removes the State Instance of this context for the given state */
	void RemoveStateInstance(const FSceneState& InState) const;

	/** Iterates the active state instances of this context */
	void ForEachStateInstance(TFunctionRef<void(uint16, const FSceneStateInstance&)> InFunc) const;

	/** Returns the Task of this context for the given task absolute index. Returns an invalid view if not found */
	FConstStructView FindTask(uint16 InAbsoluteIndex) const;

	/** Gets the task index relative to the other tasks in the state */
	uint16 FindTaskRelativeIndex(const FSceneStateTask& InTask) const;

	/** Returns the Task Instance of this context for the given task. Returns an invalid view if not found */
	UE_INTERNAL UE_API FStructView FindTaskInstance(const FSceneStateTask& InTask) const;

	/** Returns the Task Instance of this context for the given task absolute index. Returns an invalid view if not found */
	FStructView FindTaskInstance(uint16 InTaskAbsoluteIndex) const;

	/** Returns the given task's execution status, or unset if task was not found */
	UE_INTERNAL UE_API TOptional<UE::SceneState::EExecutionStatus> GetTaskStatus(const FSceneStateTask& InTask) const;

	/** Set the task status of this context for the given task */
	void SetTaskStatus(const FSceneStateTask& InTask, UE::SceneState::EExecutionStatus InStatus) const;

	/** Returns the State Machine Instance of this context for the given state machine, adding a new one if it's not found */
	FSceneStateMachineInstance* FindOrAddStateMachineInstance(const FSceneStateMachine& InStateMachine) const;

	/** Returns the existing State Machine Instance of this context for the given state machine absolute index. Null if not found */
	FSceneStateMachineInstance* FindStateMachineInstance(uint16 InAbsoluteIndex) const;

	/** Returns the existing State Machine Instance of this context for the given state machine. Null if not found */
	FSceneStateMachineInstance* FindStateMachineInstance(const FSceneStateMachine& InStateMachine) const;

	/** Removes the State Machine Instance of this context for the given state machine */
	void RemoveStateMachineInstance(const FSceneStateMachine& InStateMachine) const;

	/** Returns the transition instance of this context for the given transition, adding a new one if it's not found */
	FSceneStateTransitionInstance* FindOrAddTransitionInstance(const FSceneStateTransition& InTransition) const;

	/** Returns the transition instance of this context for the given transition absolute index, or null if not found */
	FSceneStateTransitionInstance* FindTransitionInstance(uint16 InAbsoluteIndex) const;

	/** Returns the transition instance of this context for the given transition, or null if not found */
	FSceneStateTransitionInstance* FindTransitionInstance(const FSceneStateTransition& InTransition) const;

	/** Removes the transition instance of this context for the given transition */
	void RemoveTransitionInstance(const FSceneStateTransition& InTransition) const;

	/** Returns the conduit instance of this context for the given conduit, adding a new one if it's not found */
	FSceneStateConduitInstance* FindOrAddConduitInstance(const FSceneStateConduit& InConduit) const;

	/** Removes the conduit instance of this context for the given conduit */
	void RemoveConduitInstance(const FSceneStateConduit& InConduit) const;

	/**
	 * Allocates the function instance data for a given binding batch
	 * @return true if there were valid functions within the batch, false otherwise
	 */
	bool SetupFunctionInstances(FPropertyBindingIndex16 InBindingsBatch) const;

	/** Removes the function instance data for a given binding batch */
	void RemoveFunctionInstances(FPropertyBindingIndex16 InBindingsBatch) const;

	/** Returns the function of this context for the given function absolute index. Returns an invalid view if not found */
	FConstStructView FindFunction(uint16 InAbsoluteIndex) const;

	/** Returns the function instance of this context for the given function absolute index. Returns an invalid view if not found */
	FStructView FindFunctionInstance(uint16 InAbsoluteIndex) const;

	/** Called to set the status of the given state */
	void SetStateStatus(const FSceneState& InState, FSceneStateInstance& InStateInstance, UE::SceneState::EExecutionStatus InStatus) const;

	/** Retrieves the parent execution context if existing. Can be null */
	const FSceneStateExecutionContext* FindParentContext() const;

	/** Retrieves the Absolute Index of a State in the State array */
	uint16 GetStateIndex(const FSceneState& InState) const;

	/** Retrieves the Absolute Index of a State Machine in the State Machine array */
	uint16 GetStateMachineIndex(const FSceneStateMachine& InStateMachine) const;

	/** Retrieves the Absolute Index of a Transition in the Transitions array */
	uint16 GetTransitionIndex(const FSceneStateTransition& InTransition) const;

	/** Retrieves the Absolute Index of a Conduit in the Conduits array */
	uint16 GetConduitIndex(const FSceneStateConduit& InConduit) const;

private:
	/** Weak ptr to the context holding this execution */
	UPROPERTY()
	TWeakObjectPtr<UObject> ContextObjectWeak;

	/** Weak ptr to the event stream for this execution */
	UPROPERTY()
	TWeakObjectPtr<USceneStateEventStream> EventStreamWeak;

	/** Pointer to the root object owning this execution */
	UPROPERTY()
	TObjectPtr<USceneStateObject> RootObject;

	/** Description of the context */
	UPROPERTY()
	FString ContextName;

	/** Object owning the scene state template data */
	UPROPERTY()
	TObjectPtr<const USceneStateTemplateData> TemplateData;

	/**
	 * Map of State Index to its Instance Data
	 * It's allocated when the state starts and removed on exit.
	 */
	UPROPERTY()
	mutable TMap<uint16, FSceneStateInstance> StateInstances;

	/**
 	 * Map of State Machine Index to its Instance Data
 	 * It's allocated when the State Machine starts and removed on exit.
 	 */
	UPROPERTY()
	mutable TMap<uint16, FSceneStateMachineInstance> StateMachineInstances;

	/** Array of state machine absolute indices from outermost to innermost for this execution context */
	mutable TArray<uint16, TInlineAllocator<4>> StateMachineExecutionStack;

	/**
	 * Map of the transition absolute index to its instance data
	 * It's allocated when the state starts (along with the other exit transitions) and removed on state exit
	 */
	UPROPERTY()
	mutable TMap<uint16, FSceneStateTransitionInstance> TransitionInstances;

	/**
	 * Map of the conduit absolute index to its instance data
	 * It's allocated when a state that could evaluate the conduit starts and removed on state exit
	 */
	UPROPERTY()
	mutable TMap<uint16, FSceneStateConduitInstance> ConduitInstances;

	/**
	 * Map of the batch index to the function instances within the batch
	 * It's allocated when its owner is setup, and removed on the owners exit
	 */
	UPROPERTY()
	mutable TMap<FPropertyBindingIndex16, FSceneStateFunctionInstanceBatch> FunctionInstanceBatches;

	/**
	 * Map of the function absolute index to the view of the function instance
	 * It's allocated when its owner is setup, and removed on the owners exit
	 */
	mutable TMap<uint16, FStructView> FunctionInstanceViews;

	/** Context handle to this execution. */
	UE::SceneState::FExecutionContextHandle ContextHandle;

	/** Context handle to the parent execution context. Can be invalid. */
	UE::SceneState::FExecutionContextHandle ParentHandle;

	/** Weak reference to the registry containing this context */
	TWeakPtr<UE::SceneState::FExecutionContextRegistry> ContextRegistryWeak;

	friend class UE::SceneState::FExecutionScope;
};

#undef UE_API

/** FSceneStateExecutionContext is not trivially relocatable */
template<UE::CDerivedFrom<FSceneStateExecutionContext> T>
struct TIsTriviallyRelocatable<T>
{
	static inline constexpr bool Value = false;
};
