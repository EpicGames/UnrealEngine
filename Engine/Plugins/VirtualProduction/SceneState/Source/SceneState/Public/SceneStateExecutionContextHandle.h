// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Templates/TypeHash.h"

namespace UE::SceneState
{

/** Holds a unique number mapping a context in a registry */
class FExecutionContextHandle
{
public:
	bool IsValid() const
	{
		return Id != 0;
	}

	bool operator==(FExecutionContextHandle InOtherHandle) const
	{
		return Id == InOtherHandle.Id;
	}

	friend uint32 GetTypeHash(FExecutionContextHandle InHandle)
	{
		return ::GetTypeHash(InHandle.Id);
	}

private:
	uint64 Id = 0;

	friend class FExecutionContextRegistry;
};

} // UE::SceneState
