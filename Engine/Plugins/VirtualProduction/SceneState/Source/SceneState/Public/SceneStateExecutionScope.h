// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include <limits>

struct FSceneStateExecutionContext;
struct FSceneStateMachine;

namespace UE::SceneState
{

/**
 * Adds the given State Machine to the Execution Stack on scope enter / constructor
 * and removes it on scope exit / destructor.
 * Stores references and should be short-lived within a function's scope.
 */
class FExecutionScope
{
public:
	explicit FExecutionScope(const FSceneStateExecutionContext& InContext, const FSceneStateMachine& InStateMachine);

	~FExecutionScope();

private:
	/** The execution context containing the execution stack */
	const FSceneStateExecutionContext& ExecutionContext;

	/** Index to the state machine in the execution scope */
	uint16 StateMachineIndex = std::numeric_limits<uint16>::max();
};

} // UE::SceneState
