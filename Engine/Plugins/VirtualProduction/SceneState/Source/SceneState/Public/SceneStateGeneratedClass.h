// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/BlueprintGeneratedClass.h"
#include "SceneStateGeneratedClass.generated.h"

#define UE_API SCENESTATE_API

class USceneStateTemplateData;
class UUserDefinedStruct;

namespace UE::SceneState::Editor
{
	class FBindingCompiler;
	class FBindingFunctionCompiler;
	class FBlueprintCompilerContext;
	class FStateMachineCompiler;
}

/**
 * Object Class for the Scene State Object
 * Holds the template data for scene state objects.
 * @see USceneStateTemplateData
 * @see FSceneStateExecutionContext
 */
UCLASS(MinimalAPI)
class USceneStateGeneratedClass : public UBlueprintGeneratedClass
{
	GENERATED_BODY()

public:
	UE_API USceneStateGeneratedClass();

	const USceneStateTemplateData* GetTemplateData() const
	{
		return TemplateData;
	}

	//~ Begin UClass
	UE_API virtual void Link(FArchive& Ar, bool bInRelinkExistingProperties) override;
	UE_API virtual void PurgeClass(bool bInRecompilingOnLoad) override;
	//~ End UClass

	//~ Begin UObject
	UE_API virtual void PostLoad() override;
	UE_API virtual void BeginDestroy() override;
	//~ End UObject

private:
	/** Called to link the template data */
	void LinkData();

	/** Called to resolve the template data bindings */
	UE_INTERNAL UE_API void ResolveBindings();

	/** Returns true if this class is a 'full' class (i.e. not a SKEL_ or REINST_ class) */
	UE_INTERNAL UE_API bool IsFullClass() const;

#if WITH_EDITOR
	void OnObjectsReinstanced(const TMap<UObject*, UObject*>& InReplacementMap);
	void OnStructsReinstanced(const UUserDefinedStruct& InStruct);
#endif

	/** Scene State template data for this class. */
	UPROPERTY()
	TObjectPtr<USceneStateTemplateData> TemplateData;

#if WITH_EDITOR
	FDelegateHandle OnObjectsReinstancedHandle;
	FDelegateHandle OnStructsReinstancedHandle;
#endif

	friend UE::SceneState::Editor::FBindingCompiler;
	friend UE::SceneState::Editor::FBindingFunctionCompiler;
	friend UE::SceneState::Editor::FBlueprintCompilerContext;
	friend UE::SceneState::Editor::FStateMachineCompiler;
	friend class USceneStateBlueprint;
};

#undef UE_API
