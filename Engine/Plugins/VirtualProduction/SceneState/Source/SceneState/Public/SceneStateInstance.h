// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneStateEnums.h"
#include "StructUtils/InstancedStructContainer.h"
#include "StructUtils/PropertyBag.h"
#include "SceneStateInstance.generated.h"

#define UE_API SCENESTATE_API

struct FSceneState;

/** Instance data of a State */
USTRUCT()
struct FSceneStateInstance
{
	GENERATED_BODY()

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_API FSceneStateInstance();
	FSceneStateInstance(const FSceneStateInstance&) = default;
	FSceneStateInstance(FSceneStateInstance&&) = default;
	FSceneStateInstance& operator=(const FSceneStateInstance&) = default;
	FSceneStateInstance& operator=(FSceneStateInstance&&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Called to setup the state instance for use */
	void Setup(const FSceneState& InState);

	UE_DEPRECATED(5.8, "ElapsedTime is deprecated, it is no longer in use.")
	float ElapsedTime = 0.f;

	/** Gets the execution status of the state instance */
	UE_API UE::SceneState::EExecutionStatus GetStatus() const;

	/** Sets the execution status of the state instance */
	void SetStatus(UE::SceneState::EExecutionStatus InStatus);

	/** Current status of this state instance */
	UE_DEPRECATED(5.8, "Public access to Status is deprecated, and it will become private in a future release. Please update code to use the setter/getter instead.")
	UE::SceneState::EExecutionStatus Status = UE::SceneState::EExecutionStatus::NotStarted;

	/** Get an identifier for this instance */
	UE_INTERNAL UE_API uint16 GetInstanceId() const;

	/** Returns the state parameters */
	FInstancedPropertyBag& GetParameters();

	/** Returns the instance data of the tasks owned by this state */
	FInstancedStructContainer& GetTaskInstances();

	/** Returns the status of the task at the given index */
	UE::SceneState::EExecutionStatus GetTaskStatus(uint16 InTaskRelativeIndex) const;

	/** Sets the status of the task at the given index */
	void SetTaskStatus(uint16 InTaskRelativeIndex, UE::SceneState::EExecutionStatus InTaskStatus);

private:
	/** Instanced state parameters */
	UPROPERTY()
	FInstancedPropertyBag Parameters;

	/** The task instances owned by this state */
	UPROPERTY()
	FInstancedStructContainer TaskInstances;

	/** Execution status of the tasks */
	TArray<UE::SceneState::EExecutionStatus> TaskStatuses;

	/** The id for this instance. This is used to differentiate state instances for a same state and consequently task instances for a same task*/
	uint16 InstanceId;
};

#undef UE_API
