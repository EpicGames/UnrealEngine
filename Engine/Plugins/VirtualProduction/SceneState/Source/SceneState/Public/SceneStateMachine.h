// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyBindingTypes.h"
#include "SceneStateEnums.h"
#include "SceneStateRange.h"
#include "StructUtils/PropertyBag.h"
#include "SceneStateMachine.generated.h"

struct FSceneState;
struct FSceneStateExecutionContext;
struct FSceneStateMachineInstance;
struct FSceneStateTransition;

namespace UE::SceneState
{
	struct FTransitionEvaluationParams;

	namespace Editor
	{
		class FBindingCompiler;
		class FStateMachineCompiler;
	}
}

/**
 * Runtime-immutable information of a state machine.
 * Holds the range of states it will run, as well as the conduits found in the state machine, and other template data like template state machine parameters.
 * These state machines are stored in the Scene State Generated Class.
 */
USTRUCT()
struct FSceneStateMachine
{
	GENERATED_BODY()

	bool IsValid() const
	{
		return EntryIndex < StateRange.Count;
	}

	void Setup(const FSceneStateExecutionContext& InContext) const;

	void Start(const FSceneStateExecutionContext& InContext) const;

	void Tick(const FSceneStateExecutionContext& InContext, float InDeltaSeconds) const;

	void Stop(const FSceneStateExecutionContext& InContext) const;

	FSceneStateRange GetStateRange() const
	{
		return StateRange;
	}

	FSceneStateRange GetConduitRange() const
	{
		return ConduitRange;
	}

	uint16 GetEntryIndex() const
	{
		return EntryIndex;
	}

	const UStruct* GetParametersStruct() const
	{
		return Parameters.GetPropertyBagStruct();
	}

private:
	/** Called to evaluate the exit transitions from the active state */
	void EvaluateStateTransitions(const FSceneStateExecutionContext& InContext, FSceneStateMachineInstance& InInstance, const FSceneState& InActiveState) const;

	/**
	 * Evaluates whether the current context has any exit transitions satisfied.
	 * @param InTransitions the transitions to evaluate, already ordered from the highest priority to the lowest priority
	 * @param InEvaluationParams the parameters to use when evaluating a transition
	 * @return returns the highest priority transition satisfied, or null if no transitions were satisfied
	 */
	const FSceneStateTransition* EvaluateTransitions(TConstArrayView<FSceneStateTransition> InTransitions, const UE::SceneState::FTransitionEvaluationParams& InEvaluationParams) const;

	/**
	 * Follows the transition and returns a valid transition that would lead to a state or exit.
	 * @param InTransition the transition to follow
	 * @param InEvaluationParams the parameters to use when evaluating a transition
	 * @return returns the highest priority transition satisfied, or null if no transitions were satisfied
	 */
	const FSceneStateTransition* FollowTransition(const FSceneStateTransition& InTransition, const UE::SceneState::FTransitionEvaluationParams& InEvaluationParams) const;

	/** Applies Bindings to the given State Machine Instance */
	bool ApplyBindings(const FSceneStateExecutionContext& InContext, FSceneStateMachineInstance& InStateMachineInstance) const;

	/** Stops the current active state and resets the active time to 0 */
	void StopActiveState(const FSceneStateExecutionContext& InContext, FSceneStateMachineInstance& InInstance) const;

	/** Template Parameters to use to Instance the Scene State Machine Instance */
	UPROPERTY()
	FInstancedPropertyBag Parameters;

	/** Bindings Batch where this State Machine is target */
	UPROPERTY()
	FPropertyBindingIndex16 BindingsBatch;

	/** Index and count of the states that belong to this state machine. */
	UPROPERTY()
	FSceneStateRange StateRange;

	/** Index and count of the conduits that belong to this state machine. */
	UPROPERTY()
	FSceneStateRange ConduitRange;

	/**
	 * Relative Index of the entry state that the State Machine will start in
	 * AbsoluteEntryIndex = StateRange.Index (absolute) + EntryIndex (relative)
	 */
	UPROPERTY()
	uint16 EntryIndex = FSceneStateRange::InvalidIndex;

	friend UE::SceneState::Editor::FBindingCompiler;
	friend UE::SceneState::Editor::FStateMachineCompiler;
};
