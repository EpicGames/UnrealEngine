// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneStateEnums.h"
#include "SceneStateRange.h"
#include "StructUtils/PropertyBag.h"
#include "SceneStateMachineInstance.generated.h"

/** Instance data of a State Machine */
USTRUCT()
struct FSceneStateMachineInstance
{
	GENERATED_BODY()

	/** Instanced Parameters for the given State Machine */
	UPROPERTY()
	FInstancedPropertyBag Parameters;

	/**
	 * Relative Index of the currently Active State
	 * AbsoluteActiveIndex = StateRange.Index (absolute) + ActiveIndex (relative)
	 */
	uint16 ActiveIndex = FSceneStateRange::InvalidIndex;

	/** Current status of the state machine instance */
	UE::SceneState::EExecutionStatus Status = UE::SceneState::EExecutionStatus::NotStarted;
};
