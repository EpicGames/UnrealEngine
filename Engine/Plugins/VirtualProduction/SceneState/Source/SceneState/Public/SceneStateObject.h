// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "SceneState.h"
#include "SceneStateEnums.h"
#include "SceneStateEventStream.h"
#include "SceneStateExecutionContext.h"
#include "SceneStateRange.h"
#include "UObject/Object.h"
#include "SceneStateObject.generated.h"

#define UE_API SCENESTATE_API

class USceneStateBlueprintableTask;
class USceneStateEventStream;
class USceneStateTemplateData;
struct FPropertyBindingDataView;
struct FSceneStateBindingDataHandle;
struct FSceneStateExecutionContext;

namespace UE::SceneState
{
	class FDebugger;
	class FExecutionContextRegistry;
}

/**
 * Object instanced by USceneStateGeneratedClass.
 * Holds the execution context (and instance data) that will be used to run the Scene State the USceneStateGeneratedClass holds.
 * @see USceneStateGeneratedClass
 */
UCLASS(MinimalAPI, Blueprintable)
class USceneStateObject : public UObject
{
	GENERATED_BODY()

public:
	UE_API USceneStateObject();

	/** Gets the context name of the scene state player */
	UE_API FString GetContextName() const;

	const USceneStateTemplateData* GetTemplateData() const
	{
		return TemplateData;
	}

	/** Gets the context object of the scene state player */
	UFUNCTION(BlueprintCallable, Category="State")
	UE_API UObject* GetContextObject() const;

	UFUNCTION(BlueprintPure, Category="Events")
	USceneStateEventStream* GetEventStream() const
	{
		return EventStream;
	}

	/** Returns true if the object has an active root state */
	UFUNCTION(BlueprintPure, Category="State")
	UE_API bool IsActive() const;

	/** Called by the scene state player to setup this object and the root execution context */
	void Setup();

	/** Called by the scene state player to start execution of the root state  */
	void Enter();

	/** Called by the scene state player to start ticking the root state */
	void Tick(float InDeltaSeconds);

	/** Called by the scene state player to stop the execution of the root state */
	void Exit();

	UFUNCTION(BlueprintImplementableEvent, DisplayName = "Enter", Category = "State Events")
	void ReceiveEnter();

	UFUNCTION(BlueprintImplementableEvent, DisplayName = "Tick", Category = "State Events")
	void ReceiveTick(float InDeltaSeconds);

	UFUNCTION(BlueprintImplementableEvent, DisplayName = "Exit", Category = "State Events")
	void ReceiveExit();

	/** Gets the context registry owned by this object */
	UE_API TSharedRef<UE::SceneState::FExecutionContextRegistry> GetContextRegistry() const;

#if WITH_EDITOR
	/** Gets the currently attached debugger, if any */
	UE_INTERNAL UE_API TSharedPtr<UE::SceneState::FDebugger> GetDebugger() const;

	/** Attaches a new debugger, detaching the old one if any */
	void SetDebugger(const TWeakPtr<UE::SceneState::FDebugger>& InDebuggerWeak);
#endif

	//~ Begin UObject
	UE_API virtual UWorld* GetWorld() const override final;
	UE_API virtual void BeginDestroy() override;
	//~ End UObject

private:
	/** Resolves the root state from the template data, if any */
	const FSceneState* GetRootState() const;

	/** Scene State template data to use for this object's execution. */
	UPROPERTY(Transient, DuplicateTransient, TextExportTransient, NonTransactional, meta=(NoBinding))
	TObjectPtr<const USceneStateTemplateData> TemplateData;

	/** Runtime Event System keeping track of Active Events that have been added */
	UPROPERTY(Transient, Instanced, meta=(NoBinding))
	TObjectPtr<USceneStateEventStream> EventStream;

	/** The top-level execution context of the Scene State */
	UPROPERTY(Transient, DuplicateTransient, TextExportTransient, NonTransactional, meta=(NoBinding))
	FSceneStateExecutionContext RootExecutionContext;

	/** The registry containing all the execution contexts in this object */
	TSharedPtr<UE::SceneState::FExecutionContextRegistry> ContextRegistry;

#if WITH_EDITOR
	/** Debugger attached to this object */
	TWeakPtr<UE::SceneState::FDebugger> DebuggerWeak;
#endif
};

#undef UE_API
