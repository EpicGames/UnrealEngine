// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SubclassOf.h"
#include "UObject/Object.h"
#include "SceneStatePlayer.generated.h"

#define UE_API SCENESTATE_API

class USceneStateObject;

/**
 * Scene State Players is the layer between the Context Object and the rest of Scene State
 * It instances a Scene State Object from a given class.
 * These players exist to keep shared logic re-usable across multiple possible implementers
 */
UCLASS(MinimalAPI)
class USceneStatePlayer : public UObject
{
	GENERATED_BODY()

public:
	TSubclassOf<USceneStateObject> GetSceneStateClass() const
	{
		return SceneStateClass;
	}

	/** Sets the class of the root state object. Destroying the existing one */
	UE_API void SetSceneStateClass(TSubclassOf<USceneStateObject> InSceneStateClass);

	USceneStateObject* GetSceneState() const
	{
		return RootState;
	}

	/** Returns the context name for this player, for debugging purposes */
	UE_API FString GetContextName() const;

	/** Returns the context object for this player */
	UE_API UObject* GetContextObject() const;

	/** Should be called prior to the start of the execution */
	UE_API void Setup();

	/** Should be called to start the execution of the scene state object */
	UE_API void Begin();

	/** Should be called every tick during the execution of the scene state object */
	UE_API void Tick(float InDeltaTime);

	/** Should be called to end the execution of the scene state object */
	UE_API void End();

	//~ Begin UObject
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	static FName GetSceneStateClassName()
	{
		return GET_MEMBER_NAME_CHECKED(USceneStatePlayer, SceneStateClass);
	}

	static FName GetRootStateName()
	{
		return GET_MEMBER_NAME_CHECKED(USceneStatePlayer, RootState);
	}

protected:
	/** Returns the context name for this player */
	virtual bool OnGetContextName(FString& OutContextName) const
	{
		return false;
	}

	/** Returns the context object for this player */
	virtual bool OnGetContextObject(UObject*& OutContextObject) const
	{
		return false;
	}

	/** Scene State class used to instantiate the Scene State */
	UPROPERTY(EditAnywhere, Category="Scene State", meta=(NoBinding, EditCondition="bEditableSceneStateClass", EditConditionHides, HideEditConditionToggle))
	TSubclassOf<USceneStateObject> SceneStateClass;

	/** Root Scene State object that this Component will run */
	UPROPERTY(VisibleAnywhere, Instanced, Category="Scene State", meta=(AllowPrivateAccess="true", ShowInnerProperties))
	TObjectPtr<USceneStateObject> RootState;

#if WITH_EDITORONLY_DATA
	/** Whether the scene state class is editable in Details */
	UPROPERTY()
	bool bEditableSceneStateClass = true;
#endif
};

#undef UE_API
