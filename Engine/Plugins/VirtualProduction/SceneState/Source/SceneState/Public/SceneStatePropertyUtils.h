// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneStateBindingReference.h"
#include "SceneStatePropertyReferenceUtils.h"

#define UE_API SCENESTATE_API

struct FSceneStateExecutionContext;
struct FSceneStatePropertyReference;

namespace UE::SceneState
{
	struct FResolvePropertyResult
	{
		/** the resolved value address */
		uint8* ValuePtr = nullptr;
		/** the resolved property reference mapped to the value address (can differ from the resolved reference mapped to InPropertyReference in recursive scenarios) */
		const FSceneStateBindingResolvedReference* ResolvedReference = nullptr;
	};
	/**
	 * Resolves the given reference and execution context to the value address of the property, and keeps iterating if the source is a property reference too.
	 * @param InContext the execution context containing the binding collection and way to retrieve the data views
	 * @param InPropertyReference the property reference to resolve access for
	 * @param OutResult if successful, a result containing a valid resolved reference and a valid value address.
	 * @return whether the operation succeeded
	 */
	UE_API bool ResolveProperty(const FSceneStateExecutionContext& InContext, const FSceneStatePropertyReference& InPropertyReference, FResolvePropertyResult& OutResult);

	template<class T>
	T* ResolveProperty(const FSceneStateExecutionContext& InContext, const FSceneStatePropertyReference& InPropertyReference)
	{
		FResolvePropertyResult Result;
		if (ResolveProperty(InContext, InPropertyReference, Result) && TDataTypeHelper<T>::IsValid(Result.ResolvedReference->SourceLeafProperty))
		{
			return reinterpret_cast<T*>(Result.ValuePtr);
		}
		return nullptr;
	}

} // UE::SceneState

#undef UE_API
