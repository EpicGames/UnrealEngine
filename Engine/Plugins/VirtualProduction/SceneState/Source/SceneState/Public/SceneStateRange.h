// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include <limits>
#include "SceneStateRange.generated.h"

#define UE_API SCENESTATE_API

USTRUCT()
struct FSceneStateRange
{
	GENERATED_BODY()

	static constexpr uint16 InvalidIndex = std::numeric_limits<uint16>::max();

	/** Makes a range with the given begin and end range, where end is exclusive */
	UE_API static FSceneStateRange MakeBeginEndRange(uint16 InBegin, uint16 InEnd);

	uint16 GetLastIndex() const
	{
		checkSlow(IsValid());
		return Index + Count - 1;
	}

	bool IsValid() const
	{
		return Count > 0 && Index <= InvalidIndex - Count;
	}

	UPROPERTY()
	uint16 Index = FSceneStateRange::InvalidIndex;

	UPROPERTY()
	uint16 Count = 0;
};

#undef UE_API
