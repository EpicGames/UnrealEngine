// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

struct FSceneStateExecutionContext;

namespace UE::SceneState
{

class FReentryHandle
{
	friend struct FReentryGuard;
	mutable bool bValue = false;
};

/** Ensures no reentry in a given scope */
struct FReentryGuard
{
	explicit FReentryGuard(const FReentryHandle& InCurrent, const FSceneStateExecutionContext& InContext);

	~FReentryGuard();

	bool IsReentry() const;

private:
	/** Reference to the handle that is used to monitor reentry */
	const FReentryHandle& Reference;

	/** Original value prior to this guard's construction. Used to restore the reference when this object destroys */
	FReentryHandle Original;
};

} // UE::SceneState
