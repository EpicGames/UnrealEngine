// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SubScriptStructOf.h"
#include "Templates/SubclassOf.h"
#include "UObject/Object.h"
#include "SceneStateSchema.generated.h"

#define UE_API SCENESTATE_API

struct FAssetData;
struct FSceneStateTask;

/** Determines the rules (e.g. allowed tasks) and provides context for a Scene State object type. */
UCLASS(MinimalAPI, Abstract)
class USceneStateSchema : public UObject
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	/** Determines whether the given task struct is allowed */
	UE_API bool IsTaskStructAllowed(TSubScriptStructOf<FSceneStateTask> InTaskStruct) const;

	/** Determines whether the given task asset (task blueprint) is allowed */
	UE_API bool IsTaskAssetAllowed(const FAssetData& InTaskAsset) const;
#endif

	/** Return the context object class supported */
	UE_API TSubclassOf<UObject> GetContextObjectClass() const;

protected:
#if WITH_EDITOR
	/** Determines whether a given task struct is allowed */
	virtual bool OnIsTaskStructAllowed(TSubScriptStructOf<FSceneStateTask> InTaskStruct) const
	{
		return true;
	}

	/** Determines whether the given task asset (task blueprint) is allowed */
	virtual bool OnIsTaskAssetAllowed(const FAssetData& InTaskAsset) const
	{
		return true;
	}
#endif

	/** Return the context object class supported. Called only once to keep the context object class constant throughout the schema. */
	virtual TSubclassOf<UObject> OnGetContextObjectClass() const
	{
		return TSubclassOf<UObject>();
	}

private:
	UPROPERTY()
	mutable TSubclassOf<UObject> ContextObjectClass;
};

#undef UE_API
