// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Conduit/SceneStateConduit.h"
#include "Conduit/SceneStateConduitLink.h"
#include "Containers/Array.h"
#include "SceneState.h"
#include "SceneStateBindingCollection.h"
#include "SceneStateEventHandler.h"
#include "SceneStateMachine.h"
#include "SceneStateRange.h"
#include "StructUtils/InstancedStructContainer.h"
#include "StructUtils/PropertyBag.h"
#include "Transition/SceneStateTransition.h"
#include "Transition/SceneStateTransitionLink.h"
#include "UObject/Object.h"
#include "UObject/ObjectKey.h"

#if WITH_EDITOR
#include "Functions/SceneStateFunctionMetadata.h"
#include "Tasks/SceneStateTaskMetadata.h"
#include "Transition/SceneStateTransitionMetadata.h"
#endif

#include "SceneStateTemplateData.generated.h"

#define UE_API SCENESTATE_API

namespace UE::SceneState
{
	class FExecutionContextRegistry;
}

struct FSceneStateInstance;
struct FSceneStateTask;
struct FSceneStateTaskInstance;

/**
 * Holds all the data about States, State Machines, Tasks, etc.
 * All this data is immutable in execution, and as such, it is not instanced to the Scene State Object instances.
 * @see FSceneStateExecutionContext
 */
UCLASS(MinimalAPI)
class USceneStateTemplateData : public UObject
{
	GENERATED_BODY()

public:
	/** Gets the state that contains all top-level state machines */
	const FSceneState* GetRootState() const;

	/** Finds the state machine for a given state machine guid */
	const FSceneStateMachine* FindStateMachine(const FGuid& InStateMachineId) const;

	/** Finds the struct type for the given data handle */
	const UStruct* FindDataStruct(const UClass* InOwnerClass, const FSceneStateBindingDataHandle& InDataHandle);

	/** Patches bindings and resolves the binding paths for the owning binding collection */
	void ResolveBindings(const UClass* InOwnerClass);

	/** Called at link time to cache the properties and functions to use (e.g. in transitions or conduits) */
	void Link(const UClass* InOwnerClass);

	/** Resets all the elements that get compiled for this generated class (e.g. Tasks, State Machines, etc) */
	UE_API void Reset();

#if WITH_EDITOR
	/** Finds the mapped compiled state for a given state node */
	UE_API const FSceneState* FindStateFromNode(FObjectKey InStateNode) const;

	/** Finds the mapped compiled task for a given task node */
	UE_API const FSceneStateTask* FindTaskFromNode(FObjectKey InTaskNode) const;

	UE_DEPRECATED(5.8, "ForEachStateInstance deprecated. No longer supported.")
	UE_API void ForEachStateInstance(const TSharedRef<UE::SceneState::FExecutionContextRegistry>& InContextRegistry
		, FObjectKey InStateNode
		, TFunctionRef<void(const FSceneStateInstance&)> InFunctor) const;

	/**
	 * For a given root state and task index retrieves the mapped task status
	 * @param InContextRegistry the registry containing all the execution contexts
	 * @param InTaskIndex the task index to look for
	 * @param InFunctor the functor to run for each task status
	 */
	UE_INTERNAL UE_API void ForEachTaskStatus(const TSharedRef<UE::SceneState::FExecutionContextRegistry>& InContextRegistry
		, uint16 InTaskIndex
		, TFunctionRef<void(UE::SceneState::EExecutionStatus)> InFunctor) const;

	UE_DEPRECATED(5.8, "ForEachTaskInstance deprecated. No longer supported.")
	UE_API void ForEachTaskInstance(const TSharedRef<UE::SceneState::FExecutionContextRegistry>& InContextRegistry
		, FObjectKey InTaskNode
		, TFunctionRef<void(const FSceneStateTaskInstance&)> InFunctor) const;

	void OnObjectsReinstanced(const UClass* InOwnerClass, const TMap<UObject*, UObject*>& InReplacementMap);
	void OnStructsReinstanced(const UClass* InOwnerClass, const UUserDefinedStruct& InStruct);
#endif

	/** Container for all the bindings */
	UPROPERTY()
	FSceneStateBindingCollection BindingCollection;

	/** Index to the state that contains all top-level state machines */
	UPROPERTY()
	uint16 RootStateIndex = FSceneStateRange::InvalidIndex;

	/** Holds all the compiled states including nested states */
	UPROPERTY()
	TArray<FSceneState> States;

	/** Holds all the compiled tasks (including tasks from nested states) in a contiguous block of memory */
	UPROPERTY()
	FInstancedStructContainer Tasks;

	/** Array of the task prerequisites in their relative index. Each task has a unique range indicating their pre-requisites */
	UPROPERTY()
	TArray<uint16> TaskPrerequisites;

	/** Templates used to instantiate the Task Instances */
	UPROPERTY()
	FInstancedStructContainer TaskInstances;

	/** Holds all the compiled functions (including from nested sources) in a contiguous block of memory */
	UPROPERTY()
	FInstancedStructContainer Functions;

	/** Templates used to instantiate the function data */
	UPROPERTY()
	FInstancedStructContainer FunctionInstances;

	/** Holds all the compiled event handlers (including from nested sources) contiguously */
	UPROPERTY()
	TArray<FSceneStateEventHandler> EventHandlers;

	/** Holds all the exit transitions (including nested sources) contiguously */
	UPROPERTY()
	TArray<FSceneStateTransition> Transitions;

	/** Compiled transition information only used in Link time */
	UPROPERTY()
	TArray<FSceneStateTransitionLink> TransitionLinks;

	/** Map of the Transition Index (absolute) to the template transition parameters map for evaluation function call. */
	UPROPERTY()
	TMap<uint16, FInstancedPropertyBag> TransitionParameters;

	/** All the compiled conduits */
	UPROPERTY()
	TArray<FSceneStateConduit> Conduits;

	/** Compiled conduit information used only in Link time */
	UPROPERTY()
	TArray<FSceneStateConduitLink> ConduitLinks;

	UPROPERTY()
	TArray<FSceneStateMachine> StateMachines;

	/** Map of the state machine parameters id to the index in the state machine array */
	UPROPERTY()
	TMap<FGuid, uint16> StateMachineIdToIndex;

	/** Map of the state parameters id to the index in the state array */
	UPROPERTY()
	TMap<FGuid, uint16> StateParameterIdToIndex;

#if WITH_EDITORONLY_DATA
	/** Holds all the state editor-only metadata. This array matches in index with the states array. */
	UPROPERTY()
	TArray<FSceneStateMetadata> StateMetadata;

	/** Holds all the task editor-only metadata. This array matches in index with the tasks array. */
	UPROPERTY()
	TArray<FSceneStateTaskMetadata> TaskMetadata;

	/** Holds all the function editor-only metadata. This array matches in index with the functions. */
	UPROPERTY()
	TArray<FSceneStateFunctionMetadata> FunctionMetadata;

	/** Holds all the transition editor-only metadata. This array matches in index with the transitions array. */
	UPROPERTY()
	TArray<FSceneStateTransitionMetadata> TransitionMetadata;

	/** 
	 * Map of a State Node to its State Index.
	 * Set after a compilation occurs.
	 */
	TMap<FObjectKey, uint16> StateNodeToIndex;

	/** 
	 * Map of a State Machine Graph to its State Machine Index.
	 * Set after a compilation occurs.
	 */
	TMap<FObjectKey, uint16> StateMachineGraphToIndex;

	/** 
	 * Map of a Task Node to its Task Index.
	 * Set after a compilation occurs.
	 */
	TMap<FObjectKey, uint16> TaskNodeToIndex;
#endif
};

#undef UE_API
