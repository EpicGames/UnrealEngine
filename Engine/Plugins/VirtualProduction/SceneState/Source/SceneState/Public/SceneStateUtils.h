// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "HAL/Platform.h"
#include "PropertyBindingDataView.h"
#include "Templates/Function.h"

#define UE_API SCENESTATE_API

class UClass;
class UObject;
struct FConstStructView;
struct FInstancedStructContainer;
struct FPropertyBindingIndex16;
struct FPropertyVisitorPath;
struct FSceneStateExecutionContext;
struct FSceneStateRange;
struct FStructView;
template<typename T> struct TObjectPtr;

namespace UE::SceneState
{
	/**
	 * Converts a map of keys to relative indices to a map of keys to absolute indices
	 * @param InOutMap the map to convert
	 * @param InIndexOffset the index to add to these relative indices to transform them to absolute
	 */
	template<typename InKeyType, typename InIndexType = uint16>
	void ToAbsoluteIndexMap(TMap<InKeyType, InIndexType>& InOutMap, InIndexType InIndexOffset)
	{
		for (TPair<InKeyType, InIndexType>& Pair : InOutMap)
		{
			Pair.Value += InIndexOffset;
		}
	}

	UE_API TArray<FStructView> GetStructViews(FInstancedStructContainer& InStructContainer, FSceneStateRange InRange);

	UE_API TArray<FStructView> GetStructViews(FInstancedStructContainer& InStructContainer);

	UE_API TArray<FConstStructView> GetConstStructViews(const FInstancedStructContainer& InStructContainer, FSceneStateRange InRange);

	UE_API TArray<FConstStructView> GetConstStructViews(const FInstancedStructContainer& InStructContainer);

	/** Determines whether the given begin and end forms a valid range */
	bool IsValidRange(FPropertyBindingIndex16 InBegin, FPropertyBindingIndex16 InEnd);

	struct FApplyBatchParams
	{
		/** Batch of the bindings to apply */
		uint16 BindingsBatch;
		/** Target Data to copy the source data to */
		FPropertyBindingDataView TargetDataView;
	};
	/** Applies a given binding batch to a target view within an execution context */
	bool ApplyBatch(const FSceneStateExecutionContext& InContext, const FApplyBatchParams& InParams);

	UE_DEPRECATED(5.8, "Use PropertyVisitorHelpers::ResolveVisitedPath instead")
	UE_API void* ResolveVisitedPath(const UScriptStruct* InRootObject, void* InRootData, const FPropertyVisitorPath& InPath);

	/**
	 * Discards a given object by changing its outer to transient package and adding a prefix to it while also ensuring uniqueness
	 * @param InObjectToDiscard the object to discard
	 */
	UE_API void DiscardObject(UObject* InObjectToDiscard);

	/**
	 * Discards an object with a given name in the given outer,
	 * by renaming the outer to the transient package and adding a prefix to it while also ensuring uniqueness
	 * @param InOuter outer of the object. Must be valid.
	 * @param InObjectName name to use. Must match InOutObject's name if not null
	 * @param InOnPreDiscardOldObject callback for when the old object is about to be trashed
	 * @return the object that was discarded, or null if none was found
	 */
	UE_API UObject* DiscardObject(UObject* InOuter
		, const TCHAR* InObjectName
		, TFunctionRef<void(UObject*)> InOnPreDiscardOldObject = [](UObject*){});

	/**
	 * Replaces a given Object with a new object with the same name but different class
	 * @param InOutObject object to replace. Can come in as null
	 * @param InOuter outer of the object. Must be valid.
	 * @param InClass new class of the new object that will replace the older one
	 * @param InObjectName name to use. Must match InOutObject's name if not null
	 * @param InContextName context for debugging purposes
	 * @param InOnPreDiscardOldObject callback for when the old object is about to be trashed
	 * @return true if the operation took place, false otherwise
	 */
	UE_API bool ReplaceObject(UObject*& InOutObject
		, UObject* InOuter
		, UClass* InClass
		, const TCHAR* InObjectName
		, const TCHAR* InContextName = nullptr
		, TFunctionRef<void(UObject*)> InOnPreDiscardOldObject = [](UObject*){});
	
#if WITH_EDITOR
namespace Editor
{
	/** Returns true if the struct or its super structs contains the given metadata. */
	UE_API bool StructHasMetaData(const UStruct* InStruct, FName InMetaData, bool bInIncludeSuperStructs);

	/** Returns true if the struct or its super structs contains any of the given metadata. */
	UE_API bool StructHasAnyMetaData(const UStruct* InStruct, TConstArrayView<FName> InMetaData, bool bInIncludeSuperStructs);

} // UE::SceneState::Editor
#endif

} // UE::SceneState

#undef UE_API
