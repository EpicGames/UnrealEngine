// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"

#define UE_API SCENESTATE_API

namespace UE::SceneState
{

/** Describes the versions of scene state */
struct FVersion
{
private:
	FVersion() = delete;

public:
	enum Type : uint8
	{
		PreVersioning = 0,

		/** Added schemas to set rules for scene state blueprints */
		Schema,

		/* ------------------------------------------------------ */
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	UE_API const static FGuid Guid;
};

} // UE::SceneState

#undef UE_API
