// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISceneStateEventHandlerProvider.h"
#include "PropertyBindingTypes.h"
#include "SceneStateEnums.h"
#include "SceneStateRange.h"
#include "SceneStateTaskExecutionContext.h"
#include "UObject/Object.h"
#include "SceneStateBlueprintableTask.generated.h"

class USceneStateEventStream;
class USceneStateObject;
struct FSceneStateBlueprintableTaskWrapper;
struct FSceneStateExecutionContext;

/**
 * Base class for Blueprint Tasks.
 * Following the design of blueprints, unlike C++ Tasks that have Logic and Instance Data separated, 
 * Blueprint Tasks holds the logic and mutable instance data together and so get allocated when the task starts.
 * @see FSceneStateBlueprintableTaskWrapper
 */
UCLASS(MinimalAPI, Abstract, Blueprintable, BlueprintType, AutoExpandCategories=("Task Settings"), EditInlineNew)
class USceneStateBlueprintableTask : public UObject, public ISceneStateEventHandlerProvider
{
	GENERATED_BODY()

	friend FSceneStateBlueprintableTaskWrapper;

public:
	/** Called once when the Task starts */
	UFUNCTION(BlueprintImplementableEvent, Category="Task")
	void ReceiveStart();

	/**
	 * Called every frame after the Task has started
	 * Must have bCanTick set to true
	 * @see USceneStateTask::CanTick
	 */
	UFUNCTION(BlueprintImplementableEvent, Category="Task")
	void ReceiveTick(float InDeltaSeconds);

	/** Called once when the Task ends */
	UFUNCTION(BlueprintImplementableEvent, Category="Task")
	void ReceiveStop(ESceneStateTaskStopReason InStopReason);

	/** Gets the scene state object that owns this blueprint task */
	UFUNCTION(BlueprintCallable, Category="Task")
	USceneStateObject* GetRootState() const;

	/** Gets the execution context of the task */
	const FSceneStateExecutionContext& GetExecutionContext() const;

	/** Gets the context object of the scene state object */
	UFUNCTION(BlueprintCallable, Category="Task")
	UObject* GetContextObject() const;

	/** Gets the scene state object's event stream */
	UFUNCTION(BlueprintCallable, Category="Task")
	USceneStateEventStream* GetEventStream() const;

	/** Called to indicate the task has completed */
	UFUNCTION(BlueprintCallable, Category="Task")
	void FinishTask();

	//~ Begin ISceneStateEventHandlerProvider
	virtual bool FindEventHandlerId(const FSceneStateEventSchemaHandle& InEventSchemaHandle, FGuid& OutHandlerId) const;
	//~ End ISceneStateEventHandlerProvider

	//~ Begin UObject
	virtual UWorld* GetWorld() const override final;
	//~ End UObject

private:
	/** Context to retrieve the underlying Task/Task Instance for this blueprint task */
	UE::SceneState::FTaskExecutionContext TaskExecutionContext;
};
