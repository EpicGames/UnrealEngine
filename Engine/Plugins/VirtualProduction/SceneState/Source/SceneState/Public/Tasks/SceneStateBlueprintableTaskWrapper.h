// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneStateTask.h"
#include "SceneStateTaskInstance.h"
#include "Templates/SubclassOf.h"
#include "SceneStateBlueprintableTaskWrapper.generated.h"

#define UE_API SCENESTATE_API

class USceneStateBlueprintableTask;

USTRUCT()
struct FSceneStateBlueprintableTaskInstance : public FSceneStateTaskInstance
{
	GENERATED_BODY()

	/** Task Blueprint Instance allocated everytime the Blueprintable Task Wrapper task starts */
	UPROPERTY(VisibleAnywhere, Instanced, Category="Scene State")
	TObjectPtr<USceneStateBlueprintableTask> Task;
};

/**
 * Task Wrapper for Blueprint Tasks.
 * Forwards the task start, tick and stop to the Blueprint Task held in its Task Instance struct
 * @see FSceneStateBlueprintableTaskInstance
 * @see USceneStateBlueprintableTask
 */
USTRUCT(meta=(Hidden))
struct FSceneStateBlueprintableTaskWrapper : public FSceneStateTask
{
	GENERATED_BODY()

	using FInstanceDataType = FSceneStateBlueprintableTaskInstance;

	UE_API FSceneStateBlueprintableTaskWrapper();

	UE_API bool SetTaskClass(TSubclassOf<USceneStateBlueprintableTask> InTaskClass);

	TSubclassOf<USceneStateBlueprintableTask> GetTaskClass() const
	{
		return TaskClass;
	}

protected:
	//~ Begin FSceneStateTask
#if WITH_EDITOR
	virtual const UScriptStruct* OnGetTaskInstanceType() const override;
	virtual void OnBuildTaskInstance(UObject* InOuter, FStructView InTaskInstance) const override;
#endif
	virtual void OnSetup(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance) const override;
	virtual void OnStart(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance) const override;
	virtual void OnTick(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance, float InDeltaSeconds) const override;
	virtual void OnStop(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance, ESceneStateTaskStopReason InStopReason) const override;
	//~ End FSceneStateTask

private:
	UPROPERTY()
	TSubclassOf<USceneStateBlueprintableTask> TaskClass;
};

#undef UE_API
