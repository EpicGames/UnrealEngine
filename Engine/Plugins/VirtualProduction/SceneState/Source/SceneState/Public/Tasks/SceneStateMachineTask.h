// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneStateExecutionContext.h"
#include "SceneStateMachineTaskBinding.h"
#include "StructUtils/PropertyBag.h"
#include "Tasks/SceneStateTask.h"
#include "Tasks/SceneStateTaskInstance.h"
#include "SceneStateMachineTask.generated.h"

struct FSceneStateMachine;

USTRUCT()
struct FSceneStateMachineTaskInstance : public FSceneStateTaskInstance
{
	GENERATED_BODY()

	/** Execution context created and owned by this Task / Task Instance */
	UPROPERTY(Transient, meta=(NoBinding))
	FSceneStateExecutionContext ExecutionContext;

	/** Id of the Target State Machine Parameters to copy our Parameters to */
	UPROPERTY(EditAnywhere, Category="State Machine", meta=(NoBinding))
	FGuid TargetId;

#if WITH_EDITORONLY_DATA
	/** Identifier for the Parameters this Instance owns */
	UPROPERTY(VisibleAnywhere, Category="State Machine", meta=(NoBinding))
	FGuid ParametersId;
#endif

	/** Parameters mirroring the Target State Machine */
	UPROPERTY(EditAnywhere, Category="State Machine", meta=(FixedLayout))
	FInstancedPropertyBag Parameters;
};

/** Task that runs a local state machine */
USTRUCT(DisplayName="Run State Machine", Category="Core", meta=(CoreTask))
struct FSceneStateMachineTask : public FSceneStateTask
{
	GENERATED_BODY()

	using FInstanceDataType = FSceneStateMachineTaskInstance;

	FSceneStateMachineTask();

protected:
	//~ Begin FSceneStateTask
#if WITH_EDITOR
	virtual const UScriptStruct* OnGetTaskInstanceType() const override;
	virtual void OnBuildTaskInstance(UObject* InOuter, FStructView InTaskInstance) const override;
#endif
	virtual const FSceneStateTaskBindingExtension* OnGetBindingExtension() const override;
	virtual void OnStart(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance) const override;
	virtual void OnTick(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance, float InDeltaSeconds) const override;
	virtual void OnStop(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance, ESceneStateTaskStopReason InStopReason) const override;
	//~ End FSceneStateTask

	bool IsStateMachineFinished(const FSceneStateExecutionContext& InContext, const FSceneStateMachine& InStateMachine) const;

	UPROPERTY()
	FSceneStateMachineTaskBinding Binding;
};
