// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tasks/SceneStateTaskBindingExtension.h"
#include "SceneStateMachineTaskBinding.generated.h"

struct FSceneStateMachineTaskInstance;

/**
 * Binding Extension for the State Machine Task.
 * Allows the State Machine Parameters to be discoverable as a binding desc
 */
USTRUCT()
struct FSceneStateMachineTaskBinding : public FSceneStateTaskBindingExtension
{
	GENERATED_BODY()

	using FInstanceDataType = FSceneStateMachineTaskInstance;

	/** Data View Indices */
	static constexpr uint16 PARAMETERS_DATA_INDEX = 0;

protected:
	//~ Begin FSceneStateTaskBindingExtension
#if WITH_EDITOR
	virtual void VisitBindingDescs(FStructView InTaskInstance, TFunctionRef<void(const UE::SceneState::FTaskBindingDesc&)> InFunctor) const override;
	virtual void SetBindingBatch(uint16 InDataIndex, uint16 InBatchIndex) override;
	virtual bool FindDataById(FStructView InTaskInstance, const FGuid& InStructId, FStructView& OutDataView, uint16& OutDataIndex) const override;
#endif
	virtual bool FindDataByIndex(FStructView InTaskInstance, uint16 InDataIndex, FStructView& OutDataView) const override;
	virtual void VisitBindingBatches(FStructView InTaskInstance, TFunctionRef<void(uint16, FStructView)> InFunctor) const override;
	//~ End FSceneStateTaskBindingExtension

private:
	/** Index to the binding batch for the parameters property bag */
	UPROPERTY()
	uint16 ParametersBatchIndex = std::numeric_limits<uint16>::max();
};
