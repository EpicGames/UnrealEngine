// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tasks/SceneStateBlueprintableTask.h"
#include "Tasks/SceneStateTask.h"
#include "Tasks/SceneStateTaskInstance.h"
#include "SceneStatePlayerTask.generated.h"

class USceneStatePlayer;

USTRUCT()
struct FSceneStatePlayerTaskInstance : public FSceneStateTaskInstance
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Instanced, Category="Scene State")
	TObjectPtr<USceneStatePlayer> Player;
};

/** Task that runs a Scene State Object through a Player */
USTRUCT(DisplayName="Run Player", Category="Core", meta=(CoreTask))
struct FSceneStatePlayerTask : public FSceneStateTask
{
	GENERATED_BODY()

	using FInstanceDataType = FSceneStatePlayerTaskInstance;

	FSceneStatePlayerTask();

protected:
	//~ Begin FSceneStateTask
#if WITH_EDITOR
	virtual const UScriptStruct* OnGetTaskInstanceType() const override;
	virtual void OnBuildTaskInstance(UObject* InOuter, FStructView InTaskInstance) const override;
#endif
	virtual void OnSetup(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance) const override;
	virtual void OnStart(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance) const override;
	virtual void OnTick(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance, float InDeltaSeconds) const override;
	virtual void OnStop(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance, ESceneStateTaskStopReason InStopReason) const override;
	//~ End FSceneStateTask
};
