// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyBindingTypes.h"
#include "SceneStateEnums.h"
#include "SceneStateRange.h"
#include "SceneStateTask.generated.h"

#define UE_API SCENESTATE_API

class USceneStateSchema;
struct FSceneStateBindingDesc;
struct FSceneStateExecutionContext;
struct FSceneStateTaskBindingExtension;
struct FSceneStateTaskInstance;

namespace UE::SceneState
{
	struct FTaskEditChange;

	namespace Editor
	{
		class FBindingCompiler;
		class FBlueprintCompilerContext;
		class FStateMachineCompiler;
		class FStateMachineTaskCompiler;
	}
}

/**
 * Base class for Tasks.
 * Tasks are immutable in execution time, and so are meant to only hold logic and template read-only data.
 * Each Task has a Task Instance Type that is allocated and used for instance data that the task can then mutate.
 * @see FSceneStateTaskInstance
 * @see SceneStateTaskMetadata.h for information about metadata that can be used with Tasks
 */
USTRUCT(meta=(Hidden))
struct FSceneStateTask
{
	GENERATED_BODY()

	virtual ~FSceneStateTask() = default;

#if WITH_EDITOR
	/**
	 * Called in-editor to get the task instance type
	 * @return the task instance struct. Must derive from FSceneStateTaskInstance
	 */
	UE_API const UScriptStruct* GetTaskInstanceType() const;

	/**
	 * Determines whether the given schema is supported by this task 
	 * This is called only when the task struct contains the 'WithSupportsSchema' metadata.
	 */
	UE_API bool SupportsSchema(const USceneStateSchema* InSchema) const;

	/**
	 * Called to init properties (e.g. Guids) or instances objects within the task instance
	 * @param InOuter the outer to use for instanced objects
	 * @param InTaskInstance the task instance to build
	 */
	UE_API void BuildTaskInstance(UObject* InOuter, FStructView InTaskInstance) const;

	/**
	 * Called when a property within a task or task instance has changed
	 * @param InEditChange the edit change information (property changed event, outer)
	 * @param InTaskInstance the task instance paired with the task
	 */
	UE_API void PostEditChange(UE::SceneState::FTaskEditChange& InEditChange, FStructView InTaskInstance);
#endif

	/** Called to get the optional Binding Extension for a Task */
	UE_API const FSceneStateTaskBindingExtension* GetBindingExtension() const;

	UE_DEPRECATED(5.8, "FindTaskInstance is deprecated. Use the task instance provided by the task's virtual method ")
	UE_API FStructView FindTaskInstance(const FSceneStateExecutionContext& InContext) const;

	/**
	 * Called when the State holding the Task first starts.
	 * This is called for all tasks held by the state even if these tasks end up not running, or run at a later time.
	 */
	UE_API void Setup(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance) const;

	/**
	 * Called once the State processes that the Task has all its prerequisites met.
	 * Applies and property bindings to the task and calls OnStart().
	 */
	UE_API void Start(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance) const;

	/**
	 * Called by the State each Tick.
	 * OnTick() gets called if the Task is running and set to Tickable (see TaskFlags)
	 */
	UE_API void Tick(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance, float InDeltaSeconds) const;

	/**
	 * Called to Stop the Task, because it either finished or was forcibly stopped by the state
	 * @see ESceneStateTaskStopReason
	 */
	UE_API void Stop(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance, ESceneStateTaskStopReason InStopReason) const;

	/** Calls Stop with stop reason being that the task has finished */
	UE_API void Finish(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance) const;

	uint16 GetParentStateIndex() const
	{
		return ParentStateIndex;
	}

	uint16 GetTaskIndex() const
	{
		return TaskIndex;
	}

	FSceneStateRange GetPrerequisiteRange() const
	{
		return PrerequisiteRange;
	}

protected:
	UE_API void SetFlags(ESceneStateTaskFlags InFlags);

	UE_API void ClearFlags(ESceneStateTaskFlags InFlags);

#if WITH_EDITOR
	/**
	 * Called in-editor to get the task instance type
	 * @see GetTaskInstanceType
	 * @return the task instance struct. Must derive from FSceneStateTaskInstance
	 */
	virtual const UScriptStruct* OnGetTaskInstanceType() const
	{
		return nullptr;
	}

	/**
	 * Determines whether the given schema is supported by this task 
	 * This is called only when the task struct contains the 'WithSupportsSchema' metadata.
	 */
	virtual bool OnSupportsSchema(TNotNull<const USceneStateSchema*> InSchema) const
	{
		return true;
	}

	/**
	 * Called to init properties (e.g. Guids) or instances objects within the task instance
	 * @see BuildTaskInstance
	 */
	virtual void OnBuildTaskInstance(UObject* InOuter, FStructView InTaskInstance) const
	{
	}

	/**
	 * Called when a property within a task has changed
	 * @see PostEditChange
	 */
	virtual void OnPostTaskEditChange(UE::SceneState::FTaskEditChange& InEditChange, FStructView InTaskInstance)
	{
	}

	/** DEPRECATED 5.8, "Use OnPostTaskEditChange() that is non-const instead */
	virtual void OnPostEditChange(UE::SceneState::FTaskEditChange& InEditChange, FStructView InTaskInstance) const final
	{
	}
#endif

	/**
	 * Called to get the optional Binding Extension for a Task
	 * @see GetBindingExtension
	 */
	virtual const FSceneStateTaskBindingExtension* OnGetBindingExtension() const
	{
		return nullptr;
	}

	/**
	 * Called when the State holding the Task first starts.
	 * This is called for all tasks held by the state even if these tasks end up not running, or run at a later time.
	 */
	virtual void OnSetup(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance) const
	{
	}

	/** Called once the State processes that the Task has all its prerequisites met. */
	virtual void OnStart(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance) const
	{
	}

	/** Called by the State each Tick if the Task is set to Tick via its TaskFlags */
	virtual void OnTick(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance, float InDeltaSeconds) const
	{
	}

	/**
	 * Called when the Task has finished or forcibly stopped by the state
	 * @see ESceneStateTaskStopReason
	 */
	virtual void OnStop(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance, ESceneStateTaskStopReason InStopReason) const
	{
	}

private:
	/** Returns true if the current task status on the given context matches the expected status */
	bool IsTaskStatus(const FSceneStateExecutionContext& InContext, UE::SceneState::EExecutionStatus InExpectedStatus) const;

	/** Applies Bindings to the given Task Instance */
	bool ApplyBindings(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance) const;

	/** Bindings Batch where this Task is target */
	UPROPERTY()
	FPropertyBindingIndex16 BindingsBatch;

	/** Absolute Index to the State owning this Task */
	UPROPERTY()
	uint16 ParentStateIndex = FSceneStateRange::InvalidIndex;

	/** Absolute Index of this Task */
	UPROPERTY()
	uint16 TaskIndex = FSceneStateRange::InvalidIndex;

	/** Absolute Range to the relative indices of the tasks that need to finish before this task can be executed */
	UPROPERTY()
	FSceneStateRange PrerequisiteRange;

	/** Additional information about this task (e.g. if it ticks, whether it extends bindings, etc.) */
	ESceneStateTaskFlags TaskFlags = ESceneStateTaskFlags::None;

	friend UE::SceneState::Editor::FBindingCompiler;
	friend UE::SceneState::Editor::FBlueprintCompilerContext;
	friend UE::SceneState::Editor::FStateMachineCompiler;
	friend UE::SceneState::Editor::FStateMachineTaskCompiler;
};

#undef UE_API
