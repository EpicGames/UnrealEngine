// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"
#include "PropertyBindingDataView.h"
#include "PropertyBindingTypes.h"
#include "SceneStateRange.h"
#include "StructUtils/StructView.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectPtr.h"
#include "SceneStateTaskBindingExtension.generated.h"

namespace UE::SceneState
{

#if WITH_EDITOR
struct FTaskBindingDesc
{
	/** Unique identifier of the struct at Editor Time */
	FGuid Id;

	/** Name of the struct (used for debugging, logging, cosmetic). */
	FName Name;

	/** Template data view to the struct or class data  */
	FPropertyBindingDataView DataView;

	/** Unique Index for this Custom Desc to identify at Runtime */
	uint16 DataIndex = FSceneStateRange::InvalidIndex;
};
#endif

} // UE::SceneState

/** Extension to allow custom bindings for a task. */
USTRUCT()
struct FSceneStateTaskBindingExtension
{
	GENERATED_BODY()

#if WITH_EDITOR
	/**
	 * Visits all binding descs available
	 * @param InTaskInstance the task instance with binding structs
	 * @param InFunctor the functor to execute for every available binding desc
	 */
	virtual void VisitBindingDescs(FStructView InTaskInstance, TFunctionRef<void(const UE::SceneState::FTaskBindingDesc&)> InFunctor) const
	{
	}

	/**
	 * Sets the binding batch for the given data index
	 * @param InDataIndex index to the data that the binding batch links to
	 * @param InBindingsBatchIndex index to the binding batch that compiled
	 */
	virtual void SetBindingBatch(uint16 InDataIndex, uint16 InBindingsBatchIndex)
	{
	}

	/**
	 * Finds the data view that matches the struct id
	 * @param InTaskInstance the task instance to look into
	 * @param InStructId the struct id to look for
	 * @param OutDataView the returned data view, if found
	 * @param OutDataIndex index to the returned data view, if found
	 * @return true if the data was found
	 */
	virtual bool FindDataById(FStructView InTaskInstance, const FGuid& InStructId, FStructView& OutDataView, uint16& OutDataIndex) const
	{
		return false;
	}
#endif

	/**
	 * Finds the data view that maps to the given data index
	 * @param InTaskInstance the task instance to look into
	 * @param InDataIndex the index mapped to the data view to look for
	 * @param OutDataView the returned data view, if found
	 * @return true if the data was found
	 */
	virtual bool FindDataByIndex(FStructView InTaskInstance, uint16 InDataIndex, FStructView& OutDataView) const
	{
		return false;
	}

	/**
	 * Visits all the data views and their paired binding batch index
	 * @param InTaskInstance the task instance to look into
	 * @param InFunctor the functor to execute for every data view binding batch index pair
	 */
	virtual void VisitBindingBatches(FStructView InTaskInstance, TFunctionRef<void(uint16, FStructView)> InFunctor) const
	{
	}
};
