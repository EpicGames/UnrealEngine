// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneStateEnums.h"
#include "UObject/UnrealType.h"

#define UE_API SCENESTATE_API

class UObject;

namespace UE::SceneState
{
#if WITH_EDITOR
/** Describes an Edit change for a task or task instance */
struct FTaskEditChange : FPropertyChangedEvent
{
	UE_API FTaskEditChange();

	/** Returns true if the FSceneStateTask changed */
	UE_API bool IsTaskChange() const;

	/** Returns true if the FSceneStateTaskInstance changed */
	UE_API bool IsTaskInstanceChange() const;

	/** Owning object of the task */
	UObject* Outer = nullptr;
	/** The object that was changed: whether it was the task or the task instance */
	ETaskObjectType ChangedObject = ETaskObjectType::None;
};
#endif

} // UE::SceneState

#undef UE_API
