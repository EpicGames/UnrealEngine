// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneStateExecutionContextHandle.h"
#include "SceneStateRange.h"
#include "Templates/SharedPointer.h"

#define UE_API SCENESTATE_API

struct FConstStructView;
struct FSceneStateExecutionContext;
struct FSceneStateTask;
struct FStructView;

namespace UE::SceneState
{
	class FExecutionContextRegistry;
}

namespace UE::SceneState
{

/**
 * Helper struct that can be passed over by copy in lambda captures, delegates while enabling functionality like
 * safely accessing a Task, Task Instance, and Finishing a Task.
 * It returns the Task and Task Instance only if the saved Instance Id matches the current Instance Id for the Task.
 */
struct FTaskExecutionContext
{
	UE_API FTaskExecutionContext();

	UE_API explicit FTaskExecutionContext(const FSceneStateTask& InTask, const FSceneStateExecutionContext& InContext);

	/** Resolves the execution context if it still exists */
	UE_API const FSceneStateExecutionContext* GetExecutionContext() const;

	/** Retrieves the FSceneStateTask view for this context, if the task instance this context got created for is still valid. An invalid view is returned otherwise */
	UE_API FConstStructView GetTask() const;

	/** Retrieves the FSceneStateTaskInstance view for this context, if the task instance this context got created for is still valid. An invalid view is returned otherwise */
	UE_API FStructView GetTaskInstance() const;

	/** Helper function to Finish the Task / Task Instance for this context if these are still valid */
	UE_API void FinishTask() const;

private:
	FString GetDebugString() const;

	/** Handle to the context within the saved registry */
	FExecutionContextHandle ContextHandle;

	/** Registry used to find the context */
	TWeakPtr<const FExecutionContextRegistry> ContextRegistryWeak;

	/** Absolute index to the task */
	uint16 TaskIndex;

	/** Absolute index to the task's parent state */
	uint16 StateIndex;

	/**
	 * Id assigned to the state instance to distinguish it from a different instance (within the same context) with the same state index.
	 * This can happen if this context is held past its task instance (e.g. captured from a deferred lambda)
	 * and there's an active new instance for the same state within the same context.
	 */
	uint16 StateInstanceId;
};

} // UE::SceneState

#undef UE_API
