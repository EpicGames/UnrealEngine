// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/BlueprintGeneratedClass.h"
#include "UObject/Package.h"
#include "SceneStateTaskGeneratedClass.generated.h"

/** Generated class for Scene State Blueprintable Tasks */
UCLASS(MinimalAPI)
class USceneStateTaskGeneratedClass : public UBlueprintGeneratedClass
{
	GENERATED_BODY()
};
