// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneStateEnums.h"
#include "SceneStateTaskInstance.generated.h"

/** Represents the Instance Data of a Task */
USTRUCT()
struct FSceneStateTaskInstance
{
	GENERATED_BODY()

	UE_DEPRECATED(5.8, "Status has been moved outside of task instance and now managed by the execution context ")
	UE::SceneState::EExecutionStatus GetStatus() const
	{
		return UE::SceneState::EExecutionStatus::NotStarted;
	}
};
