// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneStateTaskMetadata.generated.h"

namespace UE::SceneState::Editor
{
#if WITH_EDITOR
	/** The task should not appear in the list of available tasks (e.g. because it's a base task struct) */
	const FLazyName MD_Hidden(TEXT("Hidden"));

	/** 
	 * The task is deemed as 'Core'. These are tasks that should be allowed in most schemas. 
	 * Derived tasks will NOT be recognized as 'CoreTask' unless they also specify this metadata.
	 */
	const FLazyName MD_CoreTask(TEXT("CoreTask"));

	/**
	 * The task is utility that can be useful for most schemas. All the non-core tasks that come with the Scene State plugin will be marked with this metadata. 
	 * Derived tasks will NOT be recognized as 'UtilityTask' unless they also specify this metadata.
	 */
	const FLazyName MD_UtilityTask(TEXT("UtilityTask"));

	/**
	 * The task requires the context object to implement GetWorld().
	 * This is just a convenience metadata to avoid having to call SupportsSchema.
	 * Derived tasks will also be treated as requiring a context world.
	 */
	const FLazyName MD_RequiresContextWorld(TEXT("RequiresContextWorld"));

	/** 
	 * The task implements OnSupportsSchema function. Only tasks with this metadata will have their 'SupportsSchema' function called. 
	 * Derived tasks will also have this function called.
	 */
	const FLazyName MD_WithSupportsSchema(TEXT("WithSupportsSchema"));
#endif
}

/** Metadata information about the Task. Available only in editor */
USTRUCT()
struct FSceneStateTaskMetadata
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	/** Id used for the Property Binding System to identify a Task */
	UPROPERTY()
	FGuid TaskId;
#endif
};
