// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyBindingTypes.h"
#include "SceneStateEnums.h"
#include "SceneStateRange.h"
#include "SceneStateTransitionTarget.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectPtr.h"
#include "SceneStateTransition.generated.h"

class FBoolProperty;
class UFunction;
class UObject;
struct FSceneState;
struct FSceneStateConduit;
struct FSceneStateExecutionContext;
struct FSceneStateMachine;
struct FSceneStateMachineInstance;
struct FSceneStateTransitionInstance;
struct FSceneStateTransitionLink;

namespace UE::SceneState
{
	struct FTransitionEvaluationParams;

	namespace Editor
	{
		class FStateMachineTransitionCompiler;
		class FBindingCompiler;
	}
}

/**
 * Holds the exit transition conditions and information from its owner state to a target.
 * @see FSceneStateTransitionTarget
 */
USTRUCT()
struct FSceneStateTransition
{
	GENERATED_BODY()

	/** Called when the Owner Class is linking to cache the Event function and Result property */
	void Link(const FSceneStateTransitionLink& InTransitionLink, TNotNull<const UClass*> InOwnerClass);

	/** Called when the State has entered. Allocates and initializes the transition instance */
	void Setup(const FSceneStateExecutionContext& InContext) const;

	/** Called to evaluate this transition. Returns true if the transition should take place */
	bool Evaluate(const UE::SceneState::FTransitionEvaluationParams& InParams) const;

	/** Called when the State has exited. Cleans up the transition instance */
	void Exit(const FSceneStateExecutionContext& InContext) const;

	FSceneStateTransitionTarget GetTarget() const
	{
		return Target;
	}

private:
	/** Called to apply the transition parameter bindings to be optionally used by the transition evaluation event */
	bool ApplyBindings(const FSceneStateExecutionContext& InContext, FSceneStateTransitionInstance& InInstance) const;

	/** Evaluates whether all the target state's required events are present */
	bool ContainsAllRequiredEvents(const UE::SceneState::FTransitionEvaluationParams& InParams) const;

	/** Calls the evaluation event (can return early if optimized away via the evaluation flags) */
	bool ProcessEvaluationEvent(const UE::SceneState::FTransitionEvaluationParams& InParams) const;

	/** If the transition target is a conduit, returns the pointer to such conduit */
	const FSceneStateConduit* FindConduitTarget(const FSceneStateExecutionContext& InContext) const;

	/** Bindings Batch where this Transition is target */
	UPROPERTY()
	FPropertyBindingIndex16 BindingsBatch;

	/**
	 * Target to transition to if this transition evaluates to true
	 * This is used to set the Active State Index if the type is state (which is relative to the state machine states start index) in a State Machine
	 */
	UPROPERTY()
	FSceneStateTransitionTarget Target;

	/** Transition flags indicating how a transition should be evaluated */
	UPROPERTY()
	ESceneStateTransitionEvaluationFlags EvaluationFlags = ESceneStateTransitionEvaluationFlags::None;

	/** Transition Evaluation Event to execute */ 
	UPROPERTY(Transient)
	TObjectPtr<UFunction> EvaluationEvent;

	/** Pointer to the result property of this transition */
	FStructProperty* ResultProperty = nullptr;

	friend UE::SceneState::Editor::FBindingCompiler;
	friend UE::SceneState::Editor::FStateMachineTransitionCompiler;
};
