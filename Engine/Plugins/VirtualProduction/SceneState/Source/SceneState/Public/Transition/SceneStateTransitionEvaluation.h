// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

struct FSceneState;
struct FSceneStateExecutionContext;
struct FSceneStateMachine;
struct FSceneStateMachineInstance;

namespace UE::SceneState
{

/** The parameters required for transition evaluation */
struct FTransitionEvaluationParams
{
	/** The current execution context */
	const FSceneStateExecutionContext& ExecutionContext;
	/** The state machine the transition resides in */
	const FSceneStateMachine& StateMachine;
	/** The state machine instance data corresponding to the above state machine */
	const FSceneStateMachineInstance& StateMachineInstance;
	/** The state containing the transition as an exit transition */
	const FSceneState& SourceState;
};

} // UE::SceneState
