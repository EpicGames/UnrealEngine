// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/PropertyBag.h"
#include "SceneStateTransitionInstance.generated.h"

/** Instance data of a transition */
USTRUCT()
struct FSceneStateTransitionInstance
{
	GENERATED_BODY()

	/** The parameters to pass to the evaluation event */
	UPROPERTY()
	FInstancedPropertyBag Parameters;
};
