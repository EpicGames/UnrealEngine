// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"
#include "SceneStateTransitionLink.generated.h"

/**
 * Holds information about the transition that is only used in transition link time
 * @see FSceneStateTransition::Link
 */
USTRUCT()
struct FSceneStateTransitionLink
{
	GENERATED_BODY()

	/** Name to lookup and set the Event Function */
	UPROPERTY()
	FName EventName;

	/** Name of the Result Property the Event function sets */
	UPROPERTY()
	FName ResultPropertyName;
};
