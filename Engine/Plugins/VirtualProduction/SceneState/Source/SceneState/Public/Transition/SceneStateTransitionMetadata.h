// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"
#include "SceneStateTransitionMetadata.generated.h"

/** Metadata information about the Transition. Available only in editor */
USTRUCT()
struct FSceneStateTransitionMetadata
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	/** Parameter id of the Transition, if any */
	UPROPERTY()
	FGuid ParametersId;
#endif
};
