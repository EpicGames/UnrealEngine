// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneStateTransitionResult.generated.h"

/** The result output of a transition graph evaluation */
USTRUCT(BlueprintInternalUseOnly)
struct FSceneStateTransitionResult
{
	GENERATED_BODY()

	/** Indicator of whether the transition condition succeeded */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Scene State Transition")
	bool bCanTransition = true;
};
