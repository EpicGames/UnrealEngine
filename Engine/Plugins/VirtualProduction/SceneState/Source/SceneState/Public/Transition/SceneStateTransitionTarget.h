// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <limits>
#include "SceneStateTransitionTarget.generated.h"

/** Types of object a transition can lead to */
UENUM()
enum class ESceneStateTransitionTargetType : uint8
{
	/** The transition leads to another state */
	State,

	/** The transition leads to an exit point */
	Exit,

	/** The transition leads to a conduit */
	Conduit,
};

/** Describes a transition's target via its type (e.g. State) and an index if required (e.g. the index of the State) */
USTRUCT()
struct FSceneStateTransitionTarget
{
	GENERATED_BODY()

	/** Target type to transition to */
	UPROPERTY()
	ESceneStateTransitionTargetType Type = ESceneStateTransitionTargetType::State;

	/** Index of the target to transition to, conditionally required */
	UPROPERTY()
	uint16 Index = std::numeric_limits<uint16>::max();
};
