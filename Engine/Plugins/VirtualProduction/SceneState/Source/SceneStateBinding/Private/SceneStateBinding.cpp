// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateBinding.h"

FConstStructView FSceneStateBinding::GetSourceDataHandleStruct() const
{
	return FConstStructView::Make(SourceDataHandle);
}
