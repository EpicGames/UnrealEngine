// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateBindingCollection.h"
#include "PropertyBindingDataView.h"
#include "SceneStateBindingFunction.h"
#include "SceneStatePropertyReference.h"
#include "SceneStatePropertyReferenceUtils.h"

#if WITH_EDITOR
FPropertyBindingPath FSceneStateBindingCollection::AddBindingFunction(const UE::SceneState::FBindingFunctionInfo& InFunctionInfo, TConstArrayView<FPropertyBindingPathSegment> InSourcePathSegments, const FPropertyBindingPath& InTargetPath)
{
	FInstancedStruct BindingFunction(FSceneStateBindingFunction::StaticStruct());
	BindingFunction.InitializeAs<FSceneStateBindingFunction>(InFunctionInfo);

	Super::RemoveBindings(InTargetPath, ESearchMode::Exact);

	FPropertyBindingPath SourcePath = FPropertyBindingPath(BindingFunction.Get<FSceneStateBindingFunction>().FunctionId, InSourcePathSegments);
	Bindings.Emplace(MoveTemp(BindingFunction), SourcePath, InTargetPath);

	return SourcePath;
}
#endif

const FSceneStateBindingDesc* FSceneStateBindingCollection::FindBindingDesc(FSceneStateBindingDataHandle InDataHandle) const
{
	return BindingDescs.FindByPredicate(
		[&InDataHandle](const FSceneStateBindingDesc& InDesc)
		{
			return InDesc.DataHandle == InDataHandle;
		});
}

const FSceneStateBindingResolvedReference* FSceneStateBindingCollection::FindResolvedReference(const FSceneStatePropertyReference& InPropertyReference) const
{
	if (ResolvedReferences.IsValidIndex(InPropertyReference.ReferenceIndex))
	{
		return &ResolvedReferences[InPropertyReference.ReferenceIndex];
	}
	return nullptr;
}

uint8* FSceneStateBindingCollection::ResolveProperty(const FSceneStateBindingResolvedReference& InResolvedReference, FPropertyBindingDataView InDataView) const
{
	if (!InDataView.IsValid())
	{
		return nullptr;
	}

	if (!ensure(InDataView.GetStruct() == InResolvedReference.SourceStructType))
	{
		return nullptr;
	}

	return Super::GetAddress(InDataView, InResolvedReference.SourceIndirection, InResolvedReference.SourceLeafProperty);
}

#if WITH_EDITOR
FPropertyBindingBinding* FSceneStateBindingCollection::AddBindingInternal(const FPropertyBindingPath& InSourcePath, const FPropertyBindingPath& InTargetPath)
{
	return &Bindings.Emplace_GetRef(InSourcePath, InTargetPath);
}

void FSceneStateBindingCollection::CopyBindingsInternal(const FGuid InFromStructId, const FGuid InToStructId)
{
	struct FFunctionBindingCopyInfo
	{
		UE::SceneState::FBindingFunctionInfo FunctionInfo;
		FPropertyBindingPath SourcePath;
		FPropertyBindingPath TargetPath;
	};

	TArray<TTuple<FGuid, FGuid>, TInlineAllocator<8>> StructIds;
	StructIds.Emplace(InFromStructId, InToStructId);

	// Find the StructId and copy the binding. If we find functions, then copy the function and copy the functions binding recursively.
	for (int32 Index = 0; Index < StructIds.Num(); ++Index)
	{
		const TTuple<FGuid, FGuid>& StructId = StructIds[Index];

		TArray<FFunctionBindingCopyInfo, TInlineAllocator<8>> FunctionBindingsToCopy;

		CopyBindingsImplementation(StructId.Get<0>(), StructId.Get<1>(),
			[ToStructId = StructId.Get<1>(), &FunctionBindingsToCopy](const FPropertyBindingBinding& InBinding)
			{
				bool bCanCopyBinding = true;
				if (const FSceneStateBindingFunction* BindingFunction = InBinding.GetPropertyFunctionNode().GetPtr<const FSceneStateBindingFunction>())
				{
					FFunctionBindingCopyInfo& CopyInfo = FunctionBindingsToCopy.AddDefaulted_GetRef();
					CopyInfo.FunctionInfo.FunctionTemplate = BindingFunction->Function;
					CopyInfo.FunctionInfo.InstanceTemplate = BindingFunction->FunctionInstance;
					CopyInfo.SourcePath = InBinding.GetSourcePath();
					CopyInfo.TargetPath = FPropertyBindingPath(ToStructId, InBinding.GetTargetPath().GetSegments());

					bCanCopyBinding = false;
				}
				return bCanCopyBinding;
			});

		// Copy all functions bindings that target "FromStructId" and retarget them to "ToStructId".
		for (const FFunctionBindingCopyInfo& CopyInfo : FunctionBindingsToCopy)
		{
			const FGuid NewId = AddBindingFunction(CopyInfo.FunctionInfo, CopyInfo.SourcePath.GetSegments(), CopyInfo.TargetPath).GetStructID();

			// Adds the new function binding to the list of copy bindings to look for.
			StructIds.AddUnique(TTuple<FGuid, FGuid>{CopyInfo.SourcePath.GetStructID(), NewId});
		}
	}
}

void FSceneStateBindingCollection::RemoveBindingsInternal(TFunctionRef<bool(FPropertyBindingBinding&)> InPredicate)
{
	Bindings.RemoveAllSwap(InPredicate);
}

bool FSceneStateBindingCollection::HasBindingInternal(TFunctionRef<bool(const FPropertyBindingBinding&)> InPredicate) const
{
	return Bindings.ContainsByPredicate(InPredicate);
}

const FPropertyBindingBinding* FSceneStateBindingCollection::FindBindingInternal(TFunctionRef<bool(const FPropertyBindingBinding&)> InPredicate) const
{
	return Bindings.FindByPredicate(InPredicate);
}
#endif

int32 FSceneStateBindingCollection::GetNumBindings() const
{
	return Bindings.Num();
}

int32 FSceneStateBindingCollection::GetNumBindableStructDescriptors() const
{
	return BindingDescs.Num();
}

const FPropertyBindingBindableStructDescriptor* FSceneStateBindingCollection::GetBindableStructDescriptorFromHandle(FConstStructView InSourceHandleView) const
{
	return FindBindingDesc(InSourceHandleView.Get<const FSceneStateBindingDataHandle>());
}

void FSceneStateBindingCollection::ForEachBinding(TFunctionRef<void(const FPropertyBindingBinding& Binding)> InFunction) const
{
	for (const FSceneStateBinding& Binding : Bindings)
	{
		InFunction(Binding);
	}
}

void FSceneStateBindingCollection::ForEachBinding(FPropertyBindingIndex16 InBegin, FPropertyBindingIndex16 InEnd, TFunctionRef<void(const FPropertyBindingBinding& Binding, const int32 BindingIndex)> InFunction) const
{
	checkf(InBegin.IsValid() && InEnd.IsValid(), TEXT("Begin and end indices are not valid!"));

	for (int32 BindingIndex = InBegin.Get(); BindingIndex < InEnd.Get(); ++BindingIndex)
	{
		checkf(Bindings.IsValidIndex(BindingIndex), TEXT("Index %d out of bounds! Bindings Num: %d"), BindingIndex, Bindings.Num());
		InFunction(Bindings[BindingIndex], BindingIndex);
	}
}

void FSceneStateBindingCollection::ForEachMutableBinding(TFunctionRef<void(FPropertyBindingBinding& Binding)> InFunction)
{
	for (FSceneStateBinding& Binding : Bindings)
	{
		InFunction(Binding);
	}
}

void FSceneStateBindingCollection::VisitBindings(TFunctionRef<EVisitResult(const FPropertyBindingBinding& Binding)> InFunction) const
{
	for (const FSceneStateBinding& Binding : Bindings)
	{
		if (InFunction(Binding) == EVisitResult::Break)
		{
			break;
		}
	}
}

void FSceneStateBindingCollection::VisitBindings(FPropertyBindingIndex16 InBegin, FPropertyBindingIndex16 InEnd,
	TFunctionRef<EVisitResult(const FPropertyBindingBinding& Binding, const int32 BindingIndex)> InFunction) const
{
	checkf(InBegin.IsValid() && InEnd.IsValid(), TEXT("Begin and end indices are not valid!"));

	for (int32 BindingIndex = InBegin.Get(); BindingIndex < InEnd.Get(); ++BindingIndex)
	{
		checkf(Bindings.IsValidIndex(BindingIndex), TEXT("Index %d out of bounds! Bindings Num: %d"), BindingIndex, Bindings.Num());
		if (InFunction(Bindings[BindingIndex], BindingIndex) == EVisitResult::Break)
		{
			break;
		}
	}
}

void FSceneStateBindingCollection::VisitMutableBindings(TFunctionRef<EVisitResult(FPropertyBindingBinding& Binding)> InFunction)
{
	for (FSceneStateBinding& Binding : Bindings)
	{
		if (InFunction(Binding) == EVisitResult::Break)
		{
			break;
		}
	}
}

void FSceneStateBindingCollection::OnReset()
{
	BindingDescs.Reset();
	Bindings.Reset();
}

void FSceneStateBindingCollection::VisitSourceStructDescriptorInternal(TFunctionRef<EVisitResult(const FPropertyBindingBindableStructDescriptor&)> InFunction) const
{
	for (const FSceneStateBindingDesc& BindingDesc : BindingDescs)
	{
		if (InFunction(BindingDesc) == EVisitResult::Break)
		{
			break;
		}
	}
}

bool FSceneStateBindingCollection::OnResolvingPaths()
{
	// Base class handled common bindings, here we only need to handle Property references
	bool bResult = true;

	ResolvedReferences.Reset(References.Num());

	for (const FSceneStateBindingReference& Reference : References)
	{
		// Always adds a defaulted reference even in the unexpected occasion binding desc is not found.
		// This is so that reference indices are kept stable.
		FSceneStateBindingResolvedReference& ResolvedReference = ResolvedReferences.AddDefaulted_GetRef();
		ResolvedReference.SourceDataHandle = Reference.SourceDataHandle;

		const FPropertyBindingBindableStructDescriptor* SourceDesc = FindBindingDesc(Reference.SourceDataHandle);
		if (!ensure(SourceDesc))
		{
			continue;
		}

		ResolvedReference.SourceStructType = SourceDesc->Struct;

		FPropertyBindingPathIndirection SourceLeafIndirection;
		if (!Super::ResolvePath(SourceDesc->Struct, Reference.SourcePropertyPath, ResolvedReference.SourceIndirection, SourceLeafIndirection))
		{
			bResult = false;
		}

		ResolvedReference.SourceLeafProperty = SourceLeafIndirection.GetProperty();
	}

	return bResult;
}
