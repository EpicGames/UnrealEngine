// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateBindingDataHandle.h"

FSceneStateBindingDataHandle FSceneStateBindingDataHandle::MakeExternalDataHandle(const FExternalData& InExternalData)
{
	// Shift by 8 to differentiate an external data type from the internal ones
	FSceneStateBindingDataHandle DataHandle;
	DataHandle.Type = static_cast<uint16>(InExternalData.Type) << 8;
	DataHandle.Index = GetIndexSafeChecked(InExternalData.Index);
	DataHandle.SubIndex = GetIndexSafeChecked(InExternalData.SubIndex);
	return DataHandle;
}
