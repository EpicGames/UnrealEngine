// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateBindingDelegates.h"

namespace UE::SceneState
{

#if WITH_EDITOR
	TMulticastDelegate<void(const FStructIdChange&)> OnStructIdChanged;
#endif

} // UE::SceneState
