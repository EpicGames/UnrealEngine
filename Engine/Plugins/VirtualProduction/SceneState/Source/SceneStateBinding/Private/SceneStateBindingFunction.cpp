// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateBindingFunction.h"

#if WITH_EDITOR
FSceneStateBindingFunction::FSceneStateBindingFunction(const UE::SceneState::FBindingFunctionInfo& InFunctionInfo)
{
	FunctionId = FGuid::NewGuid();
	Function = InFunctionInfo.FunctionTemplate;
	FunctionInstance = InFunctionInfo.InstanceTemplate;
}

bool FSceneStateBindingFunction::IsValid() const
{
	return FunctionId.IsValid() && Function.IsValid() && FunctionInstance.IsValid();
}
#endif
