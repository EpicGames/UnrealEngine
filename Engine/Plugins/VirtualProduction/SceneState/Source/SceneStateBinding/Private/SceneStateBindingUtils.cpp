// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateBindingUtils.h"
#include "PropertyBindingBindingCollectionOwner.h"
#include "PropertyBindingPath.h"
#include "SceneStateBindingCollection.h"
#include "SceneStateBindingDelegates.h"

namespace UE::SceneState
{

#if WITH_EDITOR
void HandleStructIdChanged(UObject& InObject, const FGuid& InOldStructId, const FGuid& InNewStructId)
{
	// Nothing to replace if old struct id is not valid
	if (!InOldStructId.IsValid())
	{
		return;
	}

	IPropertyBindingBindingCollectionOwner* const BindingCollectionOwner = InObject.GetImplementingOuter<IPropertyBindingBindingCollectionOwner>();
	if (!BindingCollectionOwner)
	{
		return;
	}

	FPropertyBindingBindingCollection* const BindingCollection = BindingCollectionOwner->GetEditorPropertyBindings();
	if (!BindingCollection)
	{
		return;
	}

	// If the old id is still valid it is because the object with the new id is a duplicated object from a source object that has the old id as its id
	// In this case, copy bindings without replacing the old id
	FPropertyBindingDataView SourceDataView;
	if (BindingCollectionOwner->GetBindingDataViewByID(InOldStructId, SourceDataView))
	{
		BindingCollection->CopyBindings(InOldStructId, InNewStructId);
		return;
	}

	auto FixBindingPath =
		[&InOldStructId, &InNewStructId](FPropertyBindingPath& InBindingPath)
		{
			if (InBindingPath.GetStructID() == InOldStructId)
			{
				InBindingPath.SetStructID(InNewStructId);
			}
		};

	// The old struct id does not exist meaning existing bindings should point to this new id
	BindingCollection->ForEachMutableBinding(
		[&FixBindingPath](FPropertyBindingBinding& InBinding)
		{
			FixBindingPath(InBinding.GetMutableSourcePath());
			FixBindingPath(InBinding.GetMutableTargetPath());
		});

	FStructIdChange Change;
	Change.BindingOwner = Cast<UObject>(BindingCollectionOwner);
	Change.OldToNewStructIdMap.Add(InOldStructId, InNewStructId);
	OnStructIdChanged.Broadcast(Change);
}
#endif

void PatchBindingCollection(const FPatchBindingParams& InParams)
{
	PatchBindingDescs(InParams);
	PatchBindings(InParams);
	PatchCopyBatches(InParams);
}

void PatchBindingDescs(const FPatchBindingParams& InParams)
{
	for (FSceneStateBindingDesc& BindingDesc : InParams.BindingCollection.GetMutableBindingDescs())
	{
		if (const UClass* SourceClass = Cast<UClass>(BindingDesc.Struct))
		{
			BindingDesc.Struct = SourceClass->GetAuthoritativeClass();
		}

		if (const UStruct* DataStruct = InParams.FindDataStructFunctor(BindingDesc.DataHandle))
		{
			ensure(!BindingDesc.Struct || DataStruct == BindingDesc.Struct);
			BindingDesc.Struct = DataStruct;
		}
	}
}

void PatchBindings(const FPatchBindingParams& InParams)
{
	auto PatchBindingPaths = [&InParams](FSceneStateBindingDataHandle InDataHandle, FPropertyBindingPath& InPath)
		{
			for (FPropertyBindingPathSegment& Segment : InPath.GetMutableSegments())
			{
				if (const UClass* InstanceStruct = Cast<UClass>(Segment.GetInstanceStruct()))
				{
					Segment.SetInstanceStruct(InstanceStruct->GetAuthoritativeClass());
				}
			}

			if (const UStruct* SourceDataStruct = InParams.FindDataStructFunctor(InDataHandle))
			{
				InPath.UpdateSegments(SourceDataStruct);
			}
		};

	for (FSceneStateBinding& Binding : InParams.BindingCollection.GetMutableBindings())
	{
		PatchBindingPaths(Binding.SourceDataHandle, Binding.GetMutableSourcePath());
		PatchBindingPaths(Binding.TargetDataHandle, Binding.GetMutableTargetPath());
	}
}

void PatchCopyBatches(const FPatchBindingParams& InParams)
{
	for (FPropertyBindingCopyInfoBatch& CopyBatch : InParams.BindingCollection.GetMutableCopyBatches())
	{
		if (const UClass* TargetClass = Cast<UClass>(CopyBatch.TargetStruct.Get().Struct))
		{
			CopyBatch.TargetStruct.GetMutable().Struct = TargetClass->GetAuthoritativeClass();
		}

		if (FSceneStateBindingDesc* TargetDesc = CopyBatch.TargetStruct.GetMutablePtr<FSceneStateBindingDesc>())
		{
			if (const UStruct* DataStruct = InParams.FindDataStructFunctor(TargetDesc->DataHandle))
			{
				ensure(!TargetDesc->Struct || DataStruct == TargetDesc->Struct);
				TargetDesc->Struct = DataStruct;
			}
		}
	}
}

} // UE::SceneState
