// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateBlueprintPropertyReference.h"

#if WITH_EDITOR
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#endif

#if WITH_EDITOR
FSceneStateBlueprintPropertyReference FSceneStateBlueprintPropertyReference::BuildFromPinType(const FEdGraphPinType& InPinType)
{
	FSceneStateBlueprintPropertyReference PropertyReference;

	switch (InPinType.ContainerType)
	{
	case EPinContainerType::Array:
		PropertyReference.bIsReferenceToArray = true;
		break;
	case EPinContainerType::Set:
		ensureMsgf(false, TEXT("Unsupported container type [Set]"));
		break;
	case EPinContainerType::Map:
		ensureMsgf(false, TEXT("Unsupported container type [Map]"));
		break;
	default:
		break;
	}

	if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Boolean)
	{
		PropertyReference.ReferenceType = ESceneStatePropertyReferenceType::Bool;
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Byte)
	{
		if (UEnum* Enum = Cast<UEnum>(InPinType.PinSubCategoryObject))
		{
			PropertyReference.ReferenceType = ESceneStatePropertyReferenceType::Enum;
			PropertyReference.TypeObject = InPinType.PinSubCategoryObject.Get();
		}
		else
		{
			PropertyReference.ReferenceType = ESceneStatePropertyReferenceType::Byte;
		}
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Int)
	{
		PropertyReference.ReferenceType = ESceneStatePropertyReferenceType::Int32;
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Int64)
	{
		PropertyReference.ReferenceType = ESceneStatePropertyReferenceType::Int64;
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Real)
	{
		if (InPinType.PinSubCategory == UEdGraphSchema_K2::PC_Float)
		{
			PropertyReference.ReferenceType = ESceneStatePropertyReferenceType::Float;
		}
		else if (InPinType.PinSubCategory == UEdGraphSchema_K2::PC_Double)
		{
			PropertyReference.ReferenceType = ESceneStatePropertyReferenceType::Double;
		}		
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Name)
	{
		PropertyReference.ReferenceType = ESceneStatePropertyReferenceType::Name;
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_String)
	{
		PropertyReference.ReferenceType = ESceneStatePropertyReferenceType::String;
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Text)
	{
		PropertyReference.ReferenceType = ESceneStatePropertyReferenceType::Text;
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Enum)
	{
		PropertyReference.ReferenceType = ESceneStatePropertyReferenceType::Enum;
		PropertyReference.TypeObject = InPinType.PinSubCategoryObject.Get();
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Struct)
	{
		PropertyReference.ReferenceType = ESceneStatePropertyReferenceType::Struct;
		PropertyReference.TypeObject = InPinType.PinSubCategoryObject.Get();
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Object)
	{
		PropertyReference.ReferenceType = ESceneStatePropertyReferenceType::Object;
		PropertyReference.TypeObject = InPinType.PinSubCategoryObject.Get();
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_SoftObject)
	{
		PropertyReference.ReferenceType = ESceneStatePropertyReferenceType::SoftObject;
		PropertyReference.TypeObject = InPinType.PinSubCategoryObject.Get();
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Class)
	{
		PropertyReference.ReferenceType = ESceneStatePropertyReferenceType::Class;
		PropertyReference.TypeObject = InPinType.PinSubCategoryObject.Get();
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_SoftClass)
	{
		PropertyReference.ReferenceType = ESceneStatePropertyReferenceType::SoftClass;
		PropertyReference.TypeObject = InPinType.PinSubCategoryObject.Get();
	}
	else
	{
		ensureMsgf(false, TEXT("Unhandled pin category %s"), *InPinType.PinCategory.ToString());
	}

	return PropertyReference;
}
#endif

FName FSceneStateBlueprintPropertyReference::GetReferenceTypeMemberName()
{
	return GET_MEMBER_NAME_CHECKED(FSceneStateBlueprintPropertyReference, ReferenceType);
}

FName FSceneStateBlueprintPropertyReference::GetIsReferenceToArrayMemberName()
{
	return GET_MEMBER_NAME_CHECKED(FSceneStateBlueprintPropertyReference, bIsReferenceToArray);
}

FName FSceneStateBlueprintPropertyReference::GetTypeObjectMemberName()
{
	return GET_MEMBER_NAME_CHECKED(FSceneStateBlueprintPropertyReference, TypeObject);
}
