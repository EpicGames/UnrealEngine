// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStatePropertyReferenceUtils.h"
#include "SceneStateBlueprintPropertyReference.h"
#include "SceneStatePropertyReference.h"

#if WITH_EDITOR
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "SceneStatePropertyReferenceMetadata.h"
#endif

namespace UE::SceneState
{

namespace Private
{

#if WITH_EDITOR
/** Supported Property Reference Types */
static const FLazyName TYPE_Wildcard(TEXT("Wildcard"));
static const FLazyName TYPE_Bool(TEXT("bool"));
static const FLazyName TYPE_Byte(TEXT("byte"));
static const FLazyName TYPE_Int32(TEXT("int32"));
static const FLazyName TYPE_Int64(TEXT("int64"));
static const FLazyName TYPE_Float(TEXT("float"));
static const FLazyName TYPE_Double(TEXT("double"));
static const FLazyName TYPE_Name(TEXT("Name"));
static const FLazyName TYPE_String(TEXT("String"));
static const FLazyName TYPE_Text(TEXT("Text"));
static const FLazyName TYPE_AnyStruct(TEXT("AnyStruct"));

struct FReferenceTypeInfo
{
	TArray<FString> Types;
	bool bIsRefToArray = false;
	bool bCanRefToArray = false;
};

FReferenceTypeInfo GetReferenceTypeInfo(const FProperty* InReferenceProperty)
{
	check(IsPropertyReference(InReferenceProperty));

	FString RefType = InReferenceProperty->GetMetaData(Metadata::RefType);
	RefType.RemoveSpacesInline();

	FReferenceTypeInfo ReferenceTypeInfo;
	RefType.ParseIntoArray(ReferenceTypeInfo.Types, TEXT(","), /*bCullEmpty*/true);
	ReferenceTypeInfo.bIsRefToArray = InReferenceProperty->HasMetaData(Metadata::IsRefToArray);
	ReferenceTypeInfo.bCanRefToArray = InReferenceProperty->HasMetaData(Metadata::CanRefToArray);
	return ReferenceTypeInfo;
}

FReferenceTypeInfo GetReferenceTypeInfo(const FSceneStateBlueprintPropertyReference& InPropertyReference)
{
	const ESceneStatePropertyReferenceType ReferenceType = InPropertyReference.GetReferenceType();
	if (ReferenceType == ESceneStatePropertyReferenceType::None)
	{
		return FReferenceTypeInfo();
	}

	UEnum* const ReferenceTypeEnum = StaticEnum<ESceneStatePropertyReferenceType>();
	const int32 NameIndex = ReferenceTypeEnum->GetIndexByValue(static_cast<int64>(InPropertyReference.GetReferenceType()));

	FString RefType;
	if (ReferenceTypeEnum->HasMetaData(TEXT("ObjectRef"), NameIndex))
	{
		UObject* const TypeObject = InPropertyReference.GetTypeObject();
		RefType = TypeObject ? TypeObject->GetPathName() : FString();
		// todo: validation that the type object matches the ReferenceType (e.g. enums, etc)
	}
	else
	{
		RefType = ReferenceTypeEnum->GetMetaData(TEXT("RefType"), NameIndex);
	}

	if (RefType.IsEmpty())
	{
		return FReferenceTypeInfo();
	}

	FReferenceTypeInfo ReferenceTypeInfo;
	ReferenceTypeInfo.Types.Add(MoveTemp(RefType));
	ReferenceTypeInfo.bIsRefToArray = InPropertyReference.IsReferenceToArray();
	ReferenceTypeInfo.bCanRefToArray = false; // No support at the moment for BP Property References to have more than 1 pin type
	return ReferenceTypeInfo;
}

bool ArePropertyReferenceCompatible(const FProperty* InTargetReferenceProperty, const void* InTargetAddress, const FProperty* InSourceReferenceProperty, const void* InSourceAddress)
{
	check(IsPropertyReference(InSourceReferenceProperty) && IsPropertyReference(InTargetReferenceProperty));

	const TArray<FEdGraphPinType, TInlineAllocator<1>> SourceRefPins = GetPropertyReferencePinTypes(InSourceReferenceProperty, InSourceAddress);
	const TArray<FEdGraphPinType, TInlineAllocator<1>> TargetRefPins = GetPropertyReferencePinTypes(InTargetReferenceProperty, InTargetAddress);

	// NOTE: For now be stringent, and return false for properties that have the same Ref types metadata but in different order.
	return SourceRefPins == TargetRefPins;
}

bool IsPropertyCompatible(const FString& InTargetType, const FProperty* InTestProperty, bool bInIsTargetRefArray)
{
	const FName TargetTypeName = *InTargetType;

	if (TargetTypeName == TYPE_Wildcard)
	{
		return true;
	}
	if (TargetTypeName == TYPE_AnyStruct)
	{
		return InTestProperty->IsA<FStructProperty>();
	}
	if (TargetTypeName == TYPE_Bool)
	{
		return InTestProperty->IsA<FBoolProperty>();
	}
	if (TargetTypeName == TYPE_Byte)
	{
		return InTestProperty->IsA<FByteProperty>();
	}
	if (TargetTypeName == TYPE_Int32)
	{
		return InTestProperty->IsA<FIntProperty>();
	}
	if (TargetTypeName == TYPE_Int64)
	{
		return InTestProperty->IsA<FInt64Property>();
	}
	if (TargetTypeName == TYPE_Float)
	{
		return InTestProperty->IsA<FFloatProperty>();
	}
	if (TargetTypeName == TYPE_Double)
	{
		return InTestProperty->IsA<FDoubleProperty>();
	}
	if (TargetTypeName == TYPE_Name)
	{
		return InTestProperty->IsA<FNameProperty>();
	}
	if (TargetTypeName == TYPE_String)
	{
		return InTestProperty->IsA<FStrProperty>();
	}
	if (TargetTypeName == TYPE_Text)
	{
		return InTestProperty->IsA<FTextProperty>();
	}

	UField* TargetRefField = UClass::TryFindTypeSlow<UField>(InTargetType);
	if (!TargetRefField)
	{
		TargetRefField = LoadObject<UField>(nullptr, *InTargetType);
	}

	if (const FStructProperty* TestStructProperty = CastField<FStructProperty>(InTestProperty))
	{
		if (TestStructProperty->Struct && TestStructProperty->Struct->IsChildOf(Cast<UStruct>(TargetRefField)))
		{
			return true;
		}
	}
	else if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(InTestProperty))
	{
		if (ObjectProperty->PropertyClass && ObjectProperty->PropertyClass->IsChildOf(Cast<UClass>(TargetRefField)))
		{
			return true;
		}
	}
	else if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(InTestProperty))
	{
		if (EnumProperty->GetEnum() == TargetRefField)
		{
			return true;
		}
	}

	return false;
}

/** Retrieves the Ed Graph Pin Type for a given target type (in string) and a desired container type */
FEdGraphPinType GetTargetPinType(const FString& InTargetType, EPinContainerType InContainerType)
{
	if (InTargetType.IsEmpty())
	{
		return FEdGraphPinType();
	}

	const FName TargetTypeName = *InTargetType;

	FEdGraphPinType PinType;
	PinType.ContainerType = InContainerType;

	if (TargetTypeName == Private::TYPE_Wildcard)
	{
		PinType.PinCategory = UEdGraphSchema_K2::PC_Wildcard;
	}
	else if (TargetTypeName == Private::TYPE_AnyStruct)
	{
		PinType.PinCategory = UEdGraphSchema_K2::PC_Wildcard;
	}
	else if (TargetTypeName == Private::TYPE_Bool)
	{
		PinType.PinCategory = UEdGraphSchema_K2::PC_Boolean;
	}
	else if (TargetTypeName == Private::TYPE_Byte)
	{
		PinType.PinCategory = UEdGraphSchema_K2::PC_Byte;
	}
	else if (TargetTypeName == Private::TYPE_Int32)
	{
		PinType.PinCategory = UEdGraphSchema_K2::PC_Int;
	}
	else if (TargetTypeName == Private::TYPE_Int64)
	{
		PinType.PinCategory = UEdGraphSchema_K2::PC_Int64;
	}
	else if (TargetTypeName == Private::TYPE_Float)
	{
		PinType.PinCategory = UEdGraphSchema_K2::PC_Real;
		PinType.PinSubCategory = UEdGraphSchema_K2::PC_Float;
	}
	else if (TargetTypeName == Private::TYPE_Double)
	{
		PinType.PinCategory = UEdGraphSchema_K2::PC_Real;
		PinType.PinSubCategory = UEdGraphSchema_K2::PC_Double;
	}
	else if (TargetTypeName == Private::TYPE_Name)
	{
		PinType.PinCategory = UEdGraphSchema_K2::PC_Name;
	}
	else if (TargetTypeName == Private::TYPE_String)
	{
		PinType.PinCategory = UEdGraphSchema_K2::PC_String;
	}
	else if (TargetTypeName == Private::TYPE_Text)
	{
		PinType.PinCategory = UEdGraphSchema_K2::PC_Text;
	}
	else
	{
		UField* TargetRefField = UClass::TryFindTypeSlow<UField>(InTargetType);
		if (!TargetRefField)
		{
			TargetRefField = LoadObject<UField>(nullptr, *InTargetType);
		}

		if (UScriptStruct* Struct = Cast<UScriptStruct>(TargetRefField))
		{
			PinType.PinCategory = UEdGraphSchema_K2::PC_Struct;
			PinType.PinSubCategoryObject = Struct;
		}
		else if (UClass* ObjectClass = Cast<UClass>(TargetRefField))
		{
			PinType.PinCategory = UEdGraphSchema_K2::PC_Object;
			PinType.PinSubCategoryObject = ObjectClass;
		}
		else if (UEnum* Enum = Cast<UEnum>(TargetRefField))
		{
			PinType.PinCategory = UEdGraphSchema_K2::PC_Enum;
			PinType.PinSubCategoryObject = Enum;
			if (Enum->GetMaxEnumValue() <= static_cast<int64>(std::numeric_limits<uint8>::max()))
			{
				// Use byte for BP. It will use the correct picker and enum k2 node.
				PinType.PinCategory = UEdGraphSchema_K2::PC_Byte;
			}
		}
		else
		{
			checkf(false, TEXT("Typename in meta-data (%s) is invalid"), *InTargetType);
		}
	}

	return PinType;
}

/** Retrieves the Ed Graph Pin Type for a given target type (in string) and a desired container type */
TArray<FEdGraphPinType, TInlineAllocator<1>> GetTargetPinTypes(const FReferenceTypeInfo& InReferenceTypeInfo)
{
	TArray<FEdGraphPinType, TInlineAllocator<1>> PinTypes;
	PinTypes.Reserve(InReferenceTypeInfo.Types.Num());

	const EPinContainerType ContainerType = InReferenceTypeInfo.bIsRefToArray
		? EPinContainerType::Array
		: EPinContainerType::None;

	for (const FString& TargetType : InReferenceTypeInfo.Types)
	{
		const FEdGraphPinType PinType = GetTargetPinType(TargetType, ContainerType);
		PinTypes.Add(PinType);

		// If Property Reference supports arrays, add the array counterpart too (if the pin type is not an array/container already)
		if (!PinType.IsContainer() && InReferenceTypeInfo.bCanRefToArray)
		{
			FEdGraphPinType& ContainerPinType = PinTypes.Add_GetRef(PinType);
			ContainerPinType.ContainerType = EPinContainerType::Array;
		}
	}

	return PinTypes;
}
#endif

} // UE::SceneState::Private

bool IsPropertyReference(const FProperty* InProperty)
{
	const FStructProperty* StructProperty = CastField<FStructProperty>(InProperty);
	return StructProperty && StructProperty->Struct && StructProperty->Struct->IsChildOf<FSceneStatePropertyReference>();
}

#if WITH_EDITOR
bool IsPropertyReferenceCompatible(const FProperty* InReferenceProperty, const void* InReferenceAddress, const FProperty* InSourceProperty, const void* InSourceAddress)
{
	check(InReferenceProperty && InSourceProperty);

	if (!ensure(IsPropertyReference(InReferenceProperty)))
	{
		return false;
	}

	if (IsPropertyReference(InSourceProperty))
	{
		return Private::ArePropertyReferenceCompatible(InReferenceProperty, InReferenceAddress, InSourceProperty, InSourceAddress);
	}

	const FStructProperty* ReferenceStructProperty = CastFieldChecked<FStructProperty>(InReferenceProperty);
	check(ReferenceStructProperty->Struct);

	Private::FReferenceTypeInfo ReferenceTypeInfo;

	if (ReferenceStructProperty->Struct->IsChildOf<FSceneStateBlueprintPropertyReference>())
	{
		const FSceneStateBlueprintPropertyReference& PropertyReference = *static_cast<const FSceneStateBlueprintPropertyReference*>(InReferenceAddress);
		ReferenceTypeInfo = Private::GetReferenceTypeInfo(PropertyReference);
	}
	else
	{
		ReferenceTypeInfo = Private::GetReferenceTypeInfo(InReferenceProperty);
	}

	const FProperty* TestProperty = InSourceProperty;
	if (ReferenceTypeInfo.bIsRefToArray || ReferenceTypeInfo.bCanRefToArray)
	{
		if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(TestProperty))
		{
			TestProperty = ArrayProperty->Inner;
		}
		else if(!ReferenceTypeInfo.bCanRefToArray)
		{
			return false;
		}
	}

	for (const FString& TargetType : ReferenceTypeInfo.Types)
	{
		if (Private::IsPropertyCompatible(TargetType, TestProperty, ReferenceTypeInfo.bIsRefToArray))
		{
			return true;
		}
	}
	return false;
}

FEdGraphPinType GetPropertyReferencePinType(const FProperty* InProperty, const void* InReferenceAddress)
{
	const TArray<FEdGraphPinType, TInlineAllocator<1>> PropertyReferenceInfo = GetPropertyReferencePinTypes(InProperty, InReferenceAddress);
	return !PropertyReferenceInfo.IsEmpty()
		? PropertyReferenceInfo[0]
		: FEdGraphPinType();
}

TArray<FEdGraphPinType, TInlineAllocator<1>> GetPropertyReferencePinTypes(const FSceneStateBlueprintPropertyReference& InPropertyReference)
{
	return Private::GetTargetPinTypes(Private::GetReferenceTypeInfo(InPropertyReference));
}

TArray<FEdGraphPinType, TInlineAllocator<1>> GetPropertyReferencePinTypes(const FProperty* InProperty, const void* InReferenceAddress)
{
	if (!ensure(InProperty) || !ensure(IsPropertyReference(InProperty)))
	{
		return {};
	}

	const FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(InProperty);
	check(StructProperty->Struct);

	if (StructProperty->Struct->IsChildOf<FSceneStateBlueprintPropertyReference>())
	{
		const FSceneStateBlueprintPropertyReference& BlueprintPropertyReference = *static_cast<const FSceneStateBlueprintPropertyReference*>(InReferenceAddress);
		return GetPropertyReferencePinTypes(BlueprintPropertyReference);
	}

	return Private::GetTargetPinTypes(Private::GetReferenceTypeInfo(InProperty));
}
#endif

} // UE::SceneState