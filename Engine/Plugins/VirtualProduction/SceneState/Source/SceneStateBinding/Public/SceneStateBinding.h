// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyBindingBinding.h"
#include "SceneStateBindingDataHandle.h"
#include "SceneStateBinding.generated.h"

#define UE_API SCENESTATEBINDING_API

/** Represents a binding from a source property to a destination property */
USTRUCT()
struct FSceneStateBinding : public FPropertyBindingBinding
{
	GENERATED_BODY()

	using FPropertyBindingBinding::FPropertyBindingBinding;

	//~ Begin FPropertyBindingBinding
	UE_API virtual FConstStructView GetSourceDataHandleStruct() const override;
	//~ End FPropertyBindingBinding

	/** Handle to the data containing the source property */
	UPROPERTY()
	FSceneStateBindingDataHandle SourceDataHandle;

	/** Handle to the data containing the target property */
	UPROPERTY()
	FSceneStateBindingDataHandle TargetDataHandle;
};

#undef UE_API
