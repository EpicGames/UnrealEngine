// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyBindingBindingCollection.h"
#include "SceneStateBinding.h"
#include "SceneStateBindingDesc.h"
#include "SceneStateBindingReference.h"
#include "SceneStateBindingCollection.generated.h"

#define UE_API SCENESTATEBINDING_API

struct FSceneStatePropertyReference;

namespace UE::SceneState
{
	struct FBindingFunctionInfo;

	namespace Editor
	{
		class FBindingCompiler;
	}
}

/** Holds all the property bindings to use in execution */
USTRUCT()
struct FSceneStateBindingCollection : public FPropertyBindingBindingCollection
{
	GENERATED_BODY()

	TArrayView<FSceneStateBindingDesc> GetMutableBindingDescs()
	{
		return BindingDescs;
	}

	TArrayView<FSceneStateBinding> GetMutableBindings()
	{
		return Bindings;
	}

	TConstArrayView<FSceneStateBinding> GetBindings() const
	{
		return Bindings;
	}

#if WITH_EDITOR
	UE_API FPropertyBindingPath AddBindingFunction(const UE::SceneState::FBindingFunctionInfo& InFunctionInfo, TConstArrayView<FPropertyBindingPathSegment> InSourcePathSegments, const FPropertyBindingPath& InTargetPath);
#endif

	/** Finds the binding desc matching the given data handle */
	UE_API const FSceneStateBindingDesc* FindBindingDesc(FSceneStateBindingDataHandle InDataHandle) const;

	/** Retrieves the resolved reference for the given property reference */
	UE_API const FSceneStateBindingResolvedReference* FindResolvedReference(const FSceneStatePropertyReference& InPropertyReference) const;

	/**
	 * Resolves the property access for the given reference and data view
	 * @note This does not deal with recursive resolution (for cases where the reference is to another reference).
	 * Prefer using UE::SceneState::ResolveProperty instead.
	 */
	UE_API uint8* ResolveProperty(const FSceneStateBindingResolvedReference& InResolvedReference, FPropertyBindingDataView InDataView) const;

	//~ Begin FPropertyBindingBindingCollection
#if WITH_EDITOR
	UE_API virtual FPropertyBindingBinding* AddBindingInternal(const FPropertyBindingPath& InSourcePath, const FPropertyBindingPath& InTargetPath) override;
	UE_API virtual void CopyBindingsInternal(const FGuid InFromStructId, const FGuid InToStructId) override;
	UE_API virtual void RemoveBindingsInternal(TFunctionRef<bool(FPropertyBindingBinding&)> InPredicate) override;
	UE_API virtual bool HasBindingInternal(TFunctionRef<bool(const FPropertyBindingBinding&)> InPredicate) const override;
	UE_API virtual const FPropertyBindingBinding* FindBindingInternal(TFunctionRef<bool(const FPropertyBindingBinding&)> InPredicate) const override;
#endif
	UE_API virtual int32 GetNumBindings() const override;
	UE_API virtual int32 GetNumBindableStructDescriptors() const override;
	UE_API virtual const FPropertyBindingBindableStructDescriptor* GetBindableStructDescriptorFromHandle(FConstStructView InSourceHandleView) const override;
	UE_API virtual void ForEachBinding(TFunctionRef<void(const FPropertyBindingBinding& Binding)> InFunction) const override;
	UE_API virtual void ForEachBinding(FPropertyBindingIndex16 InBegin, FPropertyBindingIndex16 InEnd, TFunctionRef<void(const FPropertyBindingBinding& Binding, const int32 BindingIndex)> InFunction) const override;
	UE_API virtual void ForEachMutableBinding(TFunctionRef<void(FPropertyBindingBinding& Binding)> InFunction) override;
	UE_API virtual void VisitBindings(TFunctionRef<EVisitResult(const FPropertyBindingBinding& Binding)> InFunction) const override;
	UE_API virtual void VisitBindings(FPropertyBindingIndex16 InBegin, FPropertyBindingIndex16 InEnd, TFunctionRef<EVisitResult(const FPropertyBindingBinding& Binding, const int32 BindingIndex)> InFunction) const override;
	UE_API virtual void VisitMutableBindings(TFunctionRef<EVisitResult(FPropertyBindingBinding& Binding)> InFunction) override;
	UE_API virtual void OnReset() override;
	UE_API virtual void VisitSourceStructDescriptorInternal(TFunctionRef<EVisitResult(const FPropertyBindingBindableStructDescriptor&)> InFunction) const override;
	[[nodiscard]] UE_API virtual bool OnResolvingPaths() override;
	//~ End FPropertyBindingBindingCollection

private:
	UPROPERTY()
	TArray<FSceneStateBindingDesc> BindingDescs;

	/** Contains all the bindings from a source data to a target data */
	UPROPERTY()
	TArray<FSceneStateBinding> Bindings;

	/**
	 * Contains all the reference bindings from a source data.
	 * This gets set and populated during scene state compilation.
	 */
	UPROPERTY()
	TArray<FSceneStateBindingReference> References;

	/** Contains the references that have been resolved for use at runtime */
	UPROPERTY()
	TArray<FSceneStateBindingResolvedReference> ResolvedReferences;

	friend UE::SceneState::Editor::FBindingCompiler;
};

#undef UE_API
