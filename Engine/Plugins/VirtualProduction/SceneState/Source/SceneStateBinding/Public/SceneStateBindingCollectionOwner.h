// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyBindingBindingCollectionOwner.h"
#include "UObject/Interface.h"
#include "SceneStateBindingCollectionOwner.generated.h"

struct FSceneStateBindingCollection;
struct FSceneStateBindingDesc;
struct FSceneStateBindingFunction;

namespace UE::SceneState
{
	struct FBindingFunctionInfo;
}

UINTERFACE(MinimalAPI, meta=(CannotImplementInterfaceInBlueprint))
class USceneStateBindingCollectionOwner : public UPropertyBindingBindingCollectionOwner
{
	GENERATED_BODY()
};

class ISceneStateBindingCollectionOwner : public IPropertyBindingBindingCollectionOwner
{
	GENERATED_BODY()

public:
	/** Gets a reference to the binding collection (mutable) that the owner has */
	virtual FSceneStateBindingCollection& GetBindingCollection() = 0;

	/** Gets a reference to the binding collection that the owner has */
	virtual const FSceneStateBindingCollection& GetBindingCollection() const = 0;

#if WITH_EDITOR
	/** Iterates all functions that can be created and bound */
	virtual bool ForEachBindableFunction(TFunctionRef<bool(const FSceneStateBindingDesc&, const UE::SceneState::FBindingFunctionInfo&)> InFunc) const = 0;
#endif
};
