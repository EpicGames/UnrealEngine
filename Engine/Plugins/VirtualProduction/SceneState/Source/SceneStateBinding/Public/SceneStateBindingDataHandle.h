// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneStateBindingDataHandle.generated.h"

#define UE_API SCENESTATEBINDING_API

UENUM()
enum class ESceneStateDataType : uint8
{
	Root,
	Task,
	EventHandler,
	TaskExtension,
	StateMachine,
	Transition,
	Function,
	State,
};

USTRUCT()
struct FSceneStateBindingDataHandle
{
	GENERATED_BODY()

	FSceneStateBindingDataHandle() = default;

	struct FExternalData
	{
		uint8 Type = 0;
		int32 Index = INDEX_NONE;
		int32 SubIndex = INDEX_NONE;
	};
	UE_API static FSceneStateBindingDataHandle MakeExternalDataHandle(const FExternalData& InExternalData);

	explicit FSceneStateBindingDataHandle(ESceneStateDataType InDataType, int32 InDataIndex = INDEX_NONE, int32 InDataSubIndex = INDEX_NONE)
		: Type(static_cast<uint16>(InDataType))
		, Index(GetIndexSafeChecked(InDataIndex))
		, SubIndex(GetIndexSafeChecked(InDataSubIndex))
	{
	}

	bool operator==(const FSceneStateBindingDataHandle& InOther) const
	{
		return Type == InOther.Type && Index == InOther.Index && SubIndex == InOther.SubIndex;
	}

	bool IsValid() const
	{
		return Type != InvalidIndex;
	}

	bool IsExternalDataType() const
	{
		return Type > std::numeric_limits<uint8>::max();
	}

	bool IsDataType(ESceneStateDataType InDataType) const
	{
		return Type == static_cast<uint16>(InDataType); 
	}

	uint8 GetDataType() const
	{
		return IsExternalDataType()
			? static_cast<uint8>(Type >> 8)
			: static_cast<uint8>(Type);
	}

	uint16 GetDataIndex() const
	{
		return Index;
	}

	uint16 GetDataSubIndex() const
	{
		return SubIndex;
	}

	uint64 AsNumber() const
	{
		return static_cast<uint64>(Type) << 32 | static_cast<uint64>(Index) << 16 | static_cast<uint64>(SubIndex);
	}

	friend uint32 GetTypeHash(const FSceneStateBindingDataHandle& InDataHandle)
	{
		uint32 Hash = GetTypeHash(InDataHandle.Type);
		Hash = HashCombineFast(Hash, GetTypeHash(InDataHandle.Index));
		Hash = HashCombineFast(Hash, GetTypeHash(InDataHandle.SubIndex));
		return Hash;
	}

private:
	static constexpr uint16 InvalidIndex = std::numeric_limits<uint16>::max();

	static uint16 GetIndexSafeChecked(int32 InIndex)
	{
		checkf(FMath::IsWithin(InIndex, uint16(0), InvalidIndex) || InIndex == INDEX_NONE, TEXT("Index %d out of bounds! Max: %d"), InIndex, InvalidIndex - 1);
		return InIndex == INDEX_NONE ? InvalidIndex : static_cast<uint16>(InIndex);
	}

	/**
	 * The Data Type. The first 8 bits are reserved for Internal Data Types (see ESceneStateDataType)
	 * The last 8 bits can be used to represent External Data Types (tracked by external implementations)
	 */
	UPROPERTY()
	uint16 Type = InvalidIndex;

	/** Primary Index to access the data */
	UPROPERTY()
	uint16 Index = InvalidIndex;

	/** Secondary Index to the actual data residing somewhere within the data from the Primary Index */
	UPROPERTY()
	uint16 SubIndex = InvalidIndex;
};

#undef UE_API
