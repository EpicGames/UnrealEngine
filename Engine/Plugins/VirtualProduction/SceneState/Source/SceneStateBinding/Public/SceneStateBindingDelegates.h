// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Misc/Guid.h"
#include "UObject/ObjectPtr.h"

#define UE_API SCENESTATEBINDING_API

class UObject;

namespace UE::SceneState
{

#if WITH_EDITOR
	struct FStructIdChange
	{
		/** Owner of the binding collection */
		TObjectPtr<UObject> BindingOwner;
		/** Map of the old struct id to the new id */
		TMap<FGuid, FGuid> OldToNewStructIdMap;
	};
	/** Delegate called when a struct id has changed */
	extern UE_API TMulticastDelegate<void(const FStructIdChange&)> OnStructIdChanged;
#endif

} // UE::SceneState

#undef UE_API
