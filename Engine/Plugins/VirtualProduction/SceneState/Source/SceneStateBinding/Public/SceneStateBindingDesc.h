// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneStateBindingDataHandle.h"
#include "PropertyBindingBindableStructDescriptor.h"
#include "SceneStateBindingDesc.generated.h"

USTRUCT()
struct FSceneStateBindingDesc : public FPropertyBindingBindableStructDescriptor
{
	GENERATED_BODY()

	FSceneStateBindingDesc() = default;

#if WITH_EDITOR
	explicit FSceneStateBindingDesc(const FName InName, const UStruct* InStruct, const FGuid InGuid, const FSceneStateBindingDataHandle& InDataHandle)
		: Super(InName, InStruct, InGuid)
		, DataHandle(InDataHandle)
	{
	}
#endif

	UPROPERTY()
	FSceneStateBindingDataHandle DataHandle;

#if WITH_EDITORONLY_DATA
	/** Section of the bindable struct. Can be used to group bindable structs within the same category menu. */
	UPROPERTY()
	FText Section;
#endif
};
