// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/InstancedStruct.h"
#include "StructUtils/StructView.h"
#include "SceneStateBindingFunction.generated.h"

#define UE_API SCENESTATEBINDING_API

namespace UE::SceneState
{

#if WITH_EDITOR
/** Arguments to create a valid FSceneStateBindingFunction */
struct FBindingFunctionInfo
{
	/** Info to create a new function. Struct type is required. The struct memory is optional and only used as a template to instantiate the function */
	FConstStructView FunctionTemplate;
	/** Info to create a new function instance. Struct type is required. The struct memory is optional and only used as a template to instantiate the function instance */
	FConstStructView InstanceTemplate;
};
#endif

} // UE::SceneState

/** Holds a function object and its template instance data for compilation. Used in editor only */
USTRUCT()
struct FSceneStateBindingFunction
{
	GENERATED_BODY()

	FSceneStateBindingFunction() = default;

#if WITH_EDITOR
	/** Initializes the binding function with the given function info */
	UE_API explicit FSceneStateBindingFunction(const UE::SceneState::FBindingFunctionInfo& InFunctionInfo);

	/** Returns whether this object has a valid function and function instance */
	UE_API bool IsValid() const;
#endif

#if WITH_EDITORONLY_DATA
	/** Id of the function */
	UPROPERTY(VisibleAnywhere, Category="Scene State")
	FGuid FunctionId;

	/** The function template */
	UPROPERTY(EditAnywhere, Category="Scene State")
	FInstancedStruct Function;

	/** The function instance template */
	UPROPERTY(EditAnywhere, Category="Scene State")
	mutable FInstancedStruct FunctionInstance;
#endif
};

#undef UE_API
