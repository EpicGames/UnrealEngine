// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"

namespace UE::SceneState::Metadata
{
#if WITH_EDITOR
	/**
	 * Describes that the property does not allow binding. But this no binding policy does not propagate to its child properties
	 * For containers, this policy would apply for both the container and inner properties.
	 */
	const FLazyName NoBindingSelfOnly(TEXT("NoBindingSelfOnly"));

	/**
	 * Version of 'NoBindingSelfOnly' where it applies the no binding policy to the container property only, and not its inner properties
	 * E.g. for a TArray<FVector> property:
	 * If 'NoBindingSelfOnly' is used, both the array as a whole and its FVector inner property would be unbindable and only the FVector's individual components (X, Y, Z) would be bindable.
	 * If 'NoBindingContainerSelfOnly' is used, the array would still be non-bindable, but the FVector inner property would now be bindable.
	 */
	const FLazyName NoBindingContainerSelfOnly(TEXT("NoBindingContainerSelfOnly"));
#endif
}
