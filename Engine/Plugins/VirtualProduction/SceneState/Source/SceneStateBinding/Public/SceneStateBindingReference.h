// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyBindingPath.h"
#include "SceneStateBindingDataHandle.h"
#include "SceneStateBindingReference.generated.h"

USTRUCT()
struct FSceneStateBindingReference
{
	GENERATED_BODY()

	/** Source property path of the reference */
	UPROPERTY()
	FPropertyBindingPath SourcePropertyPath;

	/** Describes how to get the source data pointer */
	UPROPERTY()
	FSceneStateBindingDataHandle SourceDataHandle;
};

/** A resolved FSceneStateReference that contains more information about the source */
USTRUCT()
struct FSceneStateBindingResolvedReference
{
	GENERATED_BODY()

	/** Source property access. */
	UPROPERTY()
	FPropertyBindingPropertyIndirection SourceIndirection;

	/** Cached pointer to the leaf property of the access. */
	const FProperty* SourceLeafProperty = nullptr;

	/** Type of the source data, used for validation. */
	UPROPERTY(Transient)
	TObjectPtr<const UStruct> SourceStructType = nullptr;

	/** Describes how to get the source data pointer. */
	UPROPERTY()
	FSceneStateBindingDataHandle SourceDataHandle;
};
