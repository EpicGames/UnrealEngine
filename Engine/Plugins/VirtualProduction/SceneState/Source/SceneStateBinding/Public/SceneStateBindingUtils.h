// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyBindingDataView.h"
#include "Templates/Function.h"

#define UE_API SCENESTATEBINDING_API

class UObject;
struct FGuid;
struct FSceneStateBindingCollection;
struct FSceneStateBindingDataHandle;

namespace UE::SceneState
{

#if WITH_EDITOR
	/**
	 * Update the bindings of a source struct id into a target one.
	 * It goes through the given object's outer chain to get to the binding collection owner to do this operation
	 * @param InObject object to use to find the binding collection owner
	 * @param InOldStructId the struct id to queue copy bindings from
	 * @param InNewStructId the struct id to queue copy bindings to
	 */
	UE_API void HandleStructIdChanged(UObject& InObject, const FGuid& InOldStructId, const FGuid& InNewStructId);
#endif

	struct FPatchBindingParams
	{
		/** Binding collection to patch */
		FSceneStateBindingCollection& BindingCollection;
		/** Functor to find the data struct for a given Data Handle */
		TFunctionRef<const UStruct*(const FSceneStateBindingDataHandle&)> FindDataStructFunctor;
	};
	/** Patches invalidated structs (property bags, user defined structs, etc.) in the given binding collection */
	UE_API void PatchBindingCollection(const FPatchBindingParams& InParams);

	/** Patches any invalidated structs (property bags, user defined structs, etc.) in the binding descs of the given binding collection */
	void PatchBindingDescs(const FPatchBindingParams& InParams);

	/** Patches any invalidated structs (property bags, user defined structs, etc.) in the bindings of the given binding collection */
	void PatchBindings(const FPatchBindingParams& InParams);

	/** Patches any invalidated structs (property bags, user defined structs, etc.) in the copy batches of the given binding collection */
	void PatchCopyBatches(const FPatchBindingParams& InParams);

} // UE::SceneState

#undef UE_API
