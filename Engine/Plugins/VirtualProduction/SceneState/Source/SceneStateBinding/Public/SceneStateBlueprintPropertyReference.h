// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneStatePropertyReference.h"
#include "SceneStateBlueprintPropertyReference.generated.h"

#define UE_API SCENESTATEBINDING_API

#if WITH_EDITOR
struct FEdGraphPinType;
#endif

UENUM()
enum class ESceneStatePropertyReferenceType : uint8
{
	None       UMETA(Hidden),
	Bool       UMETA(RefType="bool"),
	Byte       UMETA(RefType="byte"),
	Int32      UMETA(RefType="int32"),
	Int64      UMETA(RefType="int64"),
	Float      UMETA(RefType="float"),
	Double     UMETA(RefType="double"),
	Name       UMETA(RefType="Name"),
	String     UMETA(RefType="String"),
	Text       UMETA(RefType="Text"),
	Enum       UMETA(ObjectRef),
	Struct     UMETA(ObjectRef),
	Object     UMETA(ObjectRef),
	SoftObject UMETA(ObjectRef),
	Class      UMETA(ObjectRef),
	SoftClass  UMETA(ObjectRef),
};

/** Property Reference as a Blueprint Type. Usable in blueprints (e.g. tasks) and parameters */
USTRUCT(BlueprintType, DisplayName="Scene State Property Reference")
struct FSceneStateBlueprintPropertyReference : public FSceneStatePropertyReference
{
	GENERATED_BODY()

#if WITH_EDITOR
	/** Builds a property reference instance based on the given pin type */
	UE_API static FSceneStateBlueprintPropertyReference BuildFromPinType(const FEdGraphPinType& InPinType);
#endif

	/** Returns the property reference's type */
	ESceneStatePropertyReferenceType GetReferenceType() const
	{
		return ReferenceType;
	}

	/** Returns true if referenced property is an array. */
	bool IsReferenceToArray() const
	{
		return bIsReferenceToArray;
	}

	/** Returns selected ScriptStruct, Class or Enum. */
	UObject* GetTypeObject() const
	{
		return TypeObject;
	}

	bool operator==(const FSceneStateBlueprintPropertyReference& InOther) const
	{
		return ReferenceType == InOther.ReferenceType
			&& bIsReferenceToArray == InOther.bIsReferenceToArray
			&& TypeObject == InOther.TypeObject;
	}

	/** Get Property Member Name functions */
	UE_API static FName GetReferenceTypeMemberName();
	UE_API static FName GetIsReferenceToArrayMemberName();
	UE_API static FName GetTypeObjectMemberName();

private:
	/** Specifies the type of property to reference */
	UPROPERTY(EditAnywhere, Category="Reference Type")
	ESceneStatePropertyReferenceType ReferenceType = ESceneStatePropertyReferenceType::None;

	/** If specified, the reference is to an TArray<RefType> */
	UPROPERTY(EditAnywhere, Category="Reference Type")
	bool bIsReferenceToArray = false;

	/** Specifies the type of property to reference together with ReferenceType, used for Enums, Structs, Objects and Classes. */
	UPROPERTY(EditAnywhere, Category="Reference Type")
	TObjectPtr<UObject> TypeObject = nullptr;
};

#undef UE_API
