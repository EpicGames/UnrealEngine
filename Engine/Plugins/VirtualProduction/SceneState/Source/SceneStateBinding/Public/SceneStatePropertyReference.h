// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <limits>
#include "SceneStatePropertyReference.generated.h"

/**
 * Property References allows the binding system to handle a property by reference rather than by copy.
 * A reference holds a single 'ReferenceIndex' that points to the index of both a 'Reference' and its 'Resolved Reference'.
 * To get the property by reference, the binding collection (to get the data handle) and the execution context (to get the data view) are required.
 * @see FSceneStateBindingCollection, FSceneStateBindingReference, FSceneStateBindingResolvedReference.
 *
 * This is useful for tasks that need to write to external properties (e.g. as a form of output).
 * The expected type of the reference should be set in "RefType" meta specifier.
 * Meta specifiers for the type:
 *
 * RefType="<type>"
 * Specifies the type of property that the reference supports.
 * Supported types are: bool, byte, int32, int64, float, double, Name, String, Text, Wildcard, UObject pointers, and structs.
 * Wildcards is a 'bypass' so that any valid FProperty is accepted, and is up to implementer to handle this.
 * Structs and Objects must use full path name. (e.g. for Actor type "/Script/Engine.Actor")
 *
 * IsRefToArray
 * Specified to mean that the reference is to an TArray<RefType>
 *
 * CanRefToArray
 * Specified to mean that the reference can bind to a RefType or TArray<RefType>
 */
USTRUCT()
struct FSceneStatePropertyReference
{
	GENERATED_BODY()

	/** Max number of indices that can be used */
	static constexpr uint16 IndexCapacity = std::numeric_limits<uint16>::max();

	/** Returns whether the index is a valid index. Does not guarantee that the reference itself is to a valid property */
	bool IsValidIndex() const
	{
		return ReferenceIndex != IndexCapacity;
	}

	UPROPERTY()
	uint16 ReferenceIndex = IndexCapacity;
};
