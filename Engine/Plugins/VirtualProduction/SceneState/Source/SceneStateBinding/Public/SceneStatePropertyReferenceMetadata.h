// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"

namespace UE::SceneState::Metadata
{
#if WITH_EDITOR
	const FLazyName IsRefToArray(TEXT("IsRefToArray"));
	const FLazyName CanRefToArray(TEXT("CanRefToArray"));
	const FLazyName RefType(TEXT("RefType"));
#endif
}
