// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Concepts/BaseStructureProvider.h"
#include "UObject/EnumProperty.h"
#include "UObject/StrProperty.h"
#include "UObject/TextProperty.h"
#include "UObject/UnrealType.h"

#define UE_API SCENESTATEBINDING_API

struct FEdGraphPinType;
struct FSceneStateBlueprintPropertyReference;

namespace UE::SceneState
{
	/** Returns whether the given property is a struct property of struct FSceneStatePropertyReference */
	UE_API bool IsPropertyReference(const FProperty* InProperty);

#if WITH_EDITOR
	/** Returns whether a source property is compatible with the reference property */
	UE_API bool IsPropertyReferenceCompatible(const FProperty* InReferenceProperty, const void* InReferenceAddress, const FProperty* InSourceProperty, const void* InSourceAddress);

	/** Returns the first graph pin type based on a given Property Reference property's meta-data */
	UE_API FEdGraphPinType GetPropertyReferencePinType(const FProperty* InProperty, const void* InReferenceAddress);

	/** Returns the graph pin type based on a given Property Reference */
	UE_API TArray<FEdGraphPinType, TInlineAllocator<1>> GetPropertyReferencePinTypes(const FSceneStateBlueprintPropertyReference& InPropertyReference);

	/** Returns the graph pin types based on a given Property Reference property's meta-data */
	UE_API TArray<FEdGraphPinType, TInlineAllocator<1>> GetPropertyReferencePinTypes(const FProperty* InProperty, const void* InReferenceAddress);
#endif

	template<class InDataType>
	struct TDataTypeHelper
	{
	};

	template<>
	struct TDataTypeHelper<void>
	{
		static bool IsValid(const FProperty* InProperty)
		{
			return true;
		}
	};

	template<>
	struct TDataTypeHelper<FBoolProperty::TCppType>
	{
		static bool IsValid(const FProperty* InProperty)
		{
			return InProperty && InProperty->IsA<FBoolProperty>();
		};
	};

	template<>
	struct TDataTypeHelper<FByteProperty::TCppType>
	{
		static bool IsValid(const FProperty* InProperty)
		{
			return InProperty && InProperty->IsA<FByteProperty>();
		};
	};

	template<>
	struct TDataTypeHelper<FIntProperty::TCppType>
	{
		static bool IsValid(const FProperty* InProperty)
		{
			return InProperty && InProperty->IsA<FIntProperty>();
		};
	};

	template<>
	struct TDataTypeHelper<FInt64Property::TCppType>
	{
		static bool IsValid(const FProperty* InProperty)
		{
			return InProperty && InProperty->IsA<FInt64Property>();
		};
	};

	template<>
	struct TDataTypeHelper<FFloatProperty::TCppType>
	{
		static bool IsValid(const FProperty* InProperty)
		{
			return InProperty && InProperty->IsA<FFloatProperty>();
		};
	};

	template<>
	struct TDataTypeHelper<FDoubleProperty::TCppType>
	{
		static bool IsValid(const FProperty* InProperty)
		{
			return InProperty && InProperty->IsA<FDoubleProperty>();
		};
	};

	template<>
	struct TDataTypeHelper<FNameProperty::TCppType>
	{
		static bool IsValid(const FProperty* InProperty)
		{
			return InProperty && InProperty->IsA<FNameProperty>();
		};
	};

	template<>
	struct TDataTypeHelper<FStrProperty::TCppType>
	{
		static bool IsValid(const FProperty* InProperty)
		{
			return InProperty && InProperty->IsA<FStrProperty>();
		};
	};

	template<>
	struct TDataTypeHelper<FTextProperty::TCppType>
	{
		static bool IsValid(const FProperty* InProperty)
		{
			return InProperty && InProperty->IsA<FTextProperty>();
		};
	};

	template<class InDataType UE_REQUIRES(TIsTArray_V<InDataType>)>
	struct TDataTypeHelper<InDataType>
	{
		static bool IsValid(const FProperty* InProperty)
		{
			const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(InProperty);
			return ArrayProperty && TDataTypeHelper<typename InDataType::ElementType>::IsValid(*ArrayProperty->Inner);
		}
	};

	template<class InDataType UE_REQUIRES(TModels_V<CBaseStructureProvider, InDataType>)>
	struct TDataTypeHelper<InDataType>
	{
		static bool IsValid(const FProperty* InProperty)
		{
			const FStructProperty* StructProperty = CastField<FStructProperty>(InProperty);
			return StructProperty && StructProperty->Struct && StructProperty->Struct->IsChildOf(TBaseStructure<InDataType>::Get());
		}
	};

	template<class InDataType UE_REQUIRES(TModels_V<CStaticClassProvider, typename TRemovePointer<InDataType>::Type>)>
	struct TDataTypeHelper<InDataType>
	{
		static bool IsValid(const FProperty* InProperty)
		{
			const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(InProperty);
			return ObjectProperty && ObjectProperty->PropertyClass && ObjectProperty->PropertyClass->IsChildOf(TRemovePointer<InDataType>::Type::StaticClass());
		}
	};

	template<class InDataType UE_REQUIRES(TIsEnum<InDataType>::Value)>
	struct TDataTypeHelper<InDataType>
	{
		static bool IsValid(const FProperty* InProperty)
		{
			const FEnumProperty* EnumProperty = CastField<FEnumProperty>(InProperty);
			return EnumProperty && EnumProperty->GetEnum() == StaticEnum<InDataType>();
		}
	};
}

#undef UE_API
