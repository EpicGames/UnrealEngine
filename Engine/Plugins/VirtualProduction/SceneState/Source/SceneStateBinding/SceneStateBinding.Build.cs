// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SceneStateBinding : ModuleRules
{
    public SceneStateBinding(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "Core",
                "CoreUObject",
                "PropertyBindingUtils",
            }
        );

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "Engine",
            }
        );

        if (Target.Type == TargetRules.TargetType.Editor)
        {
            PrivateDependencyModuleNames.AddRange(
                new string[]
                {
                    "BlueprintGraph",
                }
            );
        }
    }
}