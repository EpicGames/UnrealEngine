// Copyright Epic Games, Inc. All Rights Reserved.

#include "Actions/SceneStateBlueprintAction_Graph.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "SceneStateBlueprintUtils.h"
#include "SceneStateMachineGraph.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "SceneStateBlueprintAction_Graph"

namespace UE::SceneState::Graph
{

FBlueprintAction_Graph::FBlueprintAction_Graph(const FArguments& InArgs)
{
	EdGraph = InArgs.Graph;
	GraphType = InArgs.GraphType;
	Grouping = InArgs.Grouping;
	SectionID = InArgs.SectionID;
	Icon = InArgs.Icon;

	if (EdGraph)
	{
		FuncName = EdGraph->GetFName();

		if (const UEdGraphSchema* Schema = EdGraph->GetSchema())
		{
			FGraphDisplayInfo DisplayInfo;
			Schema->GetGraphDisplayInformation(*EdGraph, DisplayInfo);

			UpdateSearchData(MoveTemp(DisplayInfo.DisplayName), MoveTemp(DisplayInfo.Tooltip), InArgs.Category, FText());
		}
	}
}

FName FBlueprintAction_Graph::StaticGetTypeId()
{
	static FName Type = TEXT("UE::SceneState::Graph::FBlueprintAction_Graph");
	return Type;
}

const FSlateBrush* FBlueprintAction_Graph::GetPaletteIcon() const
{
	return Icon.GetIcon();
}

FName FBlueprintAction_Graph::GetTypeId() const
{
	return StaticGetTypeId();
}

void FBlueprintAction_Graph::MovePersistentItemToCategory(const FText& InNewCategory)
{
	USceneStateMachineGraph* const StateMachineGraph = Cast<USceneStateMachineGraph>(EdGraph);

	// Only allow top-level state machines to have a category
	if (StateMachineGraph && GraphType == EEdGraphSchemaAction_K2Graph::Graph)
	{
		UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraphChecked(EdGraph);

		FScopedTransaction Transaction(LOCTEXT("SetStateMachineGraphCategory", "Set Category"));
		StateMachineGraph->Modify();
		StateMachineGraph->Category = InNewCategory;
		FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
	}
}

int32 FBlueprintAction_Graph::GetReorderIndexInContainer() const
{
	return FindIndexOfGraphInParent(EdGraph);
}

bool FBlueprintAction_Graph::ReorderToBeforeAction(TSharedRef<FEdGraphSchemaAction> OtherAction)
{
	if (OtherAction->GetTypeId() != GetTypeId())
	{
		return false;
	}

	const int32 CurrentIndex = GetReorderIndexInContainer();
	const int32 TargetIndex = OtherAction->GetReorderIndexInContainer();

	if (CurrentIndex == INDEX_NONE || TargetIndex == INDEX_NONE || CurrentIndex == TargetIndex)
	{
		return false;
	}

	if (MoveGraph(EdGraph, TargetIndex))
	{
		// Change category to match the one we dragged on to as well
		MovePersistentItemToCategory(OtherAction->GetCategory());
		return true;
	}

	return false;
}

} // UE::SceneState::Graph

#undef LOCTEXT_NAMESPACE
