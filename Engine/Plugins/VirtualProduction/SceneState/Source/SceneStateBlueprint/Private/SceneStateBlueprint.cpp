// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateBlueprint.h"
#include "Blueprint/BlueprintExtension.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "Functions/SceneStateFunction.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Modules/ModuleManager.h"
#include "Nodes/SceneStateMachineStateNode.h"
#include "Nodes/SceneStateMachineTaskNode.h"
#include "PropertyBindingDataView.h"
#include "SceneStateBindingDesc.h"
#include "SceneStateBindingFunction.h"
#include "SceneStateBindingUtils.h"
#include "SceneStateBlueprintBindingUtils.h"
#include "SceneStateBlueprintDelegates.h"
#include "SceneStateBlueprintLog.h"
#include "SceneStateBlueprintUtils.h"
#include "SceneStateGeneratedClass.h"
#include "SceneStateMachineGraph.h"
#include "SceneStateVersion.h"
#include "Tasks/SceneStateTaskMetadata.h"
#include "UObject/UObjectIterator.h"

#define LOCTEXT_NAMESPACE "SceneStateBlueprint"

USceneStateBlueprint::USceneStateBlueprint(const FObjectInitializer& InObjectInitializer)
	: Super(InObjectInitializer)
{
	RootId = FGuid::NewGuid();

	OnRenameVariableReferencesHandle = FBlueprintEditorUtils::OnRenameVariableReferencesEvent.AddUObject(this
		, &USceneStateBlueprint::OnRenameVariableReferences);

	OnGraphParametersChangedHandle = USceneStateMachineGraph::OnParametersChanged().AddUObject(this
		, &USceneStateBlueprint::OnGraphParametersChanged);

	OnStateParametersChangedHandle = USceneStateMachineStateNode::OnParametersChanged().AddUObject(this
		, &USceneStateBlueprint::OnStateParametersChanged);

#if WITH_EDITOR
	BindingCollection.SetBindingsOwner(this);
#endif // WITH_EDITOR
}

UBlueprintExtension* USceneStateBlueprint::FindExtension(TSubclassOf<UBlueprintExtension> InClass) const
{
	const TObjectPtr<UBlueprintExtension>* FoundExtension = GetExtensions().FindByPredicate(
		[InClass](UBlueprintExtension* InExtension)
		{
			return InExtension && InExtension->IsA(InClass);
		});

	if (FoundExtension)
	{
		return *FoundExtension;
	}

	return nullptr;
}

FSceneStateBindingDesc USceneStateBlueprint::CreateRootBinding() const
{
	using namespace UE::SceneState::Graph;
	return CreateBindingDesc(*this);
}

void USceneStateBlueprint::GetBindingStructs(const FGetBindableStructsParams& InParams, TArray<TInstancedStruct<FSceneStateBindingDesc>>& OutBindingDescs) const
{
	using namespace UE::SceneState::Graph;

	auto GatherDefaultBindingDescs =
		[This=this, &InParams, &OutBindingDescs]()
		{
			if (InParams.bIncludeGlobalDescs)
			{
				GetGlobalBindingDescs(*This, InParams.TargetStructId, OutBindingDescs);	
			}
			if (InParams.bIncludeFunctions)
			{
				GetFunctionBindingDescs(*This, InParams.TargetStructId, OutBindingDescs);
			}
		};

	// Add State Machine Bindable Structs if the Target Struct Id matches the State Machine Id
	if (USceneStateMachineGraph* SceneStateMachine = FindStateMachineMatchingId(*this, InParams.TargetStructId))
	{
		GatherDefaultBindingDescs();
		GetStateMachineBindingDescs(*this, *SceneStateMachine, OutBindingDescs);
		return;
	}

	// Add State Bindable Structs if the Target Struct Id matches the state node parameters id
	if (USceneStateMachineStateNode* StateNode = FindStateMatchingId(*this, InParams.TargetStructId))
	{
		GatherDefaultBindingDescs();
		GetStateBindingDescs(StateNode, OutBindingDescs, FString());
		return;
	}

	// Add Transition Bindable Structs if the Target Struct Id matches the Transition Parameters Id
	if (USceneStateMachineTransitionNode* TransitionNode = FindTransitionMatchingId(*this, InParams.TargetStructId))
	{
		GatherDefaultBindingDescs();
		GetTransitionBindingDescs(*this, *TransitionNode, OutBindingDescs);
		return;
	}

	// Add the Task Bindable Structs if the target struct is either:
	//   - The task itself (so the task id must match the target id)
	//   - A struct within a Task Instance that is not stable across sessions, and so require their own ID.
	//     One example of this Instanced Property Bags, as the underlying UPropertyBag (UScriptStruct) is not serialized.
	if (const USceneStateMachineTaskNode* TargetTaskNode = FindTaskNodeContainingId(*this, InParams.TargetStructId))
	{
		GatherDefaultBindingDescs();
		GetTaskBindingDescs(*this, *TargetTaskNode, OutBindingDescs);
		return;
	}

	// Add only default binding descs if the target struct is a binding function
	FGuid OwnerStructId;
	if (const FSceneStateBindingFunction* BindingFunction = FindBindingFunctionMatchingId(*this, InParams.TargetStructId, &OwnerStructId))
	{
		GatherDefaultBindingDescs();

		// Gather the Binding Structs for the Owner
		const FGetBindableStructsParams Params
		{
			.TargetStructId = OwnerStructId,
			.bIncludeGlobalDescs = false,
			.bIncludeFunctions = InParams.bIncludeFunctions,
		};
		GetBindingStructs(Params, OutBindingDescs);
	}
}

bool USceneStateBlueprint::ForEachBindingFunction(TFunctionRef<bool(const FSceneStateBindingFunction&, const FPropertyBindingBinding&)> InFunc) const
{
	for (const FPropertyBindingBinding& Binding : BindingCollection.GetBindings())
	{
		const FSceneStateBindingFunction* BindingFunction = Binding.GetPropertyFunctionNode().GetPtr<const FSceneStateBindingFunction>();
		if (BindingFunction && BindingFunction->IsValid())
		{
			if (!InFunc(*BindingFunction, Binding))
			{
				return false;
			}
		}
	}
	return true;
}

FSceneStateBindingCollection& USceneStateBlueprint::GetBindingCollection()
{
	return BindingCollection;
}

const FSceneStateBindingCollection& USceneStateBlueprint::GetBindingCollection() const
{
	return BindingCollection;
}

bool USceneStateBlueprint::ForEachBindableFunction(TFunctionRef<bool(const FSceneStateBindingDesc&, const UE::SceneState::FBindingFunctionInfo&)> InFunc) const
{
	const FName MD_Category = TEXT("Category");

	const FText Section = LOCTEXT("BindingFunctionsSectionLabel", "Functions");

	// Todo: cache functions for faster iteration
	for (const UScriptStruct* Struct : TObjectRange<UScriptStruct>())
	{
		if (!Struct->IsChildOf<FSceneStateFunction>() || Struct->HasMetaData(UE::SceneState::Editor::MD_Hidden))
		{
			continue;
		}

		const UScriptStruct* InstanceType = TInstancedStruct<FSceneStateFunction>(Struct).Get().GetFunctionDataType();

		FSceneStateBindingDesc BindingDesc;
		BindingDesc.Struct = InstanceType;
		BindingDesc.Name = *Struct->GetDisplayNameText().ToString();
		BindingDesc.ID = FGuid::NewDeterministicGuid(Struct->GetStructPathName().ToString());
		BindingDesc.Section = Section;
		BindingDesc.Category = Struct->GetMetaData(MD_Category);
		BindingDesc.DataHandle = FSceneStateBindingDataHandle(ESceneStateDataType::Function);

		UE::SceneState::FBindingFunctionInfo FunctionInfo;
		FunctionInfo.FunctionTemplate = FConstStructView(Struct);
		FunctionInfo.InstanceTemplate = FConstStructView(InstanceType);

		if (!InFunc(BindingDesc, FunctionInfo))
		{
			return false;
		}
	}

	return true;
}

void USceneStateBlueprint::GetBindableStructs(const FGuid InTargetStructId, TArray<TInstancedStruct<FPropertyBindingBindableStructDescriptor>>& OutStructDescs) const
{
	const FGetBindableStructsParams Params
		{
			.TargetStructId = InTargetStructId,
			.bIncludeFunctions = false,
		};

	TArray<TInstancedStruct<FSceneStateBindingDesc>> BindingDescs;
	GetBindingStructs(Params, BindingDescs);
	OutStructDescs.Append(MoveTemp(BindingDescs));
}

bool USceneStateBlueprint::GetBindableStructByID(const FGuid InStructId, TInstancedStruct<FPropertyBindingBindableStructDescriptor>& OutStructDesc) const
{
	using namespace UE::SceneState::Graph;

	TInstancedStruct<FSceneStateBindingDesc> BindingDesc;
	if (FindBindingDescById(*this, InStructId, BindingDesc))
	{
		OutStructDesc = MoveTemp(BindingDesc);
		return true;
	}
	return false;
}

bool USceneStateBlueprint::GetBindingDataViewByID(const FGuid InStructId, FPropertyBindingDataView& OutDataView) const
{
	using namespace UE::SceneState::Graph;
	return FindDataViewById(*this, InStructId, OutDataView);
}

FPropertyBindingBindingCollection* USceneStateBlueprint::GetEditorPropertyBindings()
{
	return &BindingCollection;
}

const FPropertyBindingBindingCollection* USceneStateBlueprint::GetEditorPropertyBindings() const
{
	return &BindingCollection;
}

bool USceneStateBlueprint::CanCreateParameter(const FGuid InStructId) const
{
	using namespace UE::SceneState::Graph;

	// Only support creating parameters in BP Variables, State machine parameters and state parameters.
	return InStructId == GetRootId() || FindStateMachineMatchingId(*this, InStructId) || FindStateMatchingId(*this, InStructId);
}

void USceneStateBlueprint::CreateParametersForStruct(const FGuid InStructId, TArrayView<UE::PropertyBinding::FPropertyCreationDescriptor> InOutCreationDescs)
{
	using namespace UE::SceneState::Graph;

	if (InStructId == GetRootId())
	{
		CreateBlueprintVariables(this, InOutCreationDescs);
		return;
	}

	if (USceneStateMachineGraph* StateMachineGraph = FindStateMachineMatchingId(*this, InStructId))
	{
		CreateUniquelyNamedPropertiesInPropertyBag(InOutCreationDescs, StateMachineGraph->Parameters);
		return;
	}

	if (USceneStateMachineStateNode* StateNode = FindStateMatchingId(*this, InStructId))
	{
		CreateUniquelyNamedPropertiesInPropertyBag(InOutCreationDescs, StateNode->Parameters);
	}
}

void USceneStateBlueprint::AppendBindablePropertyFunctionStructs(TArray<TInstancedStruct<FPropertyBindingBindableStructDescriptor>>& InOutStructs) const
{
	ForEachBindableFunction(
		[&InOutStructs](const FSceneStateBindingDesc& InBindingDesc, const UE::SceneState::FBindingFunctionInfo& InFunctionInfo)
		{
			InOutStructs.Add(TInstancedStruct<FSceneStateBindingDesc>::Make(InBindingDesc));
			return true; // continue
		});
}

void USceneStateBlueprint::SetSceneStateSchema(TSubclassOf<USceneStateSchema> InSchemaClass)
{
	SceneStateSchemaClass = InSchemaClass;
}

TSubclassOf<USceneStateSchema> USceneStateBlueprint::GetSceneStateSchemaClass() const
{
	return SceneStateSchemaClass;
}

void USceneStateBlueprint::SetObjectBeingDebugged(UObject* InNewObject)
{
	using namespace UE::SceneState;

	Super::SetObjectBeingDebugged(InNewObject);

	Graph::FBlueprintDebugObjectChange Change;
	Change.Blueprint = this;
	Change.DebugObject = CurrentObjectBeingDebugged.Get();

	Graph::OnBlueprintDebugObjectChanged.Broadcast(Change);
}

UClass* USceneStateBlueprint::GetBlueprintClass() const
{
	return USceneStateGeneratedClass::StaticClass();
}

void USceneStateBlueprint::GetReparentingRules(TSet<const UClass*>& OutAllowedChildrenOfClasses, TSet<const UClass*>& OutDisallowedChildrenOfClasses) const
{
	OutAllowedChildrenOfClasses.Add(USceneStateGeneratedClass::StaticClass());
}

bool USceneStateBlueprint::SupportedByDefaultBlueprintFactory() const
{
	return false;
}

void USceneStateBlueprint::LoadModulesRequiredForCompilation()
{
	// Load the module holding the scene state blueprint compiler
	// todo: consider moving the compiler to its own module to load that instead of the entire editor module
	constexpr const TCHAR* SceneStateBlueprintEditorModule = TEXT("SceneStateBlueprintEditor");
	FModuleManager::Get().LoadModule(SceneStateBlueprintEditorModule);
}

bool USceneStateBlueprint::IsValidForBytecodeOnlyRecompile() const
{
	return false;
}

void USceneStateBlueprint::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(UE::SceneState::FVersion::Guid);
}

void USceneStateBlueprint::PostLoad()
{
	Super::PostLoad();

	// Default previous versions to the motion design schema.
	// If motion design isn't enabled, fall back to the gameplay schema.
	if (GetLinkerCustomVersion(UE::SceneState::FVersion::Guid) < UE::SceneState::FVersion::Schema)
	{
		if (UClass* const MotionDesignSchemaClass = FindObject<UClass>(nullptr, TEXT("/Script/AvalancheSceneState.AvaSceneStateSchema")))
		{
			SetSceneStateSchema(MotionDesignSchemaClass);
		}
		else if (UClass* const GameplaySchemaClass = FindObject<UClass>(nullptr, TEXT("/Script/SceneStateGameplay.SceneStateGameplaySchema")))
		{
			SetSceneStateSchema(GameplaySchemaClass);
		}
	}
}

void USceneStateBlueprint::BeginDestroy()
{
	Super::BeginDestroy();

	FBlueprintEditorUtils::OnRenameVariableReferencesEvent.Remove(OnRenameVariableReferencesHandle);
	OnRenameVariableReferencesHandle.Reset();

	USceneStateMachineGraph::OnParametersChanged().Remove(OnGraphParametersChangedHandle);
	OnGraphParametersChangedHandle.Reset();

	USceneStateMachineStateNode::OnParametersChanged().Remove(OnStateParametersChangedHandle);
	OnStateParametersChangedHandle.Reset();
}

void USceneStateBlueprint::OnRenameVariableReferences(UBlueprint* InBlueprint, UClass* InVariableClass, const FName& InOldVariableName, const FName& InNewVariableName)
{
	if (InBlueprint != this)
	{
		return;
	}

	// Note: no need to call Blueprint Modify here. It's already been called in FBlueprintEditorUtils::RenameMemberVariable
	BindingCollection.ForEachMutableBinding(
		[This=this, InOldVariableName, InNewVariableName](FPropertyBindingBinding& InBinding)
		{
			This->RenameVariableReferenceInPath(InBinding.GetMutableSourcePath(), InOldVariableName, InNewVariableName);
			This->RenameVariableReferenceInPath(InBinding.GetMutableTargetPath(), InOldVariableName, InNewVariableName);
		});
}

void USceneStateBlueprint::RenameVariableReferenceInPath(FPropertyBindingPath& InPath, FName InOldVariableName, FName InNewVariableName)
{
	// Only consider fixing paths that are set to this blueprint class (as the rename here is for blueprint variables only)
	if (InPath.GetStructID() != GetRootId())
	{
		return;
	}

	TArrayView<FPropertyBindingPathSegment> Segments = InPath.GetMutableSegments();
	if (Segments.IsEmpty())
	{
		return;
	}

	// Only need to consider the first segment of the path (i.e. the segment containing the blueprint variable)
	FPropertyBindingPathSegment& Segment = Segments[0];

	if (Segment.GetName() == InOldVariableName)
	{
		const FString OldPath = InPath.ToString();

		Segment.SetName(InNewVariableName);

		UE_LOGF(LogSceneStateBlueprint, Log, "Renamed blueprint variable binding segment '%ls' to '%ls'. (OldPath: %ls ---> New Path: %ls)"
			, *InOldVariableName.ToString()
			, *InNewVariableName.ToString()
			, *OldPath
			, *InPath.ToString());
	}
}

void USceneStateBlueprint::OnGraphParametersChanged(USceneStateMachineGraph* InGraph)
{
	if (!InGraph)
	{
		return;
	}

	BindingCollection.ForEachMutableBinding(
		[This=this, InGraph](FPropertyBindingBinding& InBinding)
		{
			This->UpdateParametersBindings(InBinding.GetMutableSourcePath(), InGraph->ParametersId, InGraph->Parameters);
			This->UpdateParametersBindings(InBinding.GetMutableTargetPath(), InGraph->ParametersId, InGraph->Parameters);
		});
}

void USceneStateBlueprint::OnStateParametersChanged(USceneStateMachineStateNode* InStateNode)
{
	if (!InStateNode)
	{
		return;
	}

	BindingCollection.ForEachMutableBinding(
		[This=this, InStateNode](FPropertyBindingBinding& InBinding)
		{
			This->UpdateParametersBindings(InBinding.GetMutableSourcePath(), InStateNode->ParametersId, InStateNode->Parameters);
			This->UpdateParametersBindings(InBinding.GetMutableTargetPath(), InStateNode->ParametersId, InStateNode->Parameters);
		});
}

void USceneStateBlueprint::UpdateParametersBindings(FPropertyBindingPath& InPath, const FGuid& InParametersId, const FInstancedPropertyBag& InParameters)
{
	// Only consider fixing paths that are set to the graph parameters
	if (InPath.GetStructID() != InParametersId)
	{
		return;
	}

	TArrayView<FPropertyBindingPathSegment> Segments = InPath.GetMutableSegments();
	if (Segments.IsEmpty())
	{
		return;
	}

	// Only need to consider the first segment of the path (i.e. the segment that might've been renamed)
	FPropertyBindingPathSegment& Segment = Segments[0];

	if (const FPropertyBagPropertyDesc* Property = InParameters.FindPropertyDescByID(Segment.GetPropertyGuid()))
	{
		const FName SegmentName = Segment.GetName();

		if (SegmentName != Property->Name)
		{
			if (GUndo)
			{
				Modify();
			}

			const FString OldPath = InPath.ToString();
			Segment.SetName(Property->Name);

			UE_LOGF(LogSceneStateBlueprint, Log, "Renamed parameter variable binding segment '%ls' to '%ls'. (OldPath: %ls ---> New Path: %ls)"
				, *SegmentName.ToString()
				, *Property->Name.ToString()
				, *OldPath
				, *InPath.ToString());
		}
	}
}

#undef LOCTEXT_NAMESPACE
