// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateBlueprintBindingUtils.h"
#include "Nodes/SceneStateMachineStateNode.h"
#include "Nodes/SceneStateMachineTaskNode.h"
#include "Nodes/SceneStateMachineTransitionNode.h"
#include "PropertyBindingDataView.h"
#include "SceneStateBindingDesc.h"
#include "SceneStateBindingFunction.h"
#include "SceneStateBlueprint.h"
#include "SceneStateBlueprint.h"
#include "SceneStateBlueprintUtils.h"
#include "SceneStateEventSchema.h"
#include "SceneStateMachineGraph.h"
#include "SceneStateMachineGraphSchema.h"
#include "StructUtils/UserDefinedStruct.h"
#include "Tasks/SceneStateBlueprintableTask.h"

#define LOCTEXT_NAMESPACE "SceneStateBlueprintBindingUtils"

namespace UE::SceneState::Graph
{

namespace Private
{
	
FText GetParametersSectionLabel()
{
	return LOCTEXT("ParametersSectionLabel", "Parameters");
}

} // UE::SceneState::Graph::Private

USceneStateMachineStateNode* FindStateMatchingId(const USceneStateBlueprint& InBlueprint, const FGuid& InStructId)
{
	USceneStateMachineStateNode* FoundNode = nullptr;

	VisitNodes(InBlueprint.StateMachineGraphs,
		[&FoundNode, &InStructId](USceneStateMachineNode* InNode, EIterationResult& IterationResult)
		{
			USceneStateMachineStateNode* StateNode = Cast<USceneStateMachineStateNode>(InNode);
			if (StateNode && StateNode->ParametersId == InStructId)
			{
				FoundNode = StateNode;
				IterationResult = EIterationResult::Break;
			}
		});

	return FoundNode;
}

void GetStateBindingDescs(const USceneStateMachineStateNode* InStateNode, TArray<TInstancedStruct<FSceneStateBindingDesc>>& OutBindingDescs, FString InCategory)
{
	if (!InStateNode)
	{
		return;
	}

	const FText Section = LOCTEXT("EventsSectionLabel", "Events");

	// Add the State Event Handlers as Bindable Sources
	for (const FSceneStateEventHandler& EventHandler : InStateNode->GetEventHandlers())
	{
		const USceneStateEventSchemaObject* EventSchema = EventHandler.GetEventSchemaHandle().GetEventSchema();
		if (EventSchema && EventSchema->Struct)
		{
			FSceneStateBindingDesc BindingDesc;
			BindingDesc.ID = EventHandler.GetHandlerId();
			BindingDesc.Name = EventSchema->Name;
			BindingDesc.Struct = EventSchema->Struct;
			BindingDesc.Category = InCategory;
			BindingDesc.Section = Section;

			OutBindingDescs.Emplace(TInstancedStruct<FSceneStateBindingDesc>::Make(MoveTemp(BindingDesc)));
		}
	}

	USceneStateMachineGraph* StateMachineGraph = Cast<USceneStateMachineGraph>(InStateNode->GetGraph());
	if (!StateMachineGraph)
	{
		return;
	}

	// Add the owning state machine binding desc
	OutBindingDescs.Emplace(TInstancedStruct<FSceneStateBindingDesc>::Make(CreateBindingDesc(*StateMachineGraph)));

	// Recurse up to add the Event Handlers of the parent state node (if any)
	if (InCategory.IsEmpty())
	{
		InCategory += TEXT("|");
	}
	InCategory += InStateNode->GetNodeName().ToString();

	if (USceneStateMachineStateNode* ParentStateNode = StateMachineGraph->GetParentStateNode())
	{
		OutBindingDescs.Emplace(TInstancedStruct<FSceneStateBindingDesc>::Make(CreateBindingDesc(*ParentStateNode)));
		GetStateBindingDescs(ParentStateNode, OutBindingDescs, InCategory);
	}
}

FSceneStateBindingDesc CreateBindingDesc(const USceneStateBlueprint& InBlueprint)
{
	FSceneStateBindingDesc BindingDesc;
	BindingDesc.Name = TEXT("Variables");
	BindingDesc.ID = InBlueprint.GetRootId();
	BindingDesc.Struct = InBlueprint.GeneratedClass;
	BindingDesc.Section = LOCTEXT("BlueprintSectionLabel", "Blueprint");
	return BindingDesc;
}

FSceneStateBindingDesc CreateBindingDesc(const USceneStateMachineGraph& InGraph)
{
	FSceneStateBindingDesc BindingDesc;
	BindingDesc.Name = TEXT("State Machine Parameters");
	BindingDesc.ID = InGraph.ParametersId;
	BindingDesc.Struct = InGraph.Parameters.GetPropertyBagStruct();
	BindingDesc.Section = Private::GetParametersSectionLabel();
	return BindingDesc;
}

FSceneStateBindingDesc CreateBindingDesc(const USceneStateMachineStateNode& InStateNode)
{
	FSceneStateBindingDesc BindingDesc;
	BindingDesc.Name = TEXT("State Parameters");
	BindingDesc.ID = InStateNode.ParametersId;
	BindingDesc.Struct = InStateNode.Parameters.GetPropertyBagStruct();
	BindingDesc.Section = Private::GetParametersSectionLabel();
	return BindingDesc;
}

FSceneStateBindingDesc CreateBindingDesc(const USceneStateMachineTransitionNode& InTransitionNode)
{
	FSceneStateBindingDesc BindingDesc;
	BindingDesc.Name = TEXT("Transition Parameters");
	BindingDesc.ID = InTransitionNode.GetParametersId();
	BindingDesc.Struct = InTransitionNode.GetParameters().GetPropertyBagStruct();
	BindingDesc.Section = Private::GetParametersSectionLabel();
	return BindingDesc;
}

FSceneStateBindingDesc CreateBindingDesc(const FSceneStateBindingFunction& InBindingFunctionChecked)
{
	FSceneStateBindingDesc BindingDesc;
	BindingDesc.Struct = InBindingFunctionChecked.FunctionInstance.GetScriptStruct();
	BindingDesc.Name = *InBindingFunctionChecked.Function.GetScriptStruct()->GetDisplayNameText().ToString();
	BindingDesc.ID = InBindingFunctionChecked.FunctionId;
	return BindingDesc;
}

void GetGlobalBindingDescs(const USceneStateBlueprint& InBlueprint, const FGuid& InTargetStructId, TArray<TInstancedStruct<FSceneStateBindingDesc>>& OutBindingDescs)
{
	OutBindingDescs.Add(TInstancedStruct<FSceneStateBindingDesc>::Make(CreateBindingDesc(InBlueprint)));
}

void GetFunctionBindingDescs(const USceneStateBlueprint& InBlueprint, const FGuid& InTargetStructId, TArray<TInstancedStruct<FSceneStateBindingDesc>>& OutBindingDescs)
{
	for (const FPropertyBindingBinding& Binding : InBlueprint.GetBindingCollection().GetBindings())
	{
		if (Binding.GetTargetPath().GetStructID() != InTargetStructId)
		{
			continue;
		}

		const FSceneStateBindingFunction* BindingFunction = Binding.GetPropertyFunctionNode().GetPtr<const FSceneStateBindingFunction>();
		if (BindingFunction && BindingFunction->IsValid())
		{
			OutBindingDescs.Add(TInstancedStruct<FSceneStateBindingDesc>::Make(CreateBindingDesc(*BindingFunction)));
		}
	}
}

USceneStateMachineGraph* FindStateMachineMatchingId(const USceneStateBlueprint& InBlueprint, const FGuid& InStructId)
{
	USceneStateMachineGraph* FoundGraph = nullptr;

	VisitGraphs(InBlueprint.StateMachineGraphs,
		[&FoundGraph, &InStructId](USceneStateMachineGraph* InGraph, EIterationResult& IterationResult)
		{
			if (InGraph->ParametersId == InStructId)
			{
				FoundGraph = InGraph;
				IterationResult = EIterationResult::Break;
			}
		});

	return FoundGraph;
}

void GetStateMachineBindingDescs(const USceneStateBlueprint& InBlueprint, const USceneStateMachineGraph& InGraph, TArray<TInstancedStruct<FSceneStateBindingDesc>>& OutBindingDescs)
{
	// If this State Machine is under a parent state node, add the event handlers of the state (recursively)
	USceneStateMachineStateNode* ParentStateNode = InGraph.GetTypedOuter<USceneStateMachineStateNode>(); 

	// Add the state binding desc
	if (ParentStateNode)
	{
		OutBindingDescs.Emplace(TInstancedStruct<FSceneStateBindingDesc>::Make(CreateBindingDesc(*ParentStateNode)));
		GetStateBindingDescs(ParentStateNode, OutBindingDescs, FString());
	}
}

USceneStateMachineTaskNode* FindTaskNodeContainingId(const USceneStateBlueprint& InBlueprint, const FGuid& InStructId)
{
	USceneStateMachineTaskNode* FoundNode = nullptr;

	VisitNodes(InBlueprint.StateMachineGraphs,
		[&FoundNode, &InStructId](USceneStateMachineNode* InNode, EIterationResult& IterationResult)
		{
			USceneStateMachineTaskNode* TaskNode = Cast<USceneStateMachineTaskNode>(InNode);
			FStructView DataView;
			if (TaskNode && TaskNode->FindDataViewById(InStructId, DataView))
			{
				FoundNode = TaskNode;
				IterationResult = EIterationResult::Break;
			}
		});

	return FoundNode;
}

void GetTaskBindingDescs(const USceneStateBlueprint& InBlueprint, const USceneStateMachineTaskNode& InTaskNode, TArray<TInstancedStruct<FSceneStateBindingDesc>>& OutBindingDescs)
{
	// Gather the Binding Descs starting from the directly connected State node.
	if (USceneStateMachineStateNode* const StateNode = USceneStateMachineGraphSchema::FindConnectedStateNode(&InTaskNode))
	{
		OutBindingDescs.Emplace(TInstancedStruct<FSceneStateBindingDesc>::Make(CreateBindingDesc(*StateNode)));
		GetStateBindingDescs(StateNode, OutBindingDescs, FString());
	}
	// If the task isn't connected to a State, gather Binding Descs starting from the outer State Machine Graph
	else if (USceneStateMachineGraph* StateMachineGraph = Cast<USceneStateMachineGraph>(InTaskNode.GetGraph()))
	{
		// Add the outer state machine binding desc
		OutBindingDescs.Emplace(TInstancedStruct<FSceneStateBindingDesc>::Make(CreateBindingDesc(*StateMachineGraph)));

		// If the Parent State Machine is under a parent outer state node, add the binding descs of that state (recursively)
		USceneStateMachineStateNode* ParentStateNode = StateMachineGraph->GetTypedOuter<USceneStateMachineStateNode>();
		if (ParentStateNode)
		{
			OutBindingDescs.Emplace(TInstancedStruct<FSceneStateBindingDesc>::Make(CreateBindingDesc(*ParentStateNode)));
			GetStateBindingDescs(ParentStateNode, OutBindingDescs, FString());
		}
	}
}

const FSceneStateBindingFunction* FindBindingFunctionMatchingId(const USceneStateBlueprint& InBlueprint, const FGuid& InStructId, FGuid* OutOwnerStructId)
{
	const FSceneStateBindingFunction* Result = nullptr;

	InBlueprint.ForEachBindingFunction(
		[&InStructId, &Result, OutOwnerStructId](const FSceneStateBindingFunction& InBindingFunction, const FPropertyBindingBinding& InBinding)
		{
			if (InBindingFunction.FunctionId == InStructId)
			{
				Result = &InBindingFunction;
				if (OutOwnerStructId)
				{
					*OutOwnerStructId = InBinding.GetTargetPath().GetStructID();
				}
				return false; // break
			}
			return true; // continue
		});

	return Result;
}

USceneStateMachineTransitionNode* FindTransitionMatchingId(const USceneStateBlueprint& InBlueprint, const FGuid& InStructId)
{
	USceneStateMachineTransitionNode* FoundNode = nullptr;

	VisitNodes(InBlueprint.StateMachineGraphs,
		[&FoundNode, &InStructId](USceneStateMachineNode* InNode, EIterationResult& IterationResult)
		{
			USceneStateMachineTransitionNode* TransitionNode = Cast<USceneStateMachineTransitionNode>(InNode);
			if (TransitionNode && TransitionNode->GetParametersId() == InStructId)
			{
				FoundNode = TransitionNode;
				IterationResult = EIterationResult::Break;
			}
		});

	return FoundNode;
}

void GetTransitionBindingDescs(const USceneStateBlueprint& InBlueprint, const USceneStateMachineTransitionNode& InTransitionNode, TArray<TInstancedStruct<FSceneStateBindingDesc>>& OutBindingDescs)
{
	// Gather the Binding Descs starting from the directly outer state machine.
	if (USceneStateMachineGraph* StateMachineGraph = Cast<USceneStateMachineGraph>(InTransitionNode.GetGraph()))
	{
		// Add the outer state machine binding desc
		OutBindingDescs.Emplace(TInstancedStruct<FSceneStateBindingDesc>::Make(CreateBindingDesc(*StateMachineGraph)));

		// If the Parent State Machine is under a parent outer state node, add the binding descs of that state (recursively)
		USceneStateMachineStateNode* ParentStateNode = StateMachineGraph->GetTypedOuter<USceneStateMachineStateNode>();
		if (ParentStateNode)
		{
			OutBindingDescs.Emplace(TInstancedStruct<FSceneStateBindingDesc>::Make(CreateBindingDesc(*ParentStateNode)));
			GetStateBindingDescs(ParentStateNode, OutBindingDescs, FString());
		}
	}
}

bool FindBindingDescById(const USceneStateBlueprint& InBlueprint, const FGuid& InStructId, TInstancedStruct<FSceneStateBindingDesc>& OutBindingDesc)
{
	if (InStructId == InBlueprint.GetRootId())
	{
		OutBindingDesc = TInstancedStruct<FSceneStateBindingDesc>::Make(CreateBindingDesc(InBlueprint));
		return true;
	}

	enum class EResult : uint8
	{
		NotFound,
		FoundInvalid,
		FoundValid,
	};
	EResult Result = EResult::NotFound;

	VisitGraphs(InBlueprint.StateMachineGraphs,
		[&InStructId, &OutBindingDesc, &Result](USceneStateMachineGraph* InGraph, EIterationResult& IterationResult)
		{
			if (InGraph->ParametersId == InStructId)
			{
				Result = EResult::FoundInvalid;
				IterationResult = EIterationResult::Break;

				FSceneStateBindingDesc BindingDesc = CreateBindingDesc(*InGraph);
				if (BindingDesc.Struct)
				{
					OutBindingDesc = TInstancedStruct<FSceneStateBindingDesc>::Make(MoveTemp(BindingDesc));
					Result = EResult::FoundValid;
				}
			}
		});

	// Early exit if the struct desc has already been found (even if invalid)
	if (Result != EResult::NotFound)
	{
		return Result == EResult::FoundValid;
	}

	VisitNodes(InBlueprint.StateMachineGraphs,
		[&InStructId, &OutBindingDesc, &Result](USceneStateMachineNode* InNode, EIterationResult& IterationResult)
		{
			if (USceneStateMachineTransitionNode* TransitionNode = Cast<USceneStateMachineTransitionNode>(InNode))
			{
				if (TransitionNode->GetParametersId() == InStructId)
				{
					// Found but not yet valid.
					Result = EResult::FoundInvalid;
					IterationResult = EIterationResult::Break;

					FSceneStateBindingDesc BindingDesc = CreateBindingDesc(*TransitionNode);
					if (BindingDesc.Struct)
					{
						// Found valid binding
						OutBindingDesc = TInstancedStruct<FSceneStateBindingDesc>::Make(MoveTemp(BindingDesc));
						Result = EResult::FoundValid;
					}
				}
			}
			else if (USceneStateMachineStateNode* StateNode = Cast<USceneStateMachineStateNode>(InNode))
			{
				if (StateNode->ParametersId == InStructId)
				{
					// Found but not yet valid.
					Result = EResult::FoundInvalid;
					IterationResult = EIterationResult::Break;

					FSceneStateBindingDesc BindingDesc = CreateBindingDesc(*StateNode);
					if (BindingDesc.Struct)
					{
						// Found valid binding
						OutBindingDesc = TInstancedStruct<FSceneStateBindingDesc>::Make(MoveTemp(BindingDesc));
						Result = EResult::FoundValid;
					}
				}
				else
				{
					for (const FSceneStateEventHandler& EventHandler : StateNode->GetEventHandlers())
					{
						if (EventHandler.GetHandlerId() != InStructId)
						{
							continue;
						}

						const USceneStateEventSchemaObject* EventSchema = EventHandler.GetEventSchemaHandle().GetEventSchema();
						if (EventSchema && EventSchema->Struct)
						{
							OutBindingDesc = TInstancedStruct<FSceneStateBindingDesc>::Make();

							FSceneStateBindingDesc& StructDesc = OutBindingDesc.GetMutable();
							StructDesc.ID = EventHandler.GetHandlerId();
							StructDesc.Name = EventSchema->Name;
							StructDesc.Struct = EventSchema->Struct;

							IterationResult = EIterationResult::Break;
							Result = EResult::FoundValid;
							return;
						}
					}
				}
			}
		});

	// Early exit if the struct desc has already been found (even if invalid)
	if (Result != EResult::NotFound)
	{
		return Result == EResult::FoundValid;
	}

	if (const FSceneStateBindingFunction* BindingFunction = FindBindingFunctionMatchingId(InBlueprint, InStructId))
	{
		OutBindingDesc = TInstancedStruct<FSceneStateBindingDesc>::Make(CreateBindingDesc(*BindingFunction));
		return true;
	}

	return false;
}

bool FindDataViewById(const USceneStateBlueprint& InBlueprint, const FGuid& InStructId, FPropertyBindingDataView& OutDataView)
{
	if (InStructId == InBlueprint.GetRootId())
	{
		OutDataView = FPropertyBindingDataView(InBlueprint.GeneratedClass, InBlueprint.GeneratedClass->GetDefaultObject());
		return true;
	}

	bool bDataViewFound = false;

	VisitGraphs(InBlueprint.StateMachineGraphs,
		[&InStructId, &OutDataView, &bDataViewFound](USceneStateMachineGraph* InGraph, EIterationResult& IterationResult)
		{
			if (InGraph->ParametersId == InStructId)
			{
				OutDataView = InGraph->Parameters.GetMutableValue();
				bDataViewFound = true;
				IterationResult = EIterationResult::Break;
			}
		});

	// Early exit if data view has already been found
	if (bDataViewFound)
	{
		return true;
	}

	VisitNodes(InBlueprint.StateMachineGraphs,
		[&InStructId, &OutDataView, &bDataViewFound](USceneStateMachineNode* InNode, EIterationResult& IterationResult)
		{
			USceneStateMachineTaskNode* TaskNode = Cast<USceneStateMachineTaskNode>(InNode);
			FStructView TaskInstanceDataView;
			if (TaskNode && TaskNode->FindDataViewById(InStructId, TaskInstanceDataView))
			{
				OutDataView = TaskInstanceDataView;
				bDataViewFound = true;
				IterationResult = EIterationResult::Break;
				return; // stop execution
			}

			if (USceneStateMachineTransitionNode* TransitionNode = Cast<USceneStateMachineTransitionNode>(InNode))
			{
				if (TransitionNode->GetParametersId() == InStructId)
				{
					OutDataView = TransitionNode->GetParametersMutable().GetMutableValue();
					bDataViewFound = true;
					IterationResult = EIterationResult::Break;
				}
			}
			// When finding the Data View for Blueprints, search all States for the Parameters or Event Handler matching the Struct Id
			else if (USceneStateMachineStateNode* StateNode = Cast<USceneStateMachineStateNode>(InNode))
			{
				if (StateNode->ParametersId == InStructId)
				{
					OutDataView = StateNode->Parameters.GetMutableValue();
					bDataViewFound = true;
					IterationResult = EIterationResult::Break;
				}
				else
				{
					for (const FSceneStateEventHandler& EventHandler : StateNode->GetEventHandlers())
					{
						if (EventHandler.GetHandlerId() != InStructId)
						{
							continue;
						}

						OutDataView = EventHandler.GetEventSchemaHandle().GetDefaultDataView();
						bDataViewFound = true;
						IterationResult = EIterationResult::Break;
						return; // stop execution
					}
				}
			}
		});

	// Early exit if data view has already been found
	if (bDataViewFound)
	{
		return true;
	}

	if (const FSceneStateBindingFunction* BindingFunction = FindBindingFunctionMatchingId(InBlueprint, InStructId))
	{
		OutDataView = BindingFunction->FunctionInstance;
		return true;
	}

	return false;
}

} // namespace UE::SceneState::Graph

#undef LOCTEXT_NAMESPACE
