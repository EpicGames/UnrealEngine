// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Templates/FunctionFwd.h"

class USceneStateBlueprint;
class USceneStateMachineGraph;
class USceneStateMachineNode;
class USceneStateMachineStateNode;
class USceneStateMachineTaskNode;
class USceneStateMachineTransitionNode;
struct FGuid;
struct FPropertyBindingDataView;
struct FSceneStateBindingDesc;
struct FSceneStateBindingFunction;
template<typename BaseStructT> struct TInstancedStruct;

namespace UE::SceneState::Graph
{
	/**
	 * Finds the state node with id that matches the given struct id.
	 * @param InBlueprint the blueprint to look into
	 * @param InStructId the struct id to look for
	 * @return the found state node matching the given struct id, or null if not found
	 */
	USceneStateMachineStateNode* FindStateMatchingId(const USceneStateBlueprint& InBlueprint, const FGuid& InStructId);

	/**
	 * Gathers the Binding Descs for a State Node
	 * @param InStateNode the state to find binding descs for
	 * @param OutBindingDescs the returned binding descs
	 * @param InCategory the category to append to the binding descs
	 */
	void GetStateBindingDescs(const USceneStateMachineStateNode* InStateNode, TArray<TInstancedStruct<FSceneStateBindingDesc>>& OutBindingDescs, FString InCategory);

	/** Creates the Binding Struct Desc for the Variables in the given Blueprint */
	FSceneStateBindingDesc CreateBindingDesc(const USceneStateBlueprint& InBlueprint);

	/** Creates the Binding Struct Desc for the Parameters in the given State Machine Graph */
	FSceneStateBindingDesc CreateBindingDesc(const USceneStateMachineGraph& InGraph);

	/** Creates the Binding Struct Desc for the Parameters in the given State Node */
	FSceneStateBindingDesc CreateBindingDesc(const USceneStateMachineStateNode& InStateNode);

	/** Creates the Binding Struct Desc for the Parameters in the given Transition Node */
	FSceneStateBindingDesc CreateBindingDesc(const USceneStateMachineTransitionNode& InTransitionNode);

	/** Creates the Binding Struct Desc for the given binding function */
	FSceneStateBindingDesc CreateBindingDesc(const FSceneStateBindingFunction& InBindingFunctionChecked);

	/**
	 * Gathers all the global binding descs. Here 'global' means within the scope of the blueprint.
	 * @param InBlueprint the blueprint
	 * @param InTargetStructId the struct to gather the binding descs for
	 * @param OutBindingDescs all the binding descs that can be bound for the target struct
	 */
	void GetGlobalBindingDescs(const USceneStateBlueprint& InBlueprint, const FGuid& InTargetStructId, TArray<TInstancedStruct<FSceneStateBindingDesc>>& OutBindingDescs);

	/**
	 * Gathers all the binding descs for functions bound to the given struct id.
	 * @param InBlueprint the blueprint
	 * @param InTargetStructId the struct to gather the binding descs for
	 * @param OutBindingDescs all the binding descs that are bound to the target struct
	 */
	void GetFunctionBindingDescs(const USceneStateBlueprint& InBlueprint, const FGuid& InTargetStructId, TArray<TInstancedStruct<FSceneStateBindingDesc>>& OutBindingDescs);

	/**
	 * Finds the state machine graph with id that matches the given struct id.
	 * @param InBlueprint the blueprint to look into
	 * @param InStructId the struct id to look for
	 * @return the found state machine graph matching the given struct id
	 */
	USceneStateMachineGraph* FindStateMachineMatchingId(const USceneStateBlueprint& InBlueprint, const FGuid& InStructId);

	/**
	 * Gathers all the binding descs for a given state machine graph
	 * @param InBlueprint the blueprint containing the graph
	 * @param InGraph the state machine graph
	 * @param OutBindingDescs all the binding descs that can be bound for the state machine graph
	 */
	void GetStateMachineBindingDescs(const USceneStateBlueprint& InBlueprint, const USceneStateMachineGraph& InGraph, TArray<TInstancedStruct<FSceneStateBindingDesc>>& OutBindingDescs);

	/**
	 * Finds the Task node with Task that contains a given struct id.
	 * @param InBlueprint the blueprint to look into
	 * @param InStructId the struct id to look for
	 * @return the found node containing the given struct id
	 */
	USceneStateMachineTaskNode* FindTaskNodeContainingId(const USceneStateBlueprint& InBlueprint, const FGuid& InStructId);

	/**
	 * Gathers all the binding descs for a given Task Node
	 * @param InBlueprint the blueprint containing the task node
	 * @param InTaskNode the task node
	 * @param OutBindingDescs all the binding descs that can be bound for the task node
	 */
	void GetTaskBindingDescs(const USceneStateBlueprint& InBlueprint, const USceneStateMachineTaskNode& InTaskNode, TArray<TInstancedStruct<FSceneStateBindingDesc>>& OutBindingDescs);

	/**
	 * Finds the binding function that matches the given struct id.
	 * @param InBlueprint the blueprint to look into
	 * @param InStructId the struct id to look for
	 * @param OutOwnerStructId (optional) the found binding function's owner struct id
	 * @return the found binding function matching the given struct id
	 */
	const FSceneStateBindingFunction* FindBindingFunctionMatchingId(const USceneStateBlueprint& InBlueprint, const FGuid& InStructId, FGuid* OutOwnerStructId = nullptr);

	/**
	 * Finds the transition node with id that matches the given struct id.
	 * @param InBlueprint the blueprint to look into
	 * @param InStructId the struct id to look for
	 * @return the found transition node matching the given struct id
	 */
	USceneStateMachineTransitionNode* FindTransitionMatchingId(const USceneStateBlueprint& InBlueprint, const FGuid& InStructId);

	/**
	 * Gathers all the binding descs for a given transition node
	 * @param InBlueprint the blueprint containing the transition node
	 * @param InTransitionNode the transition node
	 * @param OutBindingDescs all the binding descs that can be bound for the task node
	 */
	void GetTransitionBindingDescs(const USceneStateBlueprint& InBlueprint, const USceneStateMachineTransitionNode& InTransitionNode, TArray<TInstancedStruct<FSceneStateBindingDesc>>& OutBindingDescs);

	/**
	 * Finds the binding desc matching the given id
	 * @param InBlueprint the blueprint to look in
	 * @param InStructId the id to match
	 * @param OutBindingDesc the found binding desc
	 * @return true if the binding desc was found, false otherwise
	 */
	bool FindBindingDescById(const USceneStateBlueprint& InBlueprint, const FGuid& InStructId, TInstancedStruct<FSceneStateBindingDesc>& OutBindingDesc);

	/**
	 * Finds the data view matching the given struct id
	 * @param InBlueprint the blueprint to look in
	 * @param InStructId the id to match
	 * @param OutDataView the data view of the struct instance if found
	 * @return true if the struct was found, false otherwise
	 */
	bool FindDataViewById(const USceneStateBlueprint& InBlueprint, const FGuid& InStructId, FPropertyBindingDataView& OutDataView);

} // UE::SceneState::Graph
