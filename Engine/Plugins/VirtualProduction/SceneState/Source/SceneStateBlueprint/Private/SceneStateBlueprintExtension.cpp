// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateBlueprintExtension.h"
#include "SceneStateBlueprint.h"

void USceneStateBlueprintExtension::GetAllGraphs(TArray<UEdGraph*>& OutGraphs) const
{
	if (USceneStateBlueprint* Blueprint = Cast<USceneStateBlueprint>(GetOuter()))
	{
		OutGraphs.Append(Blueprint->StateMachineGraphs);
	}
}
