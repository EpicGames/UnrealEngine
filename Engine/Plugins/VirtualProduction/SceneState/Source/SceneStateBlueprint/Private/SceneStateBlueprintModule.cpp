// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "SceneStateBlueprintLog.h"

DEFINE_LOG_CATEGORY(LogSceneStateBlueprint);

IMPLEMENT_MODULE(FDefaultModuleImpl, SceneStateBlueprint)
