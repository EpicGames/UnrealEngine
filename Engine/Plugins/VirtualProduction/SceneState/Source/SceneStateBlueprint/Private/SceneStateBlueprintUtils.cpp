// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateBlueprintUtils.h"
#include "Containers/ArrayView.h"
#include "EdGraph/EdGraph.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Nodes/SceneStateMachineNode.h"
#include "PropertyBagDetails.h"
#include "SceneStateBlueprint.h"
#include "SceneStateMachineGraph.h"
#include "SceneStateMachineGraphUtils.h"
#include "Templates/Function.h"

namespace UE::SceneState::Graph
{

namespace Private
{

void VisitNodes(TConstArrayView<UEdGraph*> InGraphs, TFunctionRef<void(USceneStateMachineNode*, EIterationResult&)> InFunc, EIterationResult& IterationResult)
{
	for (UEdGraph* Graph : InGraphs)
	{
		if (!Graph)
		{
			continue;
		}

		for (UEdGraphNode* Node : Graph->Nodes)
		{
			if (USceneStateMachineNode* StateMachineNode = Cast<USceneStateMachineNode>(Node))
			{
				IterationResult = EIterationResult::Continue;
				InFunc(StateMachineNode, IterationResult);

				if (IterationResult == EIterationResult::Break)
				{
					return;
				}

				TArray<UEdGraph*> BoundGraphs = StateMachineNode->GetSubGraphs();
				VisitNodes(BoundGraphs, InFunc, IterationResult);

				if (IterationResult == EIterationResult::Break)
				{
					return;
				}
			}
		}
	}
}

void VisitGraphs(TConstArrayView<UEdGraph*> InGraphs, TFunctionRef<void(USceneStateMachineGraph*, EIterationResult&)> InFunc, EIterationResult& IterationResult)
{
	for (UEdGraph* Graph : InGraphs)
	{
		USceneStateMachineGraph* StateMachineGraph = Cast<USceneStateMachineGraph>(Graph);
		if (!StateMachineGraph)
		{
			continue;
		}

		IterationResult = EIterationResult::Continue;
		InFunc(StateMachineGraph, IterationResult);

		if (IterationResult == EIterationResult::Break)
		{
			return;
		}

		for (UEdGraphNode* Node : StateMachineGraph->Nodes)
		{
			if (USceneStateMachineNode* StateMachineNode = Cast<USceneStateMachineNode>(Node))
			{
				IterationResult = EIterationResult::Continue;
				VisitGraphs(StateMachineNode->GetSubGraphs(), InFunc, IterationResult);

				if (IterationResult == EIterationResult::Break)
				{
					return;
				}
			}
		}
	}
}

} // Private

void VisitNodes(TConstArrayView<UEdGraph*> InGraphs, TFunctionRef<void(USceneStateMachineNode*, EIterationResult&)> InFunc)
{
	EIterationResult IterationResult = EIterationResult::Continue;
	Private::VisitNodes(InGraphs, InFunc, IterationResult);
}

void VisitGraphs(TConstArrayView<UEdGraph*> InGraphs, TFunctionRef<void(USceneStateMachineGraph*, EIterationResult&)> InFunc)
{
	EIterationResult IterationResult = EIterationResult::Continue;
	Private::VisitGraphs(InGraphs, InFunc, IterationResult);
}

void CreateBlueprintVariables(USceneStateBlueprint* InBlueprint, TArrayView<UE::PropertyBinding::FPropertyCreationDescriptor> InPropertyCreationDescs)
{
	if (!InBlueprint)
	{
		return;
	}

	for (UE::PropertyBinding::FPropertyCreationDescriptor& CreationDesc : InPropertyCreationDescs)
	{
		const FEdGraphPinType VariableType = UE::StructUtils::GetPropertyDescAsPin(CreationDesc.PropertyDesc);
		if (VariableType.PinCategory == NAME_None)
		{
			continue;
		}

		const FName MemberName = FBlueprintEditorUtils::FindUniqueKismetName(InBlueprint
			, CreationDesc.PropertyDesc.Name.ToString()
			, InBlueprint->SkeletonGeneratedClass);

		FString DefaultValue;
		if (CreationDesc.SourceProperty && CreationDesc.SourceContainerAddress)
		{
			const FEdGraphPinType SourcePropertyType = GetPropertyDescAsPin(FPropertyBagPropertyDesc(CreationDesc.SourceProperty->GetFName(), CreationDesc.SourceProperty));
			// Only copy default if the values are the exact same
			if (SourcePropertyType == VariableType)
			{
				const void* SourceValue = CreationDesc.SourceProperty->ContainerPtrToValuePtr<void>(CreationDesc.SourceContainerAddress);
				CreationDesc.SourceProperty->ExportText_Direct(DefaultValue, SourceValue, SourceValue, nullptr, PPF_None);
			}
		}

		if (FBlueprintEditorUtils::AddMemberVariable(InBlueprint, MemberName, VariableType, DefaultValue))
		{
			CreationDesc.PropertyDesc.Name = MemberName;
		}
	}
}

int32 FindIndexOfGraphInParent(UEdGraph* InGraph)
{
	if (USceneStateBlueprint* Blueprint = Cast<USceneStateBlueprint>(FBlueprintEditorUtils::FindBlueprintForGraph(InGraph)))
	{
		// Only consider top-level state machines as reorderable.
		const int32 Index = Blueprint->StateMachineGraphs.IndexOfByKey(InGraph);
		if (Index != INDEX_NONE)
		{
			return Index;
		}
	}
	return FBlueprintEditorUtils::FindIndexOfGraphInParent(InGraph);
}

bool MoveGraph(UEdGraph* InGraph, int32 InTargetIndex)
{
	USceneStateBlueprint* Blueprint = Cast<USceneStateBlueprint>(FBlueprintEditorUtils::FindBlueprintForGraph(InGraph));
	if (!Blueprint)
	{
		return false;
	}

	// Deal with State Machine Graphs
	const int32 CurrentIndex = Blueprint->StateMachineGraphs.IndexOfByKey(InGraph);
	if (CurrentIndex != INDEX_NONE && CurrentIndex != InTargetIndex && Blueprint->StateMachineGraphs.IsValidIndex(InTargetIndex))
	{
		Blueprint->Modify();
		Blueprint->StateMachineGraphs.Insert(InGraph, InTargetIndex);
		Blueprint->StateMachineGraphs.RemoveAt(CurrentIndex < InTargetIndex ? CurrentIndex : CurrentIndex + 1);
		FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
		return true;
	}

	// Deal with Other Graphs in Blueprint
	return FBlueprintEditorUtils::MoveGraphBeforeOtherGraph(InGraph, InTargetIndex, /*bDontRecompile*/true);
}

} // namespace UE::SceneState::Graph
