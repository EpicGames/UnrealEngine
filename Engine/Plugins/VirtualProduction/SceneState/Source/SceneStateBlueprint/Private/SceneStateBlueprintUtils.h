// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Templates/FunctionFwd.h"

class UEdGraph;
class USceneStateBlueprint;
class USceneStateMachineGraph;
class USceneStateMachineNode;

namespace UE::PropertyBinding
{
	struct FPropertyCreationDescriptor;
}

namespace UE::SceneState::Graph
{
	enum class EIterationResult : uint8
	{
		Continue,
		Break,
	};

	/**
	 * Visits all the Scene State Machine Nodes in the given Graphs recursively.
	 * For each node it calls the passed in function
	 * @param InGraphs the graphs to look into
	 * @param InFunc the callable for each node, with option to break execution
	 */
	void VisitNodes(TConstArrayView<UEdGraph*> InGraphs, TFunctionRef<void(USceneStateMachineNode*, EIterationResult&)> InFunc);

	/**
	 * Visits all the Scene State Machine Graphs in the given Graphs recursively.
	 * For each node it calls the passed in function
	 * @param InGraphs the graphs to look into
	 * @param InFunc the callable for each node, with option to break execution
	 */
	void VisitGraphs(TConstArrayView<UEdGraph*> InGraphs, TFunctionRef<void(USceneStateMachineGraph*, EIterationResult&)> InFunc);

	/**
	 * Creates Blueprint Variables that match the given Creation Descs
	 * @param InBlueprint the blueprint to add variables to
	 * @param InPropertyCreationDescs describes property names, types, etc
	 */
	void CreateBlueprintVariables(USceneStateBlueprint* InBlueprint, TArrayView<UE::PropertyBinding::FPropertyCreationDescriptor> InPropertyCreationDescs);

	/** Finds the index of the specified graph in the parent (if it is not reorderable, then we will return INDEX_NONE) */
	int32 FindIndexOfGraphInParent(UEdGraph* InGraph);

	/** Reorders the specified graph to be at the given target index in the parent */
	bool MoveGraph(UEdGraph* InGraph, int32 InTargetIndex);

} // UE::SceneState::Graph
