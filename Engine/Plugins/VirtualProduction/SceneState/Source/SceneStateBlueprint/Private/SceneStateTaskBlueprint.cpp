// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateTaskBlueprint.h"
#include "Tasks/SceneStateTaskGeneratedClass.h"

UClass* USceneStateTaskBlueprint::GetBlueprintClass() const
{
	return USceneStateTaskGeneratedClass::StaticClass();
}

void USceneStateTaskBlueprint::GetReparentingRules(TSet<const UClass*>& OutAllowedChildrenOfClasses, TSet<const UClass*>& OutDisallowedChildrenOfClasses) const
{
	OutAllowedChildrenOfClasses.Add(USceneStateTaskGeneratedClass::StaticClass());
}

bool USceneStateTaskBlueprint::SupportedByDefaultBlueprintFactory() const
{
	return false;
}
