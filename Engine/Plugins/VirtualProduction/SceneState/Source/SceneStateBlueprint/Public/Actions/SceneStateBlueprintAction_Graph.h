// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraphSchema_K2_Actions.h"
#include "Textures/SlateIcon.h"

#define UE_API SCENESTATEBLUEPRINT_API

namespace UE::SceneState::Graph
{

struct UE_INTERNAL FBlueprintAction_Graph : public FEdGraphSchemaAction_K2Graph
{
	FBlueprintAction_Graph() = default;

	struct FArguments
	{
		UEdGraph* Graph = nullptr;
		EEdGraphSchemaAction_K2Graph::Type GraphType = EEdGraphSchemaAction_K2Graph::Graph;
		FText Category;
		FSlateIcon Icon;
		int32 Grouping = 0;
		int32 SectionID = 0;
	};
	UE_API explicit FBlueprintAction_Graph(const FArguments& InArgs);

	UE_API static FName StaticGetTypeId();

	//~ Begin FEdGraphSchemaAction
	UE_API virtual const FSlateBrush* GetPaletteIcon() const override;
	UE_API virtual FName GetTypeId() const override;
	UE_API virtual void MovePersistentItemToCategory(const FText& InNewCategory) override;
	UE_API virtual int32 GetReorderIndexInContainer() const override;
	UE_API virtual bool ReorderToBeforeAction(TSharedRef<FEdGraphSchemaAction> OtherAction) override;
	//~ End FEdGraphSchemaAction

private:
	FSlateIcon Icon;
};

} // UE::SceneState::Graph

#undef UE_API
