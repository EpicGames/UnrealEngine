// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Engine/Blueprint.h"
#include "ISceneStateSchemaProvider.h"
#include "SceneStateBindingCollection.h"
#include "SceneStateBindingCollectionOwner.h"
#include "SceneStateBlueprint.generated.h"

#define UE_API SCENESTATEBLUEPRINT_API

class USceneStateMachineGraph;
class USceneStateMachineStateNode;

UCLASS(MinimalAPI, DisplayName="Motion Design Scene State Blueprint")
class USceneStateBlueprint : public UBlueprint, public ISceneStateSchemaProvider, public ISceneStateBindingCollectionOwner
{
	GENERATED_BODY()

public:
	UE_API USceneStateBlueprint(const FObjectInitializer& InObjectInitializer);

	const FGuid& GetRootId() const
	{
		return RootId;
	}

	template<typename T>
	T* FindExtension() const
	{
		return Cast<T>(FindExtension(T::StaticClass()));
	}

	UE_API UBlueprintExtension* FindExtension(TSubclassOf<UBlueprintExtension> InClass) const;

	UE_API FSceneStateBindingDesc CreateRootBinding() const;

	struct FGetBindableStructsParams
	{
		/** Struct id to look bindable structs for */
		FGuid TargetStructId;
		/** Whether to include the global bindings to the struct */
		bool bIncludeGlobalDescs = true;
		/** Whether to include the binding functions bound to the struct */
		bool bIncludeFunctions = false;
	};
	/** Gathers the binding structs for a given struct id. */
	UE_API void GetBindingStructs(const FGetBindableStructsParams& InParams, TArray<TInstancedStruct<FSceneStateBindingDesc>>& OutBindingDescs) const;

	/** Iterates all functions found within the bindings */
	bool ForEachBindingFunction(TFunctionRef<bool(const FSceneStateBindingFunction&, const FPropertyBindingBinding&)> InFunc) const;

	//~ Begin ISceneStateBindingCollectionOwner
	UE_API virtual FSceneStateBindingCollection& GetBindingCollection() override;
	UE_API virtual const FSceneStateBindingCollection& GetBindingCollection() const override;
	UE_API virtual bool ForEachBindableFunction(TFunctionRef<bool(const FSceneStateBindingDesc&, const UE::SceneState::FBindingFunctionInfo&)> InFunc) const override;
	//~ End ISceneStateBindingCollectionOwner

	//~ Begin IPropertyBindingBindingCollectionOwner
	UE_API virtual void GetBindableStructs(const FGuid InTargetStructId, TArray<TInstancedStruct<FPropertyBindingBindableStructDescriptor>>& OutStructDescs) const override;
	UE_API virtual bool GetBindableStructByID(const FGuid InStructId, TInstancedStruct<FPropertyBindingBindableStructDescriptor>& OutStructDesc) const override;
	UE_API virtual bool GetBindingDataViewByID(const FGuid InStructId, FPropertyBindingDataView& OutDataView) const override;
	UE_API virtual FPropertyBindingBindingCollection* GetEditorPropertyBindings() override;
	UE_API virtual const FPropertyBindingBindingCollection* GetEditorPropertyBindings() const override;
	UE_API virtual bool CanCreateParameter(const FGuid InStructId) const override;
	UE_API virtual void CreateParametersForStruct(const FGuid InStructId, TArrayView<UE::PropertyBinding::FPropertyCreationDescriptor> InOutCreationDescs) override;
	UE_API virtual void AppendBindablePropertyFunctionStructs(TArray<TInstancedStruct<FPropertyBindingBindableStructDescriptor>>& InOutStructs) const override;
	//~ End IPropertyBindingBindingCollectionOwner

	/** Sets the scene state schema to use for this blueprint */
	UE_API void SetSceneStateSchema(TSubclassOf<USceneStateSchema> InSchemaClass);

	//~ Begin ISceneStateSchemaProvider
	UE_API virtual TSubclassOf<USceneStateSchema> GetSceneStateSchemaClass() const override;
	//~ End ISceneStateSchemaProvider

	//~ Begin UBlueprint
	UE_API virtual void SetObjectBeingDebugged(UObject* InNewObject) override;
	UE_API virtual UClass* GetBlueprintClass() const override;
	UE_API virtual void GetReparentingRules(TSet<const UClass*>& OutAllowedChildrenOfClasses, TSet<const UClass*>& OutDisallowedChildrenOfClasses) const override;
	UE_API virtual bool SupportedByDefaultBlueprintFactory() const override;
	UE_API virtual void LoadModulesRequiredForCompilation() override;
	UE_API virtual bool IsValidForBytecodeOnlyRecompile() const override;
	//~ End UBlueprint

	//~ Begin UObject
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostLoad() override;
	UE_API virtual void BeginDestroy() override;
	//~ End UObject

	/** The top level State Machine Graphs of this State. Does not include the state machine graphs of nested state nodes */
	UPROPERTY()
	TArray<TObjectPtr<UEdGraph>> StateMachineGraphs;

	/** Holds all the editor bindings prior to compilation */
	UPROPERTY()
	FSceneStateBindingCollection BindingCollection;

private:
	/** Called when a blueprint variable has been renamed. Used to fix the bindings to the variable being renamed to use the new name */
	void OnRenameVariableReferences(UBlueprint* InBlueprint, UClass* InVariableClass, const FName& InOldVariableName, const FName& InNewVariableName);

	/** Called when a blueprint variable has been renamed. Used to fix the path to the variable being renamed to use the new name */
	void RenameVariableReferenceInPath(FPropertyBindingPath& InPath, FName InOldVariableName, FName InNewVariableName);

	/** Called when a state machine graph's parameters have changed */
	void OnGraphParametersChanged(USceneStateMachineGraph* InGraph);

	/** Called when a state's parameters have changed */
	void OnStateParametersChanged(USceneStateMachineStateNode* InStateNode);

	/** Called when parameters have changed. Used to fix the path to parameters that have possibly been renamed */
	void UpdateParametersBindings(FPropertyBindingPath& InPath, const FGuid& InParametersId, const FInstancedPropertyBag& InParameters);

	/** Scene State schema class to use for this blueprint */
	UPROPERTY(VisibleAnywhere, Category="Scene State", AssetRegistrySearchable)
	TSubclassOf<USceneStateSchema> SceneStateSchemaClass;

	/** Unique id representing this blueprint / generated class as a bindable struct */
	UPROPERTY()
	FGuid RootId;

	/** Handle to the delegate when a blueprint variable has been renamed */
	FDelegateHandle OnRenameVariableReferencesHandle;

	/** Handle to the delegate when a state machine graph parameters have changed */
	FDelegateHandle OnGraphParametersChangedHandle;

	/** Handle to the delegate when a state parameters have changed */
	FDelegateHandle OnStateParametersChangedHandle;
};

#undef UE_API
