// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "UObject/ObjectPtr.h"

#define UE_API SCENESTATEBLUEPRINT_API

class UObject;
class USceneStateBlueprint;

namespace UE::SceneState::Graph
{

struct FBlueprintDebugObjectChange
{
	/** The blueprint whose debug object got changed */
	TObjectPtr<USceneStateBlueprint> Blueprint;
	/** The new debug object */
	TObjectPtr<UObject> DebugObject;
};
/**
 * Delegate called when the debug object of a blueprint has changed.
 * Blueprint Editor doesn't have a dedicated delegate for this and instead calls IBlueprintEditor::RefreshMyBlueprint, or IBlueprintEditor::RefreshEditors.
 * Additionally, these refresh functions are not called in every place the debug object is changed (e.g. in UWorld::TransferBlueprintDebugReferences).
 * UBlueprint::DebuggingWorldRegistrationHelper is closer to what is needed, but still requires a valid UWorld to have been found.
 * This delegate solves this by broadcasting in USceneStateBlueprint::SetObjectBeingDebugged without requiring a valid world.
 */
extern UE_API TMulticastDelegate<void(const FBlueprintDebugObjectChange&)> OnBlueprintDebugObjectChanged;

} // UE::SceneState

#undef UE_API
