// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Blueprint/BlueprintExtension.h"
#include "SceneStateBlueprintExtension.generated.h"

UCLASS(MinimalAPI)
class USceneStateBlueprintExtension : public UBlueprintExtension
{
	GENERATED_BODY()

	//~ Begin UBlueprintExtension
	virtual void GetAllGraphs(TArray<UEdGraph*>& OutGraphs) const override;
	//~ End UBlueprintExtension
};
