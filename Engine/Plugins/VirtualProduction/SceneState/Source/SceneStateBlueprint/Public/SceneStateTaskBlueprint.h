// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Blueprint.h"
#include "SceneStateTaskBlueprint.generated.h"

UCLASS(MinimalAPI, DisplayName="Motion Design Scene State Task Blueprint")
class USceneStateTaskBlueprint : public UBlueprint
{
	GENERATED_BODY()

	//~ Begin UBlueprint
	virtual UClass* GetBlueprintClass() const override;
	virtual void GetReparentingRules(TSet<const UClass*>& OutAllowedChildrenOfClasses, TSet<const UClass*>& OutDisallowedChildrenOfClasses) const override;
	virtual bool SupportedByDefaultBlueprintFactory() const override;
	//~ End UBlueprint
};
