// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SceneStateBlueprint : ModuleRules
{
    public SceneStateBlueprint(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "BlueprintGraph",
                "Core",
                "CoreUObject",
                "Engine",
                "PropertyBindingUtils",
                "SceneStateBinding",
                "SceneStateMachineGraph",
                "SlateCore",
            }
        );

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "SceneState",
                "SceneStateEvent",
                "StructUtilsEditor",
            }
        );

        if (Target.Type == TargetRules.TargetType.Editor)
        {
            PrivateDependencyModuleNames.AddRange(
                new string[]
                {
                    "UnrealEd",
                }
            );
        }
    }
}
