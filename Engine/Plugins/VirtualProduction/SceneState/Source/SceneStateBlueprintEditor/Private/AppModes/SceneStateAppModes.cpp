// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateAppModes.h"
#include "BlueprintEditorModes.h"
#include "Internationalization/Text.h"

#define LOCTEXT_NAMESPACE "SceneStateAppModes"

namespace UE::SceneState::Editor
{

const FName FAppModes::Blueprint(FBlueprintEditorApplicationModes::StandardBlueprintEditorMode);

FText FAppModes::GetAppModeDisplayName(FName InAppMode)
{
	if (InAppMode == Blueprint)
	{
		return LOCTEXT("Blueprint", "Blueprint");
	}

	return FText::GetEmpty();
}

} // UE::SceneState::Editor

#undef LOCTEXT_NAMESPACE
