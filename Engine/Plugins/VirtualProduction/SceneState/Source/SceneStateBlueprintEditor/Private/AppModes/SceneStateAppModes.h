// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"

class FText;

namespace UE::SceneState::Editor
{

/** Describes all the available App Modes for the Scene State Editor */
struct FAppModes
{
	/** Available App Modes */
	static const FName Blueprint;

	/** Retrieves the display name for a given app mode */
	static FText GetAppModeDisplayName(FName InAppMode);
};

} // UE::SceneState::Editor
