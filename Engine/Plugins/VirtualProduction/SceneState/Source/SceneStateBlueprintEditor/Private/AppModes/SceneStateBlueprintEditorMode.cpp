// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateBlueprintEditorMode.h"
#include "BlueprintEditorTabs.h"
#include "DebugControls/SceneStateDebugControlsTabFactory.h"
#include "DebugView/SceneStateDebugViewTabFactory.h"
#include "SBlueprintEditorToolbar.h"
#include "SceneStateAppModes.h"
#include "SceneStateBlueprintEditor.h"
#include "TabFactories/SceneStateStateMachineTabFactory.h"

namespace UE::SceneState::Editor
{

FBlueprintAppMode::FBlueprintAppMode(const TSharedRef<FSceneStateBlueprintEditor>& InBlueprintEditor)
	: FBlueprintEditorApplicationMode(InBlueprintEditor, FAppModes::Blueprint, &FAppModes::GetAppModeDisplayName)
{
	TabLayout = FTabManager::NewLayout("Standalone_SceneStateBlueprintEditor_V0_1")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()
				->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewSplitter()
					->SetOrientation(Orient_Vertical)
					->SetSizeCoefficient(0.15f)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.40f)
						->AddTab(FStateMachineTabFactory::TabId, ETabState::OpenedTab)
					)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.60f)
						->AddTab(FBlueprintEditorTabs::MyBlueprintID, ETabState::OpenedTab)
					)
				)
				->Split
				(
					FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.40f)
						->AddTab(FDebugViewTabFactory::TabId, ETabState::ClosedTab)
					)
					->SetSizeCoefficient(0.60f)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.60f)
						->AddTab("Document", ETabState::ClosedTab)
					)
				)
				->Split
				(
					FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)
					->SetSizeCoefficient(0.25f)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.35f)
						->AddTab(FBlueprintEditorTabs::DetailsID, ETabState::OpenedTab)
						->AddTab(FBlueprintEditorTabs::PaletteID, ETabState::ClosedTab)
					)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.35f)
						->AddTab(FDebugControlsTabFactory::TabId, ETabState::OpenedTab)
					)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.30f)
						->AddTab(FBlueprintEditorTabs::CompilerResultsID, ETabState::OpenedTab)
						->AddTab(FBlueprintEditorTabs::FindResultsID, ETabState::ClosedTab)
						->AddTab(FBlueprintEditorTabs::BookmarksID, ETabState::ClosedTab)
					)
				)
			)
		);

	BlueprintEditorTabFactories.RegisterFactory(MakeShared<FStateMachineTabFactory>(InBlueprintEditor));
	BlueprintEditorTabFactories.RegisterFactory(MakeShared<FDebugControlsTabFactory>(InBlueprintEditor));
	BlueprintEditorTabFactories.RegisterFactory(MakeShared<FDebugViewTabFactory>(InBlueprintEditor));

	if (UToolMenu* Toolbar = InBlueprintEditor->RegisterModeToolbarIfUnregistered(GetModeName()))
	{
		TSharedPtr<FBlueprintEditorToolbar> ToolbarBuilder = InBlueprintEditor->GetToolbarBuilder();
		check(ToolbarBuilder.IsValid());

		ToolbarBuilder->AddCompileToolbar(Toolbar);
		ToolbarBuilder->AddScriptingToolbar(Toolbar);
		ToolbarBuilder->AddBlueprintGlobalOptionsToolbar(Toolbar, /*bRegisterViewport*/false);
		ToolbarBuilder->AddDebuggingToolbar(Toolbar);
	}
}

} // UE::SceneState::Editor
