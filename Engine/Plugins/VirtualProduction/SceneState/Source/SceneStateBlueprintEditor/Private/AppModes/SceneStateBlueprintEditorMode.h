// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintEditorModes.h"

namespace UE::SceneState::Editor
{
	class FSceneStateBlueprintEditor;
}

namespace UE::SceneState::Editor
{

/** Defines the layout for the 'Blueprint' mode in the Scene State Editor */
class FBlueprintAppMode : public FBlueprintEditorApplicationMode
{
public:
	explicit FBlueprintAppMode(const TSharedRef<FSceneStateBlueprintEditor>& InBlueprintEditor);
};

} // UE::SceneState::Editor

