// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_SceneStateBlueprint.h"
#include "Misc/MessageDialog.h"
#include "SceneStateBlueprint.h"
#include "SceneStateBlueprintEditor.h"
#include "SceneStateBlueprintFactory.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_SceneStateBlueprint"

FText UAssetDefinition_SceneStateBlueprint::GetAssetDisplayName() const
{
	return LOCTEXT("AssetDisplayName", "Scene State Blueprint");
}

FLinearColor UAssetDefinition_SceneStateBlueprint::GetAssetColor() const
{
	return FLinearColor(FColor(155, 17, 30));
}

TSoftClassPtr<UObject> UAssetDefinition_SceneStateBlueprint::GetAssetClass() const
{
	return USceneStateBlueprint::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_SceneStateBlueprint::GetAssetCategories() const
{
	return MakeArrayView(&EAssetCategoryPaths::Blueprint, 1);
}

EAssetCommandResult UAssetDefinition_SceneStateBlueprint::OpenAssets(const FAssetOpenArgs& InOpenArgs) const
{
	for (USceneStateBlueprint* Blueprint : InOpenArgs.LoadObjects<USceneStateBlueprint>())
	{
		bool bShouldOpen = true;

		if (!Blueprint->SkeletonGeneratedClass || !Blueprint->GeneratedClass)
		{
			bShouldOpen = EAppReturnType::Yes == FMessageDialog::Open(EAppMsgType::YesNo
				, LOCTEXT("InvalidBlueprintClassPrompt",
					"Blueprint could not be loaded because it derives from an invalid class.\n"
					"Check to make sure the parent class for this blueprint hasn't been removed!\n"
					"Do you want to continue (it can crash the editor)?"));
		}

		if (bShouldOpen)
		{
			using namespace UE::SceneState::Editor;

			TSharedRef<FSceneStateBlueprintEditor> BlueprintEditor = MakeShared<FSceneStateBlueprintEditor>();
			BlueprintEditor->Init(Blueprint, InOpenArgs);
		}
	}

	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
