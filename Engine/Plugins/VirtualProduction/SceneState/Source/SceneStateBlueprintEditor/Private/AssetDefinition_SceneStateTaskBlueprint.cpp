// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_SceneStateTaskBlueprint.h"
#include "BlueprintEditorModule.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Misc/MessageDialog.h"
#include "SceneStateTaskBlueprint.h"
#include "SceneStateTaskBlueprintEditor.h"
#include "SceneStateTaskBlueprintFactory.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_SceneStateTaskBlueprint"

FText UAssetDefinition_SceneStateTaskBlueprint::GetAssetDisplayName() const
{
	return LOCTEXT("AssetDisplayName", "Scene State Task Blueprint");
}

FLinearColor UAssetDefinition_SceneStateTaskBlueprint::GetAssetColor() const
{
	return FLinearColor(FColor(15, 82, 186));
}

TSoftClassPtr<UObject> UAssetDefinition_SceneStateTaskBlueprint::GetAssetClass() const
{
	return USceneStateTaskBlueprint::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_SceneStateTaskBlueprint::GetAssetCategories() const
{
	return MakeArrayView(&EAssetCategoryPaths::Blueprint, 1);
}

EAssetCommandResult UAssetDefinition_SceneStateTaskBlueprint::OpenAssets(const FAssetOpenArgs& InOpenArgs) const
{
	FBlueprintEditorModule& BlueprintEditorModule = FModuleManager::LoadModuleChecked<FBlueprintEditorModule>("Kismet");

	for (USceneStateTaskBlueprint* Blueprint : InOpenArgs.LoadObjects<USceneStateTaskBlueprint>())
	{
		bool bShouldOpen = true;

		if (!Blueprint->SkeletonGeneratedClass || !Blueprint->GeneratedClass)
		{
			bShouldOpen = EAppReturnType::Yes == FMessageDialog::Open(EAppMsgType::YesNo
				, LOCTEXT("InvalidBlueprintClassPrompt",
					"Blueprint could not be loaded because it derives from an invalid class.\n"
					"Check to make sure the parent class for this blueprint hasn't been removed!\n"
					"Do you want to continue (it can crash the editor)?"));
		}

		if (bShouldOpen)
		{
			using namespace UE::SceneState::Editor;

			TSharedRef<FSceneStateTaskBlueprintEditor> BlueprintEditor = MakeShared<FSceneStateTaskBlueprintEditor>();
			BlueprintEditor->Init(Blueprint, InOpenArgs);
		}
	}

	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
