// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateBindingCompiler.h"
#include "Functions/SceneStateFunction.h"
#include "Misc/EnumerateRange.h"
#include "SceneStateBindingFunction.h"
#include "SceneStateBindingUtils.h"
#include "SceneStateBlueprint.h"
#include "SceneStateBlueprintCompilerContext.h"
#include "SceneStateEventSchema.h"
#include "SceneStatePropertyReference.h"
#include "SceneStateTemplateData.h"
#include "StructUtils/UserDefinedStruct.h"
#include "Tasks/SceneStateBlueprintableTask.h"
#include "Tasks/SceneStateTask.h"
#include "Tasks/SceneStateTaskBindingExtension.h"

#define LOCTEXT_NAMESPACE "SceneStateBindingCompiler"

namespace UE::SceneState::Editor
{

namespace Private
{

/** Returns whether the given BindingDesc+Path is a struct property of the given struct type */
bool IsStructPropertyOfType(const UStruct* InStruct, const FSceneStateBindingDesc& InBindingDesc, const FPropertyBindingPath& InBindingPath)
{
	TArray<FPropertyBindingPathIndirection> Indirections;
	if (InBindingPath.ResolveIndirections(InBindingDesc.Struct, Indirections) && !Indirections.IsEmpty())
	{
		const FProperty* TargetProperty = Indirections.Last().GetProperty();
		check(TargetProperty);
		if (const FStructProperty* TargetStructProperty = CastField<FStructProperty>(TargetProperty))
		{
			return TargetStructProperty->Struct->IsChildOf(InStruct);
		}
	}
	return false;
}

/**
 * Returns the preferred batching type for a given binding
 * @param InBindingCollection the collection containing the binding to check
 * @param InBinding the binding to check
 * @return the type of batching that should be performed for the given binding
 */
EDataAccessType GetBindingBatchingType(const FSceneStateBindingCollection& InBindingCollection, const FSceneStateBinding& InBinding)
{
	if (const FSceneStateBindingDesc* TargetDesc = InBindingCollection.FindBindingDesc(InBinding.TargetDataHandle))
	{
		if (IsStructPropertyOfType(FSceneStatePropertyReference::StaticStruct(), *TargetDesc, InBinding.GetTargetPath()))
		{
			return EDataAccessType::Reference;
		}
		return EDataAccessType::Copy;
	}
	return EDataAccessType::None;
}

} // UE::SceneState::Editor::Private

FBindingCompiler::FBindingCompiler(FBlueprintCompilerContext& InContext, TNotNull<USceneStateBlueprint*> InBlueprint, TNotNull<USceneStateTemplateData*> InTemplateData)
	: Context(InContext)
	, Blueprint(InBlueprint)
	, TemplateData(InTemplateData)
	, BindingFunctionCompiler(InTemplateData)
{
}

void FBindingCompiler::Compile()
{
	// Initial estimate count of the binding descs
	const int32 BindingDescInitialCount = 1 + TemplateData->Tasks.Num() + TemplateData->EventHandlers.Num() + TemplateData->TransitionParameters.Num();

	// Copy Editor Bindings and reserve for the binding descs to add
	TemplateData->BindingCollection.Bindings = Blueprint->BindingCollection.Bindings;
	TemplateData->BindingCollection.BindingDescs.Empty(BindingDescInitialCount);

	ValidBindingMap.Reserve(BindingDescInitialCount);

	BindingFunctionCompiler.Compile();

	AddRootBindingDesc();
	AddStateBindingDescs();
	AddStateMachineBindingDescs();
	AddTransitionBindingDescs();
	AddTaskBindingDescs();
	AddEventHandlerBindingDescs();
	AddFunctionBindingDescs();

	RemoveInvalidBindings();

	ResolveBindingDataHandles();

	GroupBindings();
	CompileCopies();
	CompileReferences();
}

void FBindingCompiler::AddBindingDesc(FSceneStateBindingDesc&& InBindingDesc, FPropertyBindingDataView InDataView)
{
	TemplateDataViewMap.Add(InBindingDesc.DataHandle, InDataView);

	// Record this binding in the valid binding map. This will be used to then remove all the invalid bindings from the collection
	ValidBindingMap.Add(InBindingDesc.ID, InDataView);

	TemplateData->BindingCollection.BindingDescs.Add(MoveTemp(InBindingDesc));
}

void FBindingCompiler::AddRootBindingDesc()
{
	FSceneStateBindingDesc BindingDesc = Blueprint->CreateRootBinding();

	// Set the Struct to the New Class during compilation
	BindingDesc.Struct = Context.GetGeneratedClass();
	BindingDesc.DataHandle = FSceneStateBindingDataHandle(ESceneStateDataType::Root);

	// Binding compilation happens late in the blueprint compilation, where the cdo is safe to get
	AddBindingDesc(MoveTemp(BindingDesc), FPropertyBindingDataView(Blueprint->GeneratedClass->GetDefaultObject()));
}

void FBindingCompiler::AddStateBindingDescs()
{
	for (const TPair<FGuid, uint16>& Pair : TemplateData->StateParameterIdToIndex)
	{
		const uint16 StateIndex = Pair.Value;

		FSceneState& State = TemplateData->States[StateIndex];
		if (!State.Parameters.IsValid())
		{
			continue;
		}

		FSceneStateBindingDesc BindingDesc;
		BindingDesc.ID = Pair.Key;
		BindingDesc.Name = FName("StateParameters", StateIndex);
		BindingDesc.Struct = State.Parameters.GetPropertyBagStruct();
		BindingDesc.DataHandle = FSceneStateBindingDataHandle(ESceneStateDataType::State, StateIndex);

		AddBindingDesc(MoveTemp(BindingDesc), State.Parameters.GetMutableValue());
	}
}

void FBindingCompiler::AddStateMachineBindingDescs()
{
	for (const TPair<FGuid, uint16>& Pair : TemplateData->StateMachineIdToIndex)
	{
		const uint16 StateMachineIndex = Pair.Value;

		FSceneStateMachine& StateMachine = TemplateData->StateMachines[StateMachineIndex];
		if (!StateMachine.Parameters.IsValid())
		{
			continue;
		}

		FSceneStateBindingDesc BindingDesc;
		BindingDesc.ID = Pair.Key;
		BindingDesc.Name = FName("StateMachineParameters", StateMachineIndex);
		BindingDesc.Struct = StateMachine.Parameters.GetPropertyBagStruct();
		BindingDesc.DataHandle = FSceneStateBindingDataHandle(ESceneStateDataType::StateMachine, StateMachineIndex);

		AddBindingDesc(MoveTemp(BindingDesc), StateMachine.Parameters.GetMutableValue());
	}
}

void FBindingCompiler::AddTransitionBindingDescs()
{
	for (TPair<uint16, FInstancedPropertyBag>& Pair : TemplateData->TransitionParameters)
	{
		const uint16 TransitionIndex = Pair.Key;

		const FSceneStateTransitionMetadata& TransitionMetadata = TemplateData->TransitionMetadata[TransitionIndex];

		FSceneStateBindingDesc BindingDesc;
		BindingDesc.ID = TransitionMetadata.ParametersId;
		BindingDesc.Name = FName("TransitionParameters", TransitionIndex);
		BindingDesc.Struct = Pair.Value.GetPropertyBagStruct();
		BindingDesc.DataHandle = FSceneStateBindingDataHandle(ESceneStateDataType::Transition, TransitionIndex);

		AddBindingDesc(MoveTemp(BindingDesc), Pair.Value.GetMutableValue());
	}
}

void FBindingCompiler::AddTaskBindingDescs()
{
	check(TemplateData->Tasks.Num() == TemplateData->TaskMetadata.Num());

	TArray<FSceneStateBindingDesc>& BindingDescs = TemplateData->BindingCollection.BindingDescs;

	for (int32 TaskIndex = 0; TaskIndex < TemplateData->Tasks.Num(); ++TaskIndex)
	{
		const FSceneStateTask& Task = TemplateData->Tasks[TaskIndex].Get<FSceneStateTask>();
		const FSceneStateTaskMetadata& TaskMetadata = TemplateData->TaskMetadata[TaskIndex];
		const FStructView TaskInstance = TemplateData->TaskInstances[TaskIndex];

		// Task Instance Binding
		{
			FSceneStateBindingDesc BindingDesc;
			BindingDesc.ID = TaskMetadata.TaskId;
			BindingDesc.Name = FName("Task", TaskIndex);
			BindingDesc.Struct = TaskInstance.GetScriptStruct();
			BindingDesc.DataHandle = FSceneStateBindingDataHandle(ESceneStateDataType::Task, TaskIndex);

			AddBindingDesc(MoveTemp(BindingDesc), TaskInstance);
		}

		const FSceneStateTaskBindingExtension* BindingExtension = Task.GetBindingExtension();
		if (!BindingExtension)
		{
			continue;
		}

		TSet<uint16> DataIndices;
		BindingExtension->VisitBindingDescs(TaskInstance,
			[&BindingDescs, TaskIndex, &DataIndices, This = this](const FTaskBindingDesc& InBindingDesc)
			{
				bool bAlreadyInSet = false;
				DataIndices.Add(InBindingDesc.DataIndex, &bAlreadyInSet);
				checkf(!bAlreadyInSet, TEXT("Data Index already being used by another Desc in this Task!"));

				FSceneStateBindingDesc BindingDesc;
				BindingDesc.ID = InBindingDesc.Id;
				BindingDesc.Name = InBindingDesc.Name;
				BindingDesc.Struct = InBindingDesc.DataView.GetStruct();
				BindingDesc.DataHandle = FSceneStateBindingDataHandle(ESceneStateDataType::TaskExtension, TaskIndex, InBindingDesc.DataIndex);

				This->AddBindingDesc(MoveTemp(BindingDesc), InBindingDesc.DataView);
			});
	}
}

void FBindingCompiler::AddEventHandlerBindingDescs()
{
	for (int32 EventHandlerIndex = 0; EventHandlerIndex < TemplateData->EventHandlers.Num(); ++EventHandlerIndex)
	{
		const FSceneStateEventHandler& EventHandler = TemplateData->EventHandlers[EventHandlerIndex];

		const USceneStateEventSchemaObject* EventSchema = EventHandler.GetEventSchemaHandle().GetEventSchema();

		// Only event schemas with a valid struct are bindable
		if (!EventSchema || !EventSchema->Struct)
		{
			continue;
		}

		FSceneStateBindingDesc BindingDesc;
		BindingDesc.ID = EventHandler.GetHandlerId();
		BindingDesc.Name = EventSchema->Name;
		BindingDesc.Struct = EventSchema->Struct;
		BindingDesc.DataHandle = FSceneStateBindingDataHandle(ESceneStateDataType::EventHandler, EventHandlerIndex);

		AddBindingDesc(MoveTemp(BindingDesc), FPropertyBindingDataView(EventSchema->Struct, const_cast<uint8*>(EventSchema->Struct->GetDefaultInstance())));
	}
}

void FBindingCompiler::AddFunctionBindingDescs()
{
	check(TemplateData->Functions.Num() == TemplateData->FunctionMetadata.Num()
		&& TemplateData->Functions.Num() == TemplateData->FunctionInstances.Num());

	for (int32 FunctionIndex = 0; FunctionIndex < TemplateData->Functions.Num(); ++FunctionIndex)
	{
		const FStructView FunctionInstance = TemplateData->FunctionInstances[FunctionIndex];
		const FSceneStateFunctionMetadata& FunctionMetadata = TemplateData->FunctionMetadata[FunctionIndex];

		FSceneStateBindingDesc BindingDesc;
		BindingDesc.ID = FunctionMetadata.FunctionId;
		BindingDesc.Name = FName(TEXT("Function"), FunctionIndex);
		BindingDesc.Struct = FunctionInstance.GetScriptStruct();
		BindingDesc.DataHandle = FSceneStateBindingDataHandle(ESceneStateDataType::Function, FunctionIndex);

		AddBindingDesc(MoveTemp(BindingDesc), FunctionInstance);
	}
}

bool FBindingCompiler::ValidateBinding(const FSceneStateBinding& InBinding) const
{
	const USceneStateBlueprint::FGetBindableStructsParams Params
		{
			.TargetStructId = InBinding.GetTargetPath().GetStructID(),
			.bIncludeFunctions = true,
		};

	// Source must be accessible by the target struct.
	// This mismatch could happen if copying an object bound to a scoped parameter and pasting it outside such scope
	TArray<TInstancedStruct<FSceneStateBindingDesc>> AccessibleStructs;
	Blueprint->GetBindingStructs(Params, AccessibleStructs);

	const FGuid& SourceStructId = InBinding.GetSourcePath().GetStructID();
	const bool bSourceAccessible = AccessibleStructs.ContainsByPredicate(
		[&SourceStructId](TConstStructView<FPropertyBindingBindableStructDescriptor> InBindableStruct)
		{
			return InBindableStruct.Get().ID == SourceStructId;
		});

	if (!bSourceAccessible)
	{
		const FText ErrorMessageFormat = FText::Format(LOCTEXT("InaccessibleSourceError"
			, "Source '{0}' cannot be bound to target '{1}' because it's inaccessible (in another scope)")
			, FText::FromString(InBinding.GetSourcePath().ToString())
			, FText::FromString(InBinding.GetTargetPath().ToString()));

		Context.MessageLog.Error(*ErrorMessageFormat.ToString());
		return false;
	}

	if (!InBinding.SourceDataHandle.IsValid())
	{
		const FText ErrorMessageFormat = FText::Format(LOCTEXT("InvalidSourceHandleError"
			, "Source '{0}' data handle was not found")
			, FText::FromString(InBinding.GetSourcePath().ToString()));

		Context.MessageLog.Error(*ErrorMessageFormat.ToString());
		return false;
	}

	if (!InBinding.TargetDataHandle.IsValid())
	{
		const FText ErrorMessageFormat = FText::Format(LOCTEXT("InvalidTargetHandleError"
			, "Target '{0}' data handle was not found")
			, FText::FromString(InBinding.GetTargetPath().ToString()));

		Context.MessageLog.Error(*ErrorMessageFormat.ToString());
		return false;
	}

	return true;
}

void FBindingCompiler::ResolveBindingDataHandles()
{
	for (TArray<FSceneStateBinding>::TIterator Iter(TemplateData->BindingCollection.Bindings); Iter; ++Iter)
	{
		FSceneStateBinding& Binding = *Iter;
		Binding.SourceDataHandle = GetDataHandleById(Binding.GetSourcePath().GetStructID());
		Binding.TargetDataHandle = GetDataHandleById(Binding.GetTargetPath().GetStructID());

		if (!ValidateBinding(Binding))
		{
			Iter.RemoveCurrentSwap();
		}
	}
}

void FBindingCompiler::RemoveInvalidBindings()
{
	// Remove all the bindings not present in the valid binding map
	Blueprint->BindingCollection.RemoveInvalidBindings(ValidBindingMap);
	TemplateData->BindingCollection.RemoveInvalidBindings(ValidBindingMap);
}

void FBindingCompiler::GroupBindings()
{
	FSceneStateBindingCollection& BindingCollection = TemplateData->BindingCollection;

	// Sort by Batching Type (Copy,Reference) then by the Target Data.
	BindingCollection.Bindings.StableSort(
		[&BindingCollection](const FSceneStateBinding& InLeft, const FSceneStateBinding& InRight)
		{
			const EDataAccessType LeftBatchingType = Private::GetBindingBatchingType(BindingCollection, InLeft);
			const EDataAccessType RightBatchingType = Private::GetBindingBatchingType(BindingCollection, InRight);

			if (LeftBatchingType != RightBatchingType)
			{
				return LeftBatchingType < RightBatchingType;
			}

			return InLeft.TargetDataHandle.AsNumber() < InRight.TargetDataHandle.AsNumber();
		});

	BatchRangeMap.Reset();

	// Keeps track of the latest batching type to determine its range
	TOptional<EDataAccessType> LatestBatchingType;

	for (TConstEnumerateRef<FSceneStateBinding> Binding : EnumerateRange(BindingCollection.Bindings))
	{
		const EDataAccessType BatchingType = Private::GetBindingBatchingType(BindingCollection, *Binding);
		if (LatestBatchingType.IsSet() && BatchingType == *LatestBatchingType)
		{
			// Increment the count for the latest batch range
			++BatchRangeMap[*LatestBatchingType].Count;
		}
		else
		{
			LatestBatchingType = BatchingType;

			// Batch Start Index Map must not contain batching types that have been previously added as the bindings were already sorted by batching types 
			ensure(!BatchRangeMap.Contains(BatchingType));

			FBatchRange& BatchRange = BatchRangeMap.Add(BatchingType);
			BatchRange.Index = Binding.GetIndex();
			BatchRange.Count = 1;
		}
	}
}

void FBindingCompiler::CompileCopies()
{
	const FBatchRange* CopyRange = BatchRangeMap.Find(EDataAccessType::Copy);
	if (!CopyRange)
	{
		return;
	}

	TConstArrayView<FSceneStateBinding> CopyBindings = MakeArrayView(TemplateData->BindingCollection.Bindings).Slice(CopyRange->Index, CopyRange->Count);

	// Add 1 batch per group of bindings that all have the same target data handle
	for (int32 BindingIndex = 0; BindingIndex < CopyBindings.Num();)
	{
		const FSceneStateBinding& CopyBinding = CopyBindings[BindingIndex];

		const FSceneStateBindingDesc* TargetDesc = TemplateData->BindingCollection.FindBindingDesc(CopyBinding.TargetDataHandle);
		if (!ensure(TargetDesc))
		{
			++BindingIndex;
			continue;
		}

		const FPropertyBindingIndex16 BatchBindings(TemplateData->BindingCollection.GetNumCopyBatches());

		FPropertyBindingCopyInfoBatch& CopyBatch = TemplateData->BindingCollection.AddCopyBatch();
		CopyBatch.TargetStruct = TInstancedStruct<FSceneStateBindingDesc>::Make(*TargetDesc);
		CopyBatch.BindingsBegin = FPropertyBindingIndex16(BindingIndex);

		// Get the last index that matches the current target data handle
		for (; BindingIndex < CopyBindings.Num() && CopyBindings[BindingIndex].TargetDataHandle == CopyBinding.TargetDataHandle; ++BindingIndex)
		{
		}

		// Binding end is one past the last binding.
		CopyBatch.BindingsEnd = FPropertyBindingIndex16(BindingIndex);

		// Get the compiled function range for this struct
		const FSceneStateRange FunctionRange = BindingFunctionCompiler.GetFunctionRange(TargetDesc->ID);
		if (FunctionRange.IsValid())
		{
			CopyBatch.PropertyFunctionsBegin = FPropertyBindingIndex16(FunctionRange.Index);
			CopyBatch.PropertyFunctionsEnd = FPropertyBindingIndex16(FunctionRange.Index + FunctionRange.Count);
		}

		OnBindingsBatchCompiled(BatchBindings, CopyBinding.TargetDataHandle);
	}
}

void FBindingCompiler::CompileReferences()
{
	const FBatchRange* ReferenceRange = BatchRangeMap.Find(EDataAccessType::Reference);
	if (!ReferenceRange)
	{
		return;
	}

	if (ReferenceRange->Count > FSceneStatePropertyReference::IndexCapacity)
	{
		const FText ErrorMessageFormat = FText::Format(LOCTEXT("MaxReferenceCountReached"
			, "Reference range {0} has more entries than the allowed number {1}")
			, FText::AsNumber(ReferenceRange->Count)
			, FText::AsNumber(FSceneStatePropertyReference::IndexCapacity));

		Context.MessageLog.Error(*ErrorMessageFormat.ToString());
		return;
	}

	TConstArrayView<FSceneStateBinding> ReferenceBindings = MakeArrayView(TemplateData->BindingCollection.Bindings).Slice(ReferenceRange->Index, ReferenceRange->Count);

	TemplateData->BindingCollection.References.Empty(ReferenceBindings.Num());

	TArray<FPropertyBindingPathIndirection> TargetIndirections;
	FString ErrorMessage;

	for (const FSceneStateBinding& ReferenceBinding : ReferenceBindings)
	{
		const FPropertyBindingDataView* TemplateDataView = TemplateDataViewMap.Find(ReferenceBinding.TargetDataHandle);
		if (!TemplateDataView)
		{
			const FText ErrorMessageFormat = FText::Format(LOCTEXT("FailedResolvingTargetDataView", "Error finding target data view for binding {0}")
				, FText::FromString(ReferenceBinding.ToString()));

			Context.MessageLog.Error(*ErrorMessageFormat.ToString());
			continue;
		}

		TargetIndirections.Reset();
		if (!ReferenceBinding.GetTargetPath().ResolveIndirectionsWithValue(*TemplateDataView, TargetIndirections, &ErrorMessage))
		{
			const FText ErrorMessageFormat = FText::Format(LOCTEXT("FailedResolvingReferenceIndirection"
				, "Error resolving path in {0} ({1}): {2}")
				, FText::FromString(GetNameSafe(TemplateDataView->GetStruct()))
				, FText::FromString(ReferenceBinding.ToString())
				, FText::FromString(ErrorMessage));

			Context.MessageLog.Error(*ErrorMessageFormat.ToString());
			continue;
		}

		FSceneStatePropertyReference* PropertyReference = static_cast<FSceneStatePropertyReference*>(const_cast<void*>(TargetIndirections.Last().GetPropertyAddress()));
		check(PropertyReference);
		PropertyReference->ReferenceIndex = TemplateData->BindingCollection.References.AddDefaulted();

		FSceneStateBindingReference& Reference =  TemplateData->BindingCollection.References[PropertyReference->ReferenceIndex];
		Reference.SourcePropertyPath = ReferenceBinding.GetSourcePath();
		Reference.SourceDataHandle = ReferenceBinding.SourceDataHandle;
	}
}

void FBindingCompiler::OnBindingsBatchCompiled(FPropertyBindingIndex16 InBindingsBatch, const FSceneStateBindingDataHandle& InTargetDataHandle)
{
	// External Data Types as Targets are not supported by design.
	check(!InTargetDataHandle.IsExternalDataType());

	const ESceneStateDataType DataType = static_cast<ESceneStateDataType>(InTargetDataHandle.GetDataType());
	const uint16 DataIndex = InTargetDataHandle.GetDataIndex();

	if (DataType == ESceneStateDataType::Task)
	{
		FSceneStateTask& Task = TemplateData->Tasks[DataIndex].Get<FSceneStateTask>();
		Task.BindingsBatch = InBindingsBatch;
	}
	else if (DataType == ESceneStateDataType::TaskExtension)
	{
		FSceneStateTask& Task = TemplateData->Tasks[DataIndex].Get<FSceneStateTask>();

		FSceneStateTaskBindingExtension* BindingExtension = const_cast<FSceneStateTaskBindingExtension*>(Task.GetBindingExtension());
		check(BindingExtension);
		BindingExtension->SetBindingBatch(InTargetDataHandle.GetDataSubIndex(), InBindingsBatch.Get());
	}
	else if (DataType == ESceneStateDataType::StateMachine)
	{
		FSceneStateMachine& StateMachine = TemplateData->StateMachines[DataIndex];
		StateMachine.BindingsBatch = InBindingsBatch;
	}
	else if (DataType == ESceneStateDataType::Transition)
	{
		FSceneStateTransition& Transition = TemplateData->Transitions[DataIndex];
		Transition.BindingsBatch = InBindingsBatch;
	}
	else if (DataType == ESceneStateDataType::Function)
	{
		FSceneStateFunction& Function = TemplateData->Functions[DataIndex].Get<FSceneStateFunction>();
		Function.BindingsBatch = InBindingsBatch;
	}
	else if (DataType == ESceneStateDataType::State)
	{
		FSceneState& State = TemplateData->States[DataIndex];
		State.BindingsBatch = InBindingsBatch;
	}
	else
	{
		// No other Data Types are supported as Targets at the moment
		checkNoEntry();
	}
}

FSceneStateBindingDataHandle FBindingCompiler::GetDataHandleById(const FGuid& InStructId)
{
	if (Blueprint->GetRootId() == InStructId)
	{
		return FSceneStateBindingDataHandle(ESceneStateDataType::Root);
	}

	// Tasks
	for (int32 TaskIndex = 0; TaskIndex < TemplateData->TaskMetadata.Num(); ++TaskIndex)
	{
		const FSceneStateTaskMetadata& TaskMetadata = TemplateData->TaskMetadata[TaskIndex];
		if (TaskMetadata.TaskId == InStructId)
		{
			return FSceneStateBindingDataHandle(ESceneStateDataType::Task, TaskIndex);
		}
	}

	// State Machines
	if (const uint16* StateMachineIndex = TemplateData->StateMachineIdToIndex.Find(InStructId))
	{
		return FSceneStateBindingDataHandle(ESceneStateDataType::StateMachine, *StateMachineIndex);
	}

	// Transitions
	for (const TPair<uint16, FInstancedPropertyBag>& Pair : TemplateData->TransitionParameters)
	{
		const uint16 TransitionIndex = Pair.Key;

		const FSceneStateTransitionMetadata& TransitionMetadata = TemplateData->TransitionMetadata[TransitionIndex];
		if (TransitionMetadata.ParametersId == InStructId)
		{
			return FSceneStateBindingDataHandle(ESceneStateDataType::Transition, TransitionIndex);
		}
	}

	// Event Handlers
	for (int32 EventHandlerIndex = 0; EventHandlerIndex < TemplateData->EventHandlers.Num(); ++EventHandlerIndex)
	{
		const FSceneStateEventHandler& EventHandler = TemplateData->EventHandlers[EventHandlerIndex];
		if (EventHandler.GetHandlerId() == InStructId)
		{
			return FSceneStateBindingDataHandle(ESceneStateDataType::EventHandler, EventHandlerIndex);
		}
	}

	// Task Custom Data 
	for (int32 TaskIndex = 0; TaskIndex < TemplateData->Tasks.Num(); ++TaskIndex)
	{
		const FSceneStateTask& Task = TemplateData->Tasks[TaskIndex].Get<const FSceneStateTask>();
		const FStructView TaskInstance = TemplateData->TaskInstances[TaskIndex];

		FStructView DataView;
		uint16 DataIndex;

		const FSceneStateTaskBindingExtension* BindingExtension = Task.GetBindingExtension();
		if (BindingExtension && BindingExtension->FindDataById(TaskInstance, InStructId, DataView, DataIndex))
		{
			return FSceneStateBindingDataHandle(ESceneStateDataType::TaskExtension, TaskIndex, DataIndex);
		}
	}

	// Functions
	for (int32 FunctionIndex = 0; FunctionIndex < TemplateData->Functions.Num(); ++FunctionIndex)
	{
		const FSceneStateFunctionMetadata& FunctionMetadata = TemplateData->FunctionMetadata[FunctionIndex];
		if (FunctionMetadata.FunctionId == InStructId)
		{
			return FSceneStateBindingDataHandle(ESceneStateDataType::Function, FunctionIndex);
		}
	}

	// States
	if (const uint16* StateIndex = TemplateData->StateParameterIdToIndex.Find(InStructId))
	{
		return FSceneStateBindingDataHandle(ESceneStateDataType::State, *StateIndex);
	}

	return FSceneStateBindingDataHandle();
}

} // UE::SceneState::Editor

#undef LOCTEXT_NAMESPACE
