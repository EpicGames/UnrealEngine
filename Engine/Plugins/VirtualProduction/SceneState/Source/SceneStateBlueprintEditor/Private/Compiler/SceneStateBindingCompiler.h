// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"
#include "PropertyBindingDataView.h"
#include "SceneStateBindingDataHandle.h"
#include "SceneStateBindingFunctionCompiler.h"
#include "UObject/ObjectPtr.h"

class USceneStateBlueprint;
class USceneStateTemplateData;
struct FPropertyBindingIndex16;
struct FSceneStateBinding;
struct FSceneStateBindingDesc;

namespace UE::SceneState::Editor
{
	class FBlueprintCompilerContext;
}

namespace UE::SceneState::Editor
{

/** Describes the types how a binding can be batched */
enum class EDataAccessType : uint8
{
	None,
	Copy,
	Reference,
};

class FBindingCompiler
{
public:
	explicit FBindingCompiler(FBlueprintCompilerContext& InContext, TNotNull<USceneStateBlueprint*> InBlueprint, TNotNull<USceneStateTemplateData*> InTemplateData);

	void Compile();

private:
	void AddBindingDesc(FSceneStateBindingDesc&& InBindingDesc, FPropertyBindingDataView InDataView);

	void AddRootBindingDesc();
	void AddStateBindingDescs();
	void AddStateMachineBindingDescs();
	void AddTransitionBindingDescs();
	void AddTaskBindingDescs();
	void AddEventHandlerBindingDescs();
	void AddFunctionBindingDescs();

	bool ValidateBinding(const FSceneStateBinding& InBinding) const;

	void ResolveBindingDataHandles();
	void RemoveInvalidBindings();

	void GroupBindings();
	void CompileCopies();
	void CompileReferences();

	void OnBindingsBatchCompiled(FPropertyBindingIndex16 InBindingsBatch, const FSceneStateBindingDataHandle& InTargetDataHandle);

	FSceneStateBindingDataHandle GetDataHandleById(const FGuid& InStructId);

	/** Compiler context used for error logging */
	FBlueprintCompilerContext& Context;

	/** The blueprint that generated the class */
	TObjectPtr<USceneStateBlueprint> Blueprint;

	/** The target to store the compiled bindings */
	TObjectPtr<USceneStateTemplateData> TemplateData;

	/** Compiler for the functions found within the bindings */
	FBindingFunctionCompiler BindingFunctionCompiler;

	/** Map of struct ids to their binding data view */
	TMap<FGuid, const FPropertyBindingDataView> ValidBindingMap;

	/** Map of gathered binding data handle to the template data view */
	TMap<FSceneStateBindingDataHandle, FPropertyBindingDataView> TemplateDataViewMap;

	struct FBatchRange
	{
		/** Start index of the batch, inclusive */
		int32 Index = INDEX_NONE;
		/** Number of elements in the batch */
		int32 Count = 0;
	};
	/**
	 * Map of the batching type to the binding range index for that type
	 * A map is not the best candidate to hold this information, and only used for readability.
	 */
	TMap<EDataAccessType, FBatchRange> BatchRangeMap;
};

} // UE::SceneState::Editor
