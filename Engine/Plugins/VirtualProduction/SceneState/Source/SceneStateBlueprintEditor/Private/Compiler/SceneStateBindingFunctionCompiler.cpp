// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateBindingFunctionCompiler.h"
#include "SceneStateBindingFunction.h"
#include "SceneStateTemplateData.h"
#include "Tasks/SceneStateTask.h"
#include "Tasks/SceneStateTaskBindingExtension.h"

namespace UE::SceneState::Editor
{

void FBindingFunctionCompiler::Compile()
{
	FunctionRangeMap.Reset();

	CompileStateMachineFunctions();
	CompileTransitionFunctions();
	CompileTaskFunctions();
}

FSceneStateRange FBindingFunctionCompiler::GetFunctionRange(const FGuid& InTargetStructId) const
{
	return FunctionRangeMap.FindRef(InTargetStructId);
}

void FBindingFunctionCompiler::CompileFunctions(const FGuid& InTargetStructId)
{
	if (!InTargetStructId.IsValid())
	{
		return;
	}

	// If this happens, function bindings are circular.
	if (!ensure(!FunctionRangeMap.Contains(InTargetStructId)))
	{
		return;
	}
	FunctionRangeMap.Add(InTargetStructId);

	const FSceneStateBindingCollection& BindingCollection = TemplateData->BindingCollection;

	TArray<FConstStructView> Functions;
	TArray<FConstStructView> FunctionInstances;
	TArray<FSceneStateFunctionMetadata> FunctionMetadata;

	BindingCollection.ForEachBinding(
		[This=this, &Functions, &FunctionInstances, &FunctionMetadata, &InTargetStructId](const FPropertyBindingBinding& InBinding)
		{
			// Skip bindings that don't have a valid source or don't match the target struct id
			if (!InBinding.GetSourcePath().GetStructID().IsValid() || InBinding.GetTargetPath().GetStructID() != InTargetStructId)
			{
				return;
			}

			if (const FSceneStateBindingFunction* BindingFunction = InBinding.GetPropertyFunctionNode().GetPtr<const FSceneStateBindingFunction>())
			{
				Functions.Add(BindingFunction->Function);
				FunctionInstances.Add(BindingFunction->FunctionInstance);

				FSceneStateFunctionMetadata& Metadata = FunctionMetadata.AddDefaulted_GetRef();
				Metadata.FunctionId = BindingFunction->FunctionId;

				// Compile the functions that target this binding function
				This->CompileFunctions(BindingFunction->FunctionId);
			}
		});

	check(Functions.Num() == FunctionInstances.Num() && Functions.Num() == FunctionMetadata.Num());

	if (Functions.IsEmpty())
	{
		FunctionRangeMap.Remove(InTargetStructId);
		return;
	}

	FSceneStateRange& FunctionRange = FunctionRangeMap[InTargetStructId];
	FunctionRange.Index = TemplateData->Functions.Num();
	FunctionRange.Count = Functions.Num();

	TemplateData->Functions.Append(Functions);
	TemplateData->FunctionInstances.Append(FunctionInstances);
	TemplateData->FunctionMetadata.Append(MoveTemp(FunctionMetadata));
}

void FBindingFunctionCompiler::CompileStateMachineFunctions()
{
	for (const TPair<FGuid, uint16>& Pair : TemplateData->StateMachineIdToIndex)
	{
		CompileFunctions(Pair.Key);
	}
}

void FBindingFunctionCompiler::CompileTransitionFunctions()
{
	for (const TPair<uint16, FInstancedPropertyBag>& Pair : TemplateData->TransitionParameters)
	{
		const FSceneStateTransitionMetadata& TransitionMetadata = TemplateData->TransitionMetadata[Pair.Key];
		CompileFunctions(TransitionMetadata.ParametersId);
	}
}

void FBindingFunctionCompiler::CompileTaskFunctions()
{
	check(TemplateData->Tasks.Num() == TemplateData->TaskMetadata.Num());

	for (int32 TaskIndex = 0; TaskIndex < TemplateData->Tasks.Num(); ++TaskIndex)
	{
		const FSceneStateTaskMetadata& TaskMetadata = TemplateData->TaskMetadata[TaskIndex];
		CompileFunctions(TaskMetadata.TaskId);

		const FSceneStateTask& Task = TemplateData->Tasks[TaskIndex].Get<const FSceneStateTask>();
		const FStructView TaskInstance = TemplateData->TaskInstances[TaskIndex];

		if (const FSceneStateTaskBindingExtension* BindingExtension = Task.GetBindingExtension())
		{
			BindingExtension->VisitBindingDescs(TaskInstance,
				[This=this](const FTaskBindingDesc& InBindingDesc)
				{
					This->CompileFunctions(InBindingDesc.Id);
				});
		}
	}
}

} // UE::SceneState::Editor
