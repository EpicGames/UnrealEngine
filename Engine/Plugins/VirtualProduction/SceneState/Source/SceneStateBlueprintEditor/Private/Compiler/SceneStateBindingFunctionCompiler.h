// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneStateRange.h"
#include "UObject/ObjectPtr.h"

class USceneStateTemplateData;

namespace UE::SceneState::Editor
{

/** Compiles all the functions found throughout all the bindings */
class FBindingFunctionCompiler
{
public:
	explicit FBindingFunctionCompiler(TNotNull<USceneStateTemplateData*> InTemplateData)
		: TemplateData(InTemplateData)
	{
	}

	/** Compiles all the functions in all the bindings meant to have functions (Tasks, State Machines, Transitions) */
	void Compile();

	/** Gets the range of the compiled functions that are bound to the target struct id */
	FSceneStateRange GetFunctionRange(const FGuid& InTargetStructId) const;

private:
	/** Compiles the functions found in all the bindings pointing towards target struct id */
	void CompileFunctions(const FGuid& InTargetStructId);

	/** Compiles the functions for all the compiled tasks */
	void CompileTaskFunctions();

	/** Compiles the functions for all the compiled state machines */
	void CompileStateMachineFunctions();

	/** Compiles the functions for all the compiled transitions */
	void CompileTransitionFunctions();

	/** The target to store the compiled functions */
	TObjectPtr<USceneStateTemplateData> TemplateData;

	/** Map of the target struct id to its range of the functions in the generated class */
	TMap<FGuid, FSceneStateRange> FunctionRangeMap;
};

} // UE::SceneState::Editor
