// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateBlueprintCompilerContext.h"
#include "Kismet2/KismetReinstanceUtilities.h"
#include "Nodes/SceneStateMachineStateNode.h"
#include "Nodes/SceneStateMachineTaskNode.h"
#include "SceneStateBindingCompiler.h"
#include "SceneStateBlueprint.h"
#include "SceneStateGeneratedClass.h"
#include "SceneStateMachine.h"
#include "SceneStateMachineCompiler.h"
#include "SceneStateMachineGraph.h"
#include "SceneStateTemplateData.h"
#include "SceneStateTransitionGraphCompiler.h"
#include "SceneStateUtils.h"
#include "Tasks/SceneStateTask.h"

namespace UE::SceneState::Editor
{

namespace Private
{

void TrashObject(UObject* InObject, FName InBaseName, bool bInClearFlags, bool bInRecompilingOnLoad)
{
	// Rename will remove the renamed object's linker when moving to a new package so invalidate the export beforehand
	FLinkerLoad::InvalidateExport(InObject);
	InObject->SetFlags(RF_Transient);
	if (bInClearFlags)
	{
		InObject->ClearFlags(RF_Public | RF_Standalone | RF_ArchetypeObject);
	}

	ERenameFlags RenameFlags = REN_DontCreateRedirectors | REN_NonTransactional | REN_DoNotDirty;
	UPackage* const TransientPackage = GetTransientPackage();
	if (InBaseName.IsNone())
	{
		InObject->Rename(nullptr, TransientPackage, RenameFlags);
	}
	else
	{
		FName UniqueName = MakeUniqueObjectName(TransientPackage, InObject->GetClass(), InBaseName);
		InObject->Rename(*UniqueName.ToString(), TransientPackage, RenameFlags);
	}
};

TArray<USceneStateMachineGraph*> GetStateMachines(TConstArrayView<UEdGraph*> InStateMachineGraphs)
{
	TArray<USceneStateMachineGraph*> StateMachineGraphs;
	StateMachineGraphs.Reserve(InStateMachineGraphs.Num());

	for (UEdGraph* Graph : InStateMachineGraphs)
	{
		if (USceneStateMachineGraph* StateMachineGraph = Cast<USceneStateMachineGraph>(Graph))
		{
			StateMachineGraphs.Add(StateMachineGraph);
		}
	}

	return StateMachineGraphs;
}

} // Private
	
FBlueprintCompilerContext::FBlueprintCompilerContext(USceneStateBlueprint* InBlueprint, FCompilerResultsLog& InMessageLog, const FKismetCompilerOptions& InCompilerOptions)
	: FKismetCompilerContext(InBlueprint, InMessageLog, InCompilerOptions)
{
}

const UClass* FBlueprintCompilerContext::GetGeneratedClass() const
{
	return NewGeneratedClass;
}

void FBlueprintCompilerContext::SpawnNewClass(const FString& InNewClassName)
{
	check(Blueprint);
	NewGeneratedClass = FindObject<USceneStateGeneratedClass>(Blueprint->GetPackage(), *InNewClassName);

	if (!NewGeneratedClass)
	{
		NewGeneratedClass = NewObject<USceneStateGeneratedClass>(Blueprint->GetPackage(), *InNewClassName, RF_Public | RF_Transactional);
	}
	else
	{
		// Already existed, but wasn't linked in the Blueprint yet due to load ordering issues
		NewGeneratedClass->ClassGeneratedBy = Blueprint;
		FBlueprintCompileReinstancer::Create(NewGeneratedClass);
	}

	NewClass = NewGeneratedClass;
}

void FBlueprintCompilerContext::OnNewClassSet(UBlueprintGeneratedClass* InClassToUse)
{
	NewGeneratedClass = CastChecked<USceneStateGeneratedClass>(InClassToUse);
}

void FBlueprintCompilerContext::EnsureProperGeneratedClass(UClass*& InOutTargetClass)
{
	if (InOutTargetClass && !InOutTargetClass->UObject::IsA<USceneStateGeneratedClass>())
	{
		check(Blueprint);
		FKismetCompilerUtilities::ConsignToOblivion(InOutTargetClass, Blueprint->bIsRegeneratingOnLoad);
		InOutTargetClass = nullptr;
	}
}

void FBlueprintCompilerContext::SaveSubObjectsFromCleanAndSanitizeClass(FSubobjectCollection& OutSubObjectsToSave, UBlueprintGeneratedClass* InClassToClean)
{
	FKismetCompilerContext::SaveSubObjectsFromCleanAndSanitizeClass(OutSubObjectsToSave, InClassToClean);

	// Make sure our typed pointer is set
	check(InClassToClean == NewClass);

	NewGeneratedClass = CastChecked<USceneStateGeneratedClass>(NewClass);
}

void FBlueprintCompilerContext::MergeUbergraphPagesIn(UEdGraph* InUbergraph)
{
	FKismetCompilerContext::MergeUbergraphPagesIn(InUbergraph);

	USceneStateBlueprint* SceneStateBlueprint = CastChecked<USceneStateBlueprint>(Blueprint);

	RecreateTemplateData();
	check(NewGeneratedClass->TemplateData);

	// Top Level State Machines
	TArray<FSceneStateMachine> StateMachines;
	StateMachines.Reserve(SceneStateBlueprint->StateMachineGraphs.Num());

	TMap<FGuid, uint16> StateMachineIdToIndex;
	StateMachineIdToIndex.Reserve(SceneStateBlueprint->StateMachineGraphs.Num());

	TMap<FObjectKey, uint16> StateMachineGraphToIndex;
	StateMachineGraphToIndex.Reserve(SceneStateBlueprint->StateMachineGraphs.Num());

	const int32 RootStateIndex = NewGeneratedClass->TemplateData->States.AddDefaulted();
	const int32 RootStateMetadataIndex = NewGeneratedClass->TemplateData->StateMetadata.AddDefaulted();

	check(RootStateIndex == RootStateMetadataIndex);

	// Sort the state machines so that the ones that auto run are at the start
	TArray<USceneStateMachineGraph*> StateMachineGraphs = Private::GetStateMachines(SceneStateBlueprint->StateMachineGraphs);
	StateMachineGraphs.StableSort(
		[](const USceneStateMachineGraph& InGraphA, const USceneStateMachineGraph& InGraphB)
		{
			return InGraphA.RunMode < InGraphB.RunMode;
		});

	uint16 AutoRunCount = 0;
	for (USceneStateMachineGraph* StateMachineGraph : StateMachineGraphs)
	{
		FStateMachineCompiler StateMachineCompiler(StateMachineGraph, *this);
		FSceneStateMachine NewStateMachine = StateMachineCompiler.Compile();
		if (!NewStateMachine.IsValid())
		{
			continue;
		}

		const int32 StateMachineIndex = StateMachines.Emplace(MoveTemp(NewStateMachine));

		StateMachineIdToIndex.Emplace(StateMachineGraph->ParametersId, StateMachineIndex);
		StateMachineGraphToIndex.Emplace(StateMachineGraph, StateMachineIndex);

		if (StateMachineGraph->RunMode == ESceneStateMachineRunMode::Auto)
		{
			++AutoRunCount;
		}
	}

	NewGeneratedClass->TemplateData->RootStateIndex = RootStateIndex;

	// Root State only considers the range of the Auto-Run State Machines
	FSceneState& RootState = NewGeneratedClass->TemplateData->States[RootStateIndex];
	RootState.StateMachineRange.Index = NewGeneratedClass->TemplateData->StateMachines.Num();
	RootState.StateMachineRange.Count = AutoRunCount;

	NewGeneratedClass->TemplateData->StateMachines.Append(MoveTemp(StateMachines));

	// Upgrade the map to absolute indices before baking it to the generated class' map
	ToAbsoluteIndexMap(StateMachineIdToIndex, RootState.StateMachineRange.Index);
	NewGeneratedClass->TemplateData->StateMachineIdToIndex.Append(MoveTemp(StateMachineIdToIndex));

	// Upgrade the map to absolute indices before baking it to the generated class' map
	ToAbsoluteIndexMap(StateMachineGraphToIndex, RootState.StateMachineRange.Index);
	NewGeneratedClass->TemplateData->StateMachineGraphToIndex.Append(MoveTemp(StateMachineGraphToIndex));

	UpdateTaskIndices();
	SetCompiledIndices();
}

void FBlueprintCompilerContext::FinishCompilingClass(UClass* InClass)
{
	FKismetCompilerContext::FinishCompilingClass(InClass);
	if (NewGeneratedClass->IsFullClass())
	{
		CompileBindings();
	}
}

void FBlueprintCompilerContext::OnPostCDOCompiled(const UObject::FPostCDOCompiledContext& InContext)
{
	FKismetCompilerContext::OnPostCDOCompiled(InContext);
	NewGeneratedClass->ResolveBindings();
}

USceneStateTemplateData* FBlueprintCompilerContext::GetTemplateData()
{
	return NewGeneratedClass->TemplateData;
}

FTransitionGraphCompileResult FBlueprintCompilerContext::CompileTransitionGraph(USceneStateTransitionGraph* InTransitionGraph)
{
	FTransitionGraphCompiler TransitionCompiler(*this, InTransitionGraph);

	const ETransitionGraphCompileReturnCode ReturnCode = TransitionCompiler.Compile();

	FTransitionGraphCompileResult CompileResult;
	CompileResult.ReturnCode = ReturnCode;

	if (ReturnCode == ETransitionGraphCompileReturnCode::Success)
	{
		CompileResult.EventName = TransitionCompiler.GetCustomEventName();
		CompileResult.ResultPropertyName = TransitionCompiler.GetResultPropertyName();
	}

	return CompileResult;
}

void FBlueprintCompilerContext::RecreateTemplateData()
{
	const TCHAR* TemplateDataName = TEXT("SceneStateTemplateData");

	// Discard the current template data object as well as the unlikely event there's another object 
	DiscardObject(NewGeneratedClass->TemplateData);
	DiscardObject(NewGeneratedClass, TemplateDataName);

	NewGeneratedClass->TemplateData = NewObject<USceneStateTemplateData>(NewGeneratedClass, TemplateDataName, RF_Public);
}

void FBlueprintCompilerContext::UpdateTaskIndices()
{
	check(NewGeneratedClass->TemplateData);

	TConstArrayView<FSceneState> States = NewGeneratedClass->TemplateData->States;
	FInstancedStructContainer& Tasks = NewGeneratedClass->TemplateData->Tasks;

	for (int32 StateIndex = 0; StateIndex < States.Num(); ++StateIndex)
	{
		const FSceneStateRange TaskRange = States[StateIndex].GetTaskRange();
		if (!TaskRange.IsValid() || !Tasks.IsValidIndex(TaskRange.Index) || !Tasks.IsValidIndex(TaskRange.GetLastIndex()))
		{
			continue;
		}

		for (int32 TaskIndex = TaskRange.Index; TaskIndex <= TaskRange.GetLastIndex(); ++TaskIndex)
		{
			FSceneStateTask& Task = Tasks[TaskIndex].Get<FSceneStateTask>();
			Task.TaskIndex = TaskIndex;
			Task.ParentStateIndex = StateIndex;
		}
	}
}

void FBlueprintCompilerContext::SetCompiledIndices()
{
	check(NewGeneratedClass->TemplateData);

	for (TPair<FObjectKey, uint16>& Pair : NewGeneratedClass->TemplateData->StateNodeToIndex)
	{
		USceneStateMachineStateNode* const StateNode = CastChecked<USceneStateMachineStateNode>(Pair.Key.ResolveObjectPtr());
		StateNode->SetCompiledIndex(Pair.Value);
	}

	for (TPair<FObjectKey, uint16>& Pair : NewGeneratedClass->TemplateData->StateMachineGraphToIndex)
	{
		USceneStateMachineGraph* const StateMachineGraph = CastChecked<USceneStateMachineGraph>(Pair.Key.ResolveObjectPtr());
		StateMachineGraph->SetCompiledIndex(Pair.Value);
	}

	for (TPair<FObjectKey, uint16>& Pair : NewGeneratedClass->TemplateData->TaskNodeToIndex)
	{
		USceneStateMachineTaskNode* const TaskNode = CastChecked<USceneStateMachineTaskNode>(Pair.Key.ResolveObjectPtr());
		TaskNode->SetCompiledIndex(Pair.Value);
	}
}

void FBlueprintCompilerContext::CompileBindings()
{
	FBindingCompiler BindingCompiler(*this, CastChecked<USceneStateBlueprint>(Blueprint), NewGeneratedClass->TemplateData);
	BindingCompiler.Compile();
}

} // UE::SceneState::Editor
