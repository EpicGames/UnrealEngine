// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISceneStateMachineCompilerContext.h"
#include "KismetCompiler.h"
#include "SceneStateMachineCompiler.h"

class USceneStateBlueprint;
class USceneStateGeneratedClass;
class USceneStateTemplateData;

namespace UE::SceneState::Editor
{

class FBlueprintCompilerContext : public FKismetCompilerContext, public IStateMachineCompilerContext
{
public:
	explicit FBlueprintCompilerContext(USceneStateBlueprint* InBlueprint, FCompilerResultsLog& InMessageLog, const FKismetCompilerOptions& InCompilerOptions);

	/** Returns the new generated class for this compilation */
	const UClass* GetGeneratedClass() const;

private:
	//~ Begin FKismetCompilerContext
	virtual void SpawnNewClass(const FString& InNewClassName) override;
	virtual void OnNewClassSet(UBlueprintGeneratedClass* InClassToUse) override;
	virtual void EnsureProperGeneratedClass(UClass*& InOutTargetClass) override;
	virtual void SaveSubObjectsFromCleanAndSanitizeClass(FSubobjectCollection& OutSubObjectsToSave, UBlueprintGeneratedClass* InClassToClean) override;
	virtual void MergeUbergraphPagesIn(UEdGraph* InUbergraph) override;
	virtual void FinishCompilingClass(UClass* InClass) override;
	virtual void OnPostCDOCompiled(const UObject::FPostCDOCompiledContext& InContext) override;
	//~ End FKismetCompilerContext

	//~ Begin ISceneStateMachineCompilerContext
	virtual USceneStateTemplateData* GetTemplateData() override;
	virtual FTransitionGraphCompileResult CompileTransitionGraph(USceneStateTransitionGraph* InTransitionGraph) override;
	//~ End ISceneStateMachineCompilerContext

	/** Recreates the template data owned by the generated class */
	void RecreateTemplateData();

	/** Sets the Task Index and Parent State Index of each Task */
	void UpdateTaskIndices();

	/** Sets the indices of each mapped type (State, StateMachine, Task) to their editor node for debugging */
	void SetCompiledIndices();

	void CompileBindings();

	USceneStateGeneratedClass* NewGeneratedClass = nullptr;

	friend class FTransitionGraphCompiler;
};

} // UE::SceneState::Editor
