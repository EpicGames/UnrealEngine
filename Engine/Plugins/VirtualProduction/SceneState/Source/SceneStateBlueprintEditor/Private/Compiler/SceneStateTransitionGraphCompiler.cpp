// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateTransitionGraphCompiler.h"
#include "EdGraphUtilities.h"
#include "K2Node_CustomEvent.h"
#include "K2Node_StructMemberSet.h"
#include "Nodes/SceneStateMachineTransitionNode.h"
#include "Nodes/SceneStateTransitionResultNode.h"
#include "PropertyBagDetails.h"
#include "SceneStateBlueprintCompilerContext.h"
#include "SceneStateMachineGraph.h"
#include "SceneStateMachineGraphUtils.h"
#include "SceneStateTransitionGraph.h"
#include "Transition/SceneStateTransitionParametersNode.h"

namespace UE::SceneState::Editor
{

FTransitionGraphCompiler::FTransitionGraphCompiler(FBlueprintCompilerContext& InCompilerContext, USceneStateTransitionGraph* InOriginalGraph)
	: Context(InCompilerContext)
	, OriginalGraph(InOriginalGraph)
{
	check(OriginalGraph);
}

ETransitionGraphCompileReturnCode FTransitionGraphCompiler::Compile()
{
	// Evaluate whether this transition graph always evaluates to a fixed result
	bool bFixedResult;
	if (CanSkipCompilation(bFixedResult))
	{
		return bFixedResult
			? ETransitionGraphCompileReturnCode::SkippedAlwaysTrue
			: ETransitionGraphCompileReturnCode::SkippedAlwaysFalse;
	}

	// Clone Graph and move the clone into the Ubergraph
	CloneTransitionGraph();
	check(ClonedGraph && ResultNode);

	// Make a result variable that will be used to read the result when evaluating this graph
	if (!CreateTransitionResultProperty())
	{
		return ETransitionGraphCompileReturnCode::Failed;
	}

	// Make a Custom Event that evaluates this cloned graph result and sets it to the Result Property
	CreateTransitionEvaluationEvent();
	check(!CustomEventName.IsNone());

	// Destroy cloned result node, as all the logic has been relinked to the Custom Event, and the node has no exec pins.
	ResultNode->DestroyNode();
	return ETransitionGraphCompileReturnCode::Success;
}

FName FTransitionGraphCompiler::GetCustomEventName() const
{
	return CustomEventName;
}

FName FTransitionGraphCompiler::GetResultPropertyName() const
{
	return ResultProperty ? ResultProperty->GetFName() : NAME_None;
}

bool FTransitionGraphCompiler::CanSkipCompilation(bool& OutResult) const
{
	check(OriginalGraph->ResultNode);

	UEdGraphPin* Result = OriginalGraph->ResultNode->FindPin(GET_MEMBER_NAME_CHECKED(FSceneStateTransitionResult, bCanTransition));
	if (!Result)
	{
		return false;
	}

	if (Result->LinkedTo.IsEmpty())
	{
		LexFromString(OutResult, Result->DefaultValue);
		return true;
	}
	return false;
}

void FTransitionGraphCompiler::CloneTransitionGraph()
{
	check(OriginalGraph->ResultNode);

	// Clone the nodes from the source graph
	// Note that we outer this graph to the ConsolidatedEventGraph to allow ExpansionStep to 
	// correctly retrieve the context for any expanded function calls (custom make/break structs etc.)
	ClonedGraph = CastChecked<USceneStateTransitionGraph>(FEdGraphUtilities::CloneGraph(OriginalGraph
		, Context.ConsolidatedEventGraph
		, &Context.MessageLog
		, /*bCloningForCompile*/true));

	Context.MessageLog.NotifyIntermediateObjectCreation(ClonedGraph, OriginalGraph);

	// Find the cloned result by looking up if source objects match with original result node
	DiscoverResultNode();
	check(ResultNode);

	DiscoverParametersNodes();

	Context.ExpansionStep(ClonedGraph, /*bAllowUbergraphExpansions*/false);
	Context.ValidateGraphIsWellFormed(ClonedGraph);

	// Move the cloned nodes into the consolidated event graph
	const bool bIsLoading = (Context.Blueprint && Context.Blueprint->bIsRegeneratingOnLoad) || IsAsyncLoading();
	const bool bIsCompiling = Context.Blueprint && Context.Blueprint->bBeingCompiled;

	ClonedGraph->MoveNodesToAnotherGraph(Context.ConsolidatedEventGraph, bIsLoading, bIsCompiling);;
}

void FTransitionGraphCompiler::DiscoverResultNode()
{
	UObject* SourceResultNode = Context.MessageLog.FindSourceObject(OriginalGraph->ResultNode);

	for (UEdGraphNode* ClonedNode : ClonedGraph->Nodes)
	{
		if (USceneStateTransitionResultNode* ResultNodeCandidate = Cast<USceneStateTransitionResultNode>(ClonedNode))
		{
			if (Context.MessageLog.FindSourceObject(ResultNodeCandidate) == SourceResultNode)
			{
				ResultNode = ResultNodeCandidate;
				break;
			}
		}
	}
}

void FTransitionGraphCompiler::DiscoverParametersNodes()
{
	ParametersNodes.Reset();

	for (UEdGraphNode* ClonedNode : ClonedGraph->Nodes)
	{
		if (USceneStateTransitionParametersNode* ParametersNode = Cast<USceneStateTransitionParametersNode>(ClonedNode))
		{
			ParametersNodes.Add(ParametersNode);
		}
	}
}

bool FTransitionGraphCompiler::CreateTransitionResultProperty()
{
	const FName NodeVariableName = *Context.ClassScopeNetNameMap.MakeValidName(ResultNode);

	FEdGraphPinType NodeVariableType;
	NodeVariableType.PinCategory = UEdGraphSchema_K2::PC_Struct;
	NodeVariableType.PinSubCategoryObject = FSceneStateTransitionResult::StaticStruct();

	ResultProperty = CastField<FStructProperty>(Context.CreateVariable(NodeVariableName, NodeVariableType));
	if (!ResultProperty)
	{
		Context.MessageLog.Error(TEXT("Internal Compiler Error: Failed to create result property for @@"), ResultNode);
		return false;
	}

	return true;
}

void FTransitionGraphCompiler::CreateTransitionEvaluationEvent()
{
	const FName ResultPropertyName = ResultProperty->GetFName();

	// Use the node GUID for a stable name across compiles
	CustomEventName = *FString::Printf(TEXT("Get_%s_%s_%s")
		, *ResultPropertyName.ToString()
		, *ResultNode->GetName()
		, *ResultNode->NodeGuid.ToString());

	// The ExecChain is the current exec output pin in the linear chain
	UEdGraphPin* ExecutionChain = nullptr;

	auto ChainNode = [Schema = Context.Schema, &ExecutionChain](UEdGraphNode* InNode)
		{
			if (ExecutionChain)
			{
				UEdGraphPin* InputPin = Schema->FindExecutionPin(*InNode, EGPD_Input);
				ExecutionChain->MakeLinkTo(InputPin);
			}
			ExecutionChain = Schema->FindExecutionPin(*InNode, EGPD_Output);
		};

	// Add a custom event in the graph
	UK2Node_CustomEvent* CustomEventNode = CreateCustomEventNode();
	ChainNode(CustomEventNode);

	// New set node for the result property
	UK2Node_StructMemberSet* SetResultNode = Context.SpawnIntermediateNode<UK2Node_StructMemberSet>(ResultNode, Context.ConsolidatedEventGraph);
	{
		SetResultNode->VariableReference.SetSelfMember(ResultPropertyName);
		SetResultNode->StructType = ResultProperty->Struct;
		SetResultNode->AllocateExecPins();

		// FOptionalPinManager by default exposes all pins
		FOptionalPinManager OptionalPinManager;
		OptionalPinManager.RebuildPropertyList(SetResultNode->ShowPinForProperties, SetResultNode->StructType);
		OptionalPinManager.CreateVisiblePins(SetResultNode->ShowPinForProperties, SetResultNode->StructType, EGPD_Input, SetResultNode);

		// Copy the pin data (link up to the source nodes)
		for (UEdGraphPin* TargetPin : SetResultNode->Pins)
		{
			// Ignore Exec pins
			if (!TargetPin || TargetPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Exec)
			{
				continue;
			}

			for (UEdGraphPin* SourcePin : ResultNode->Pins)
			{
				if (SourcePin && TargetPin->PinName == SourcePin->PinName)
				{
					check(SourcePin->Direction == EGPD_Input);
					TargetPin->CopyPersistentDataFromOldPin(*SourcePin);
					Context.MessageLog.NotifyIntermediatePinCreation(TargetPin, SourcePin);
					break;
				}
			}
		}

		SetResultNode->ReconstructNode();
	}
	ChainNode(SetResultNode);
}

UK2Node_CustomEvent* FTransitionGraphCompiler::CreateCustomEventNode()
{
	UK2Node_CustomEvent* CustomEventNode = Context.SpawnIntermediateNode<UK2Node_CustomEvent>(ResultNode, Context.ConsolidatedEventGraph);
	CustomEventNode->bInternalEvent = true;
	CustomEventNode->CustomFunctionName = CustomEventName;

	// No parameter nodes were found so avoid overhead of having extra parameters (memcpy) in the custom event
	if (ParametersNodes.IsEmpty())
	{
		CustomEventNode->AllocateDefaultPins();
		return CustomEventNode;
	}

	USceneStateMachineTransitionNode* TransitionNode = OriginalGraph->GetTypedOuter<USceneStateMachineTransitionNode>();
	if (!TransitionNode)
	{
		CustomEventNode->AllocateDefaultPins();
		return CustomEventNode;
	}

	const UPropertyBag* PropertyBag = TransitionNode->GetParameters().GetPropertyBagStruct();
	if (!PropertyBag)
	{
		CustomEventNode->AllocateDefaultPins();
		return CustomEventNode;
	}

	TConstArrayView<FPropertyBagPropertyDesc> PropertyDescs = PropertyBag->GetPropertyDescs();
	CustomEventNode->UserDefinedPins.Empty(PropertyDescs.Num());

	for (const FPropertyBagPropertyDesc& PropertyDesc : PropertyDescs)
	{
		const FEdGraphPinType PinType = UE::SceneState::Graph::GetPropertyDescAsPin(PropertyDesc);
		CustomEventNode->CreateUserDefinedPin(PropertyDesc.Name, PinType, EGPD_Output, /*bUseUniqueName*/false);
	}

	CustomEventNode->AllocateDefaultPins();
	LinkParametersNodes(CustomEventNode);
	return CustomEventNode;
}

void FTransitionGraphCompiler::LinkParametersNodes(UK2Node_CustomEvent* InCustomEvent)
{
	for (USceneStateTransitionParametersNode* ParametersNode : ParametersNodes)
	{
		for (UEdGraphPin* SourcePin : ParametersNode->Pins)
		{
			if (SourcePin)
			{
				if (UEdGraphPin* IntermediatePin = InCustomEvent->FindPin(SourcePin->PinName))
				{
					Context.MovePinLinksToIntermediate(*SourcePin, *IntermediatePin);
				}
			}
		}
		ParametersNode->BreakAllNodeLinks();
		ParametersNode->DestroyNode();
	}
}

} // UE::SceneState::Editor
