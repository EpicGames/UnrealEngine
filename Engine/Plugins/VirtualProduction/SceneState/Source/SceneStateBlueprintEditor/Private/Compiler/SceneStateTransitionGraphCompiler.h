// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "ISceneStateMachineCompilerContext.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectPtr.h"

class FStructProperty;
class UK2Node_CustomEvent;
class USceneStateTransitionParametersNode;
class USceneStateTransitionGraph;
class USceneStateTransitionResultNode;

namespace UE::SceneState::Editor
{
	class FBlueprintCompilerContext;
}

namespace UE::SceneState::Editor
{

/** Compiler for the Transition K2 Graphs */
class FTransitionGraphCompiler
{
public:
	explicit FTransitionGraphCompiler(FBlueprintCompilerContext& InCompilerContext, USceneStateTransitionGraph* InOriginalGraph);

	ETransitionGraphCompileReturnCode Compile();

	/** Returns the name of the compiled event to call, if compilation succeeded */
	FName GetCustomEventName() const;

	/** Returns the name of the compiled result property set by the compiled event, if compilation succeeded */
	FName GetResultPropertyName() const;

private:
	/**
	 * Evaluates whether the transition graph can be skipped entirely if it always returns true or false
	 * @param OutResult the result that the transition will always evaluate to. Only correctly set if this function returns true
	 * @return true if the transition always evaluates to a fixed value
	 */
	bool CanSkipCompilation(bool& OutResult) const;

	/** Copies the transition graph and moves the cloned nodes to the consolidated graph */
	void CloneTransitionGraph();

	/** Finds the result node from the cloned graph (prior to the nodes moving to the consolidated graph) */
	void DiscoverResultNode();

	/** Finds all the parameters nodes from the cloned graph (prior to the nodes moving to the consolidated graph) */
	void DiscoverParametersNodes();

	/** Creates the transition result property to set for the custom event */
	bool CreateTransitionResultProperty();

	/** Creates and chains the nodes for evaluation (custom event, set result variable value , etc) */
	void CreateTransitionEvaluationEvent();

	/** Creates the custom event node of the Evaluation Event */
	UK2Node_CustomEvent* CreateCustomEventNode();

	/** Links all the found parameter nodes to the pins (i.e. params) of the custom event */
	void LinkParametersNodes(UK2Node_CustomEvent* InCustomEvent);

	/** the owning blueprint compiler context */
	FBlueprintCompilerContext& Context;

	/** the original graph to clone */
	TObjectPtr<USceneStateTransitionGraph> OriginalGraph = nullptr;

	/** the copied graph */
	TObjectPtr<USceneStateTransitionGraph> ClonedGraph = nullptr;

	/** the cloned result node discovered */
	TObjectPtr<USceneStateTransitionResultNode> ResultNode = nullptr;

	/** The compiled result property */
	FStructProperty* ResultProperty = nullptr;

	/** the name of the compiled custom event to call */
	FName CustomEventName;

	/** the cloned parameter nodes discovered */
	TArray<TObjectPtr<USceneStateTransitionParametersNode>> ParametersNodes;
};

} // UE::SceneState::Editor
