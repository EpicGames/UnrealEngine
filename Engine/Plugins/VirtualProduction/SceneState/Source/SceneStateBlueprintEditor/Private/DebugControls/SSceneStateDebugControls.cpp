// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSceneStateDebugControls.h"
#include "DetailLayoutBuilder.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SceneStateBlueprint.h"
#include "SceneStateBlueprintDelegates.h"
#include "SceneStateBlueprintEditor.h"
#include "SceneStateBlueprintEditorCommands.h"
#include "SceneStateDebugControlsObjectDetails.h"
#include "SceneStateDebugControlsTool.h"
#include "SceneStateDebugControlsToolbar.h"
#include "SceneStateEventTemplate.h"
#include "SceneStateEventUtils.h"
#include "SceneStateObject.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SSceneStateDebugControls"

namespace UE::SceneState::Editor
{

void SDebugControls::Construct(const FArguments& InArgs, const TSharedRef<FSceneStateBlueprintEditor>& InBlueprintEditor)
{
	BlueprintEditorWeak = InBlueprintEditor;

	OnBlueprintDebugObjectChangedHandle = Graph::OnBlueprintDebugObjectChanged.AddSP(this, &SDebugControls::OnBlueprintDebugObjectChanged);

	DebugControlsTool = MakeShared<FDebugControlsTool>(InBlueprintEditor);
	DebugControlsTool->Initialize();

	DebugControlsDetailsView = CreateDebugControlsDetailsView(InBlueprintEditor);
	DebugControlsDetailsView->SetObject(DebugControlsTool->GetDebugControlsObject());

	DebuggedObjectDetailsView = CreateDebuggedObjectDetailsView(InBlueprintEditor);

	ContentWidget = CreateContentWidget();
	PlaceholderWidget = CreatePlaceholderWidget();

	ChildSlot
	[
		SAssignNew(WidgetContainer, SBorder)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		.Padding(0)
		[
			PlaceholderWidget.ToSharedRef()
		]
	];

	Refresh();
}

SDebugControls::~SDebugControls()
{
	Graph::OnBlueprintDebugObjectChanged.Remove(OnBlueprintDebugObjectChangedHandle);
	OnBlueprintDebugObjectChangedHandle.Reset();
}

void SDebugControls::Refresh()
{
	USceneStateObject* const DebuggedObject = DebugControlsTool->GetDebuggedObject();
	DebuggedObjectDetailsView->SetObject(DebuggedObject);
}

void SDebugControls::Tick(const FGeometry& InAllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(InAllottedGeometry, InCurrentTime, InDeltaTime);
	DebugControlsTool->Tick(InDeltaTime);

	if (DebugControlsTool->IsAvailable())
	{
		WidgetContainer->SetContent(ContentWidget.ToSharedRef());
	}
	else
	{
		WidgetContainer->SetContent(PlaceholderWidget.ToSharedRef());
	}
}

TSharedRef<SWidget> SDebugControls::CreateContentWidget()
{
	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			CreateDebugControlsToolbar(DebugControlsTool->GetCommandList())
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			DebugControlsDetailsView.ToSharedRef()	
		]
		+ SVerticalBox::Slot()
		.FillHeight(1.f)
		.Padding(0, 5, 0, 0)
		[
			DebuggedObjectDetailsView.ToSharedRef()	
		];
}

TSharedRef<SWidget> SDebugControls::CreatePlaceholderWidget()
{
	return SNew(SBox)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Top)
		.Padding(2, 24, 2, 2)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("PlaceholderTitle", "Select an actively playing debug object"))
			.Font(IDetailLayoutBuilder::GetDetailFontItalic())
		];
}

TSharedRef<IDetailsView> SDebugControls::CreateDebugControlsDetailsView(const TSharedRef<FSceneStateBlueprintEditor>& InBlueprintEditor) const
{
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Hide;
	DetailsViewArgs.HostCommandList = InBlueprintEditor->GetToolkitCommands();
	DetailsViewArgs.HostTabManager = InBlueprintEditor->GetTabManager();
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.bShowOptions = false;
	DetailsViewArgs.bShowSectionSelector = false;

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	TSharedRef<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	DetailsView->RegisterInstancedCustomPropertyLayout(USceneStateDebugControlsObject::StaticClass()
		, FOnGetDetailCustomizationInstance::CreateStatic(&FDebugControlsObjectDetails::MakeInstance));

	return DetailsView;
}

TSharedRef<IDetailsView> SDebugControls::CreateDebuggedObjectDetailsView(const TSharedRef<FSceneStateBlueprintEditor>& InBlueprintEditor) const
{
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Hide;
	DetailsViewArgs.HostCommandList = InBlueprintEditor->GetToolkitCommands();
	DetailsViewArgs.HostTabManager = InBlueprintEditor->GetTabManager();
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bAllowSearch = true;
	DetailsViewArgs.bShowOptions = true;
	DetailsViewArgs.bShowSectionSelector = true;
	
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	return PropertyEditorModule.CreateDetailView(DetailsViewArgs);
}

void SDebugControls::OnBlueprintDebugObjectChanged(const Graph::FBlueprintDebugObjectChange& InChange)
{
	TSharedPtr<FSceneStateBlueprintEditor> BlueprintEditor = BlueprintEditorWeak.Pin();
	if (BlueprintEditor.IsValid() && BlueprintEditor->GetBlueprintObj() == InChange.Blueprint)
	{
		Refresh();
	}
}

} // UE::SceneState::Editor

#undef LOCTEXT_NAMESPACE
