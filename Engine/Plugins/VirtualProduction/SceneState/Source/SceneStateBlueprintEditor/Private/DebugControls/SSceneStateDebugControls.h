// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneStateDebugControlsObject.h"
#include "SceneStateEventTemplate.h"
#include "Widgets/SCompoundWidget.h"

class IDetailsView;
class SBorder;
class SWidget;
class USceneStateObject;

namespace UE::SceneState
{
	namespace Editor
	{
		class FDebugControlsTool;
		class FSceneStateBlueprintEditor;
	}

	namespace Graph
	{
		struct FBlueprintDebugObjectChange;
	}
}

namespace UE::SceneState::Editor
{

/** Widget for displaying the controls for the debugged scene state object */
class SDebugControls : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDebugControls) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FSceneStateBlueprintEditor>& InBlueprintEditor);

	virtual ~SDebugControls() override;

	/** Updates to the latest blueprint's object being debugged */
	void Refresh();

	//~ Begin SWidget
	virtual void Tick(const FGeometry& InAllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	//~ End SWidget

private:
	/** Creates the widget to display when the tool is available */
	TSharedRef<SWidget> CreateContentWidget();

	/** Creates the widget to display when the tool is not available */
	TSharedRef<SWidget> CreatePlaceholderWidget();

	/** Creates a new details view for debug controls */
	TSharedRef<IDetailsView> CreateDebugControlsDetailsView(const TSharedRef<FSceneStateBlueprintEditor>& InBlueprintEditor) const;

	/** Creates a new details view for debugged object */
	TSharedRef<IDetailsView> CreateDebuggedObjectDetailsView(const TSharedRef<FSceneStateBlueprintEditor>& InBlueprintEditor) const;

	/** Called when the blueprint debug object has changed */
	void OnBlueprintDebugObjectChanged(const Graph::FBlueprintDebugObjectChange& InChange);

	/** Details view of the debug controls object */
	TSharedPtr<IDetailsView> DebugControlsDetailsView;

	/** Details view of the debugged object */
	TSharedPtr<IDetailsView> DebuggedObjectDetailsView;

	/** The blueprint editor owning this widget */
	TWeakPtr<FSceneStateBlueprintEditor> BlueprintEditorWeak;

	/** Container for the widget with the actual content */
	TSharedPtr<SBorder> WidgetContainer;

	/** Debug control tools content widget when the tool is available */
	TSharedPtr<SWidget> ContentWidget;

	/** Placeholder to use when the debug control tool is not available */
	TSharedPtr<SWidget> PlaceholderWidget;

	/** Class that handles the logic of this widget */
	TSharedPtr<FDebugControlsTool> DebugControlsTool;

	/** Handle to the blueprint debug object changed delegate */
	FDelegateHandle OnBlueprintDebugObjectChangedHandle;
};

} // UE::SceneState::Editor
