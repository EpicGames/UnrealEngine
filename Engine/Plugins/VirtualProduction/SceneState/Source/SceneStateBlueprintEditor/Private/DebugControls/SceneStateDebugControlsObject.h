// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneStateEventTemplate.h"
#include "UObject/Object.h"
#include "SceneStateDebugControlsObject.generated.h"

class USceneStateObject;

/** Temporary object used for debug controls */
UCLASS(Transient)
class USceneStateDebugControlsObject : public UObject
{
	GENERATED_BODY()

public:
	/** Debug events to push */
	UPROPERTY(EditAnywhere, Category="Event")
	TArray<FSceneStateEventTemplate> Events;
};
