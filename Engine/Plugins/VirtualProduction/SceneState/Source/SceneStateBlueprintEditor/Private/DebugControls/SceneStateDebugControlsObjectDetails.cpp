// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateDebugControlsObjectDetails.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "SceneStateDebugControlsObject.h"
#include "SceneStateEventUtils.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SceneStateDebugControlsObjectDetails"

namespace UE::SceneState::Editor
{

void FDebugControlsObjectDetails::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	DebugControls = InDetailBuilder.GetObjectsOfTypeBeingCustomized<USceneStateDebugControlsObject>();
	CustomizeEventDetails(InDetailBuilder);
}

void FDebugControlsObjectDetails::CustomizeEventDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	TSharedRef<IPropertyHandle> EventsHandle = InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(USceneStateDebugControlsObject, Events));
	EventsHandle->MarkHiddenByCustomization();

	IDetailCategoryBuilder& EventsCategory = InDetailBuilder.EditCategory(TEXT("Events"));

	TSharedRef<SWidget> HeaderContentWidget = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(EventsCategory.GetDisplayName())
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		[
			EventsHandle->CreateDefaultPropertyButtonWidgets()
		];

	EventsCategory.HeaderContent(HeaderContentWidget, /*bWholeRowContent*/true);

	TSharedRef<FDetailArrayBuilder> EventsArrayBuilder = MakeShared<FDetailArrayBuilder>(EventsHandle
		, /*GenerateHeader*/false
		, /*DisplayResetToDefault*/false
		, /*bDisplayElementNum*/false);

	TWeakPtr<IPropertyUtilities> PropertyUtilitiesWeak = InDetailBuilder.GetPropertyUtilities();

	EventsArrayBuilder->OnGenerateArrayElementWidget(FOnGenerateArrayElementWidget::CreateLambda(
		[PropertyUtilitiesWeak](TSharedRef<IPropertyHandle> InPropertyHandle, int32 InArrayIndex, IDetailChildrenBuilder& InChildrenBuilder)
		{
			InChildrenBuilder.AddProperty(InPropertyHandle);
		}));

	EventsCategory.AddCustomBuilder(EventsArrayBuilder);
}

} // UE::SceneState::Editor

#undef LOCTEXT_NAMESPACE
