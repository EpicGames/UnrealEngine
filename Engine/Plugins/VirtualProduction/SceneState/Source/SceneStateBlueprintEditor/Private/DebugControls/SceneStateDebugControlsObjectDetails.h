// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "IDetailCustomization.h"
#include "UObject/WeakObjectPtr.h"

class FReply;
class IPropertyHandle;
class USceneStateDebugControlsObject;
class USceneStateObject;
struct FSceneStateEventTemplate;

namespace UE::SceneState::Editor
{

/** Details customization for USceneStateDebugControlsObject */
class FDebugControlsObjectDetails : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FDebugControlsObjectDetails>();
	}

	//~ Begin IDetailCustomization
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder) override;
	//~ End IDetailCustomization

private:
	void CustomizeEventDetails(IDetailLayoutBuilder& InDetailBuilder);

	TArray<TWeakObjectPtr<USceneStateDebugControlsObject>> DebugControls;
};

} // UE::SceneState::Editor
