// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateDebugControlsTabFactory.h"
#include "SSceneStateDebugControls.h"
#include "SceneStateBlueprintEditor.h"
#include "Widgets/SNullWidget.h"

#define LOCTEXT_NAMESPACE "SceneStateDebugControlsTabFactory"

namespace UE::SceneState::Editor
{

const FName FDebugControlsTabFactory::TabId(TEXT("SceneStateDebugControls"));

FDebugControlsTabFactory::FDebugControlsTabFactory(const TSharedRef<FSceneStateBlueprintEditor>& InEditor)
	: FTabFactory(TabId, InEditor)
{
	TabIcon             = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Debug");
	TabLabel            = LOCTEXT("TabLabel", "Debug Controls");
	ViewMenuTooltip     = LOCTEXT("ViewMenuTooltip", "Debug Controls");
	ViewMenuDescription = LOCTEXT("ViewMenuDescription", "Debug Controls");
	bIsSingleton        = true;
}

TSharedRef<SWidget> FDebugControlsTabFactory::CreateTabBody(const FWorkflowTabSpawnInfo& InInfo) const
{
	TSharedPtr<FSceneStateBlueprintEditor> Editor = GetEditor();
	if (!Editor.IsValid())
	{
		return SNullWidget::NullWidget;
	}
	return SNew(SDebugControls, Editor.ToSharedRef());
}

} // UE::SceneState::Editor

#undef LOCTEXT_NAMESPACE
