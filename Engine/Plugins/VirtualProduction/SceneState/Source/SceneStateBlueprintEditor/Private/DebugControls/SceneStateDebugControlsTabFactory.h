// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TabFactories/SceneStateTabFactory.h"

namespace UE::SceneState::Editor
{

class FDebugControlsTabFactory : public FTabFactory
{
public:
	static const FName TabId;

	explicit FDebugControlsTabFactory(const TSharedRef<FSceneStateBlueprintEditor>& InEditor);

	//~ Begin FWorkflowTabFactory
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& InInfo) const override;
	//~ End FWorkflowTabFactory
};

} // UE::SceneState::Editor
