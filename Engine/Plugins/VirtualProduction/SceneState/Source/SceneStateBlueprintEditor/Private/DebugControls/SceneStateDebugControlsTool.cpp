// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateDebugControlsTool.h"
#include "DebugExecutors/SceneStateDebugStateExecutor.h"
#include "DebugExecutors/SceneStateDebugTaskExecutor.h"
#include "Nodes/SceneStateMachineStateNode.h"
#include "SceneStateBlueprint.h"
#include "SceneStateBlueprintDelegates.h"
#include "SceneStateBlueprintEditor.h"
#include "SceneStateBlueprintEditorCommands.h"
#include "SceneStateDebugControlsObject.h"
#include "SceneStateEventUtils.h"
#include "SceneStateObject.h"

namespace UE::SceneState::Editor
{

FDebugControlsTool::FDebugControlsTool(const TSharedRef<FSceneStateBlueprintEditor>& InBlueprintEditor)
	: BlueprintEditorWeak(InBlueprintEditor)
	, ToolCommandList(MakeShared<FUICommandList>())
{
	ToolCommandList->Append(InBlueprintEditor->GetToolkitCommands());
}

bool FDebugControlsTool::IsAvailable() const
{
	const USceneStateObject* DebuggedObject = GetDebuggedObject();
	return DebuggedObject && DebuggedObject->IsActive();
}

void FDebugControlsTool::Initialize()
{
	DebugControlsObject.Reset(NewObject<USceneStateDebugControlsObject>(GetTransientPackage(), NAME_None, RF_Transient | RF_Transactional));
	BindCommands();
}

void FDebugControlsTool::Tick(float InDeltaSeconds)
{
	if (DebugExecutor.IsValid())
	{
		DebugExecutor->Tick(InDeltaSeconds);
	}
}

UObject* FDebugControlsTool::GetDebugControlsObject() const
{
	return DebugControlsObject.Get();
}

USceneStateObject* FDebugControlsTool::GetDebuggedObject() const
{
	if (TSharedPtr<FSceneStateBlueprintEditor> BlueprintEditor = BlueprintEditorWeak.Pin())
	{
		if (UBlueprint* Blueprint = BlueprintEditor->GetBlueprintObj())
		{
			return Cast<USceneStateObject>(Blueprint->GetObjectBeingDebugged());
		}
	}
	return nullptr;
}

const USceneStateMachineNode* FDebugControlsTool::GetSingleSelectedNode() const
{
	if (TSharedPtr<FSceneStateBlueprintEditor> BlueprintEditor = BlueprintEditorWeak.Pin())
	{
		return Cast<USceneStateMachineNode>(BlueprintEditor->GetSingleSelectedNode());
	}
	return nullptr;
}

void FDebugControlsTool::BindCommands()
{
	const FBlueprintEditorCommands& EditorCommands = FBlueprintEditorCommands::Get();

	ToolCommandList->MapAction(EditorCommands.DebugPushEvent, FExecuteAction::CreateSP(this, &FDebugControlsTool::PushEvents));
	ToolCommandList->MapAction(EditorCommands.DebugRunSelection
		, FExecuteAction::CreateSP(this, &FDebugControlsTool::DebugRunSelection)
		, FCanExecuteAction::CreateSP(this, &FDebugControlsTool::CanDebugRunSelection));
}

bool FDebugControlsTool::CanDebugRunSelection()
{
	USceneStateObject* const DebuggedObject = GetDebuggedObject();
	if (!DebuggedObject || !DebuggedObject->IsActive())
	{
		return false;
	}

	if (const USceneStateMachineNode* SelectedNode = GetSingleSelectedNode())
	{
		const Graph::EStateMachineNodeType SelectedNodeType = SelectedNode->GetNodeType();
		return SelectedNodeType == Graph::EStateMachineNodeType::State
			|| SelectedNodeType == Graph::EStateMachineNodeType::Task;
	}
	return false;
}

void FDebugControlsTool::DebugRunSelection()
{
	DebugExecutor.Reset();

	USceneStateObject* const DebuggedObject = GetDebuggedObject();
	if (!DebuggedObject || !DebuggedObject->IsActive())
	{
		return;
	}

	const USceneStateMachineNode* SelectedNode = GetSingleSelectedNode();
	if (!SelectedNode)
	{
		return;
	}

	switch (SelectedNode->GetNodeType())
	{
	case Graph::EStateMachineNodeType::State:
		DebugExecutor = MakeShared<FDebugStateExecutor>(DebuggedObject, SelectedNode);
		break;

	case Graph::EStateMachineNodeType::Task:
		DebugExecutor = MakeShared<FDebugTaskExecutor>(DebuggedObject, SelectedNode);
		break;
	}

	if (DebugExecutor.IsValid())
	{
		DebugExecutor->Start();
	}
}

void FDebugControlsTool::PushEvents()
{
	USceneStateObject* const DebuggedObject = GetDebuggedObject();
	if (!DebuggedObject || !DebugControlsObject.IsValid())
	{
		return;
	}

	USceneStateEventStream* const EventStream = DebuggedObject->GetEventStream();
	if (!EventStream)
	{
		return;
	}

	for (const FSceneStateEventTemplate& Event : DebugControlsObject->Events)
	{
		PushEvent(EventStream, Event);
	}
}

} // UE::SceneState::Editor
