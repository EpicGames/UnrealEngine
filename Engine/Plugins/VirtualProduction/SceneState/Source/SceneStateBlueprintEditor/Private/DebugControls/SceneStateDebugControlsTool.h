// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/StrongObjectPtrTemplates.h"

class FUICommandList;
class USceneStateDebugControlsObject;
class USceneStateMachineNode;
class USceneStateObject;

namespace UE::SceneState
{
	namespace Editor
	{
		class FDebugExecutor;
		class FSceneStateBlueprintEditor;
	}

	namespace Graph
	{
		struct FBlueprintDebugObjectChange;
	}
}

namespace UE::SceneState::Editor
{

/** Logic class for the debug controls widget */
class FDebugControlsTool : public TSharedFromThis<FDebugControlsTool>
{
public:
	explicit FDebugControlsTool(const TSharedRef<FSceneStateBlueprintEditor>& InBlueprintEditor);

	const TSharedRef<FUICommandList>& GetCommandList() const
	{
		return ToolCommandList;
	}

	/** Returns whether the tool is ready to be used, by checking if the currently debugged object is active */
	bool IsAvailable() const;

	/** Creates the debug controls objects and binds the command actions */
	void Initialize();

	/** Called to update the active debug executor if any */
	void Tick(float InDeltaSeconds);

	/** Gets the debug controls object to show in details view */
	UObject* GetDebugControlsObject() const;

	/** Retrieves the currently debugged object */
	USceneStateObject* GetDebuggedObject() const;

private:
	/** Gets the single selected state machine node, or null if nothing is selected or multiple items are selected */
	const USceneStateMachineNode* GetSingleSelectedNode() const;

	/** Adds the debug controls commands to the command list */
	void BindCommands();

	/** Whether there's any selected elements in the graph that can be debug executed. Only supports 1 selected element at a time */
	bool CanDebugRunSelection();

	/** Runs the selected element in standalone*/
	void DebugRunSelection();

	/** Push the events in the debug controls object */
	void PushEvents();

	/** Blueprint editor owning this tool */
	TWeakPtr<FSceneStateBlueprintEditor> BlueprintEditorWeak;

	/** Command list holding all the debug controls commands */
	TSharedRef<FUICommandList> ToolCommandList;

	/** Current active executor */
	TSharedPtr<FDebugExecutor> DebugExecutor;

	/** Object to add additional controls while also supporting transaction */
	TStrongObjectPtr<USceneStateDebugControlsObject> DebugControlsObject;
};

} // UE::SceneState::Editor
