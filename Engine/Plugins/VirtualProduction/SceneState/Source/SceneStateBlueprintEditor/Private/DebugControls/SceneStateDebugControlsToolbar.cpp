// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateDebugControlsToolbar.h"
#include "SceneStateBlueprintEditor.h"
#include "SceneStateBlueprintEditorCommands.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"
#include "Widgets/SNullWidget.h"

namespace UE::SceneState::Editor
{

namespace Private
{

static const FLazyName DebugControlsToolbarName = TEXT("SceneStateDebugControlsToolbar");

void ConditionallyRegisterToolbar()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (!ToolMenus || ToolMenus->IsMenuRegistered(DebugControlsToolbarName))
	{
		return;
	}

	UToolMenu* const ToolbarMenu = ToolMenus->RegisterMenu(DebugControlsToolbarName, NAME_None, EMultiBoxType::SlimHorizontalToolBar);
	if (!ToolbarMenu)
	{
		return;
	}

	const FBlueprintEditorCommands& EditorCommands = FBlueprintEditorCommands::Get();

	FToolMenuSection& ControlsSection = ToolbarMenu->FindOrAddSection(TEXT("ControlsSection"));
	ControlsSection.AddEntry(FToolMenuEntry::InitToolBarButton(EditorCommands.DebugRunSelection));
	ControlsSection.AddEntry(FToolMenuEntry::InitToolBarButton(EditorCommands.DebugPushEvent));
}

} // Private

TSharedRef<SWidget> CreateDebugControlsToolbar(const TSharedRef<FUICommandList>& InCommandList)
{
	// Register toolbar if it hasn't been registered yet
	Private::ConditionallyRegisterToolbar();

	UToolMenus* const ToolMenus = UToolMenus::Get();
	if (!ToolMenus)
	{
		return SNullWidget::NullWidget;
	}

	FToolMenuContext Context(InCommandList);
	return ToolMenus->GenerateWidget(Private::DebugControlsToolbarName, Context);
}

} // UE::SceneState::Editor
