// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointerFwd.h"

class FUICommandList;
class SWidget;

namespace UE::SceneState::Editor
{
	/** Creates the toolbar widget for the debug controls tab */
	TSharedRef<SWidget> CreateDebugControlsToolbar(const TSharedRef<FUICommandList>& InCommandList);

} // UE::SceneState::Editor
