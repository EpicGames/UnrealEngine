// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateDebugExecutor.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Nodes/SceneStateMachineNode.h"
#include "SceneStateBlueprint.h"

namespace UE::SceneState::Editor
{

FDebugExecutor::FDebugExecutor(USceneStateObject* InRootObject, const USceneStateMachineNode* InNode)
	: NodeKey(InNode)
{
	ExecutionContext.Setup(InRootObject);
}

FDebugExecutor::~FDebugExecutor()
{
	Exit();
}

void FDebugExecutor::Start()
{
	OnStart(ExecutionContext);
}

void FDebugExecutor::Tick(float InDeltaSeconds)
{
	OnTick(ExecutionContext, InDeltaSeconds);
}

void FDebugExecutor::Exit()
{
	OnExit(ExecutionContext);
	ExecutionContext.Reset();
}

FString FDebugExecutor::GetReferencerName() const
{
	return TEXT("SceneStateDebugExecutor");
}

void FDebugExecutor::AddReferencedObjects(FReferenceCollector& InCollector)
{
	InCollector.AddPropertyReferencesWithStructARO(FSceneStateExecutionContext::StaticStruct(), &ExecutionContext);
}

} // UE::SceneState::Editor
