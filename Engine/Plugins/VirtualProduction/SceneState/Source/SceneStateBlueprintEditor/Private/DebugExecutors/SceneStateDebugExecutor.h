// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneStateExecutionContext.h"
#include "Templates/SharedPointer.h"
#include "UObject/GCObject.h"
#include "UObject/ObjectKey.h"

class USceneStateMachineNode;

namespace UE::SceneState::Editor
{

/** Base class to execute */
class FDebugExecutor : public FGCObject, public TSharedFromThis<FDebugExecutor>
{
public:
	explicit FDebugExecutor(USceneStateObject* InRootObject, const USceneStateMachineNode* InNode);

	virtual ~FDebugExecutor() override;

	/** Called to start the executor */
	void Start();

	/** Called every tick to update the executor */
	void Tick(float InDeltaSeconds);

	/** Called to exit the executor */
	void Exit();

protected:
	FObjectKey GetNodeKey() const
	{
		return NodeKey;
	}

	/** Called to start the executor */
	virtual void OnStart(const FSceneStateExecutionContext& InExecutionContext)
	{
	}

	/** Called every tick to update the executor */
	virtual void OnTick(const FSceneStateExecutionContext& InExecutionContext, float InDeltaSeconds)
	{
	}

	/** Called to exit the executor */
	virtual void OnExit(const FSceneStateExecutionContext& InExecutionContext)
	{
	}

	//~ Begin FGCObject
	virtual FString GetReferencerName() const override;
	virtual void AddReferencedObjects(FReferenceCollector& InCollector) override;
	//~ End FGCObject

private:
	/** The object key of the state machine node. Used to get the corresponding element in the generated class */
	FObjectKey NodeKey;

	/** Execution context to use */
	FSceneStateExecutionContext ExecutionContext;
};

} // UE::SceneState::Editor
