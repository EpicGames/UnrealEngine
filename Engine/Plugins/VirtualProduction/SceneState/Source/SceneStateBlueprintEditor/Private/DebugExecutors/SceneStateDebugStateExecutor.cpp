// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateDebugStateExecutor.h"
#include "SceneStateTemplateData.h"

namespace UE::SceneState::Editor
{

const FSceneState* FDebugStateExecutor::GetState(const FSceneStateExecutionContext& InExecutionContext) const
{
	if (const USceneStateTemplateData* TemplateData = InExecutionContext.GetTemplateData())
	{
		return TemplateData->FindStateFromNode(GetNodeKey());
	}
	return nullptr;
}

void FDebugStateExecutor::OnStart(const FSceneStateExecutionContext& InExecutionContext)
{
	if (const FSceneState* State = GetState(InExecutionContext))
	{
		State->Enter(InExecutionContext);
		ConditionallyExit(InExecutionContext, *State);
	}
}

void FDebugStateExecutor::OnTick(const FSceneStateExecutionContext& InExecutionContext, float InDeltaSeconds)
{
	if (const FSceneState* State = GetState(InExecutionContext))
	{
		State->Tick(InExecutionContext, InDeltaSeconds);
		ConditionallyExit(InExecutionContext, *State);
	}
}

void FDebugStateExecutor::OnExit(const FSceneStateExecutionContext& InExecutionContext)
{
	if (const FSceneState* State = GetState(InExecutionContext))
	{
		State->Exit(InExecutionContext);
	}
}

void FDebugStateExecutor::ConditionallyExit(const FSceneStateExecutionContext& InExecutionContext, const FSceneState& InState)
{
	if (!InState.HasPendingTasks(InExecutionContext))
	{
		Exit();
	}
}

} // UE::SceneState::Editor
