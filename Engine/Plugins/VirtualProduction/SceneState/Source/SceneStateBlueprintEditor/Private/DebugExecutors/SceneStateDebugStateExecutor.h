// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneStateDebugExecutor.h"

class USceneStateMachineStateNode;

namespace UE::SceneState::Editor
{

/** Debug executes the state corresponding to a given state node */
class FDebugStateExecutor : public FDebugExecutor
{
public:
	explicit FDebugStateExecutor(USceneStateObject* InRootObject, const USceneStateMachineNode* InNode)
		: FDebugExecutor(InRootObject, InNode)
	{
	}

protected:
	/** Retrieves the state corresponding to the stored state node */
	const FSceneState* GetState(const FSceneStateExecutionContext& InExecutionContext) const;

	//~ Begin FDebugExecutor
	virtual void OnStart(const FSceneStateExecutionContext& InExecutionContext) override;
	virtual void OnTick(const FSceneStateExecutionContext& InExecutionContext, float InDeltaSeconds) override;
	virtual void OnExit(const FSceneStateExecutionContext& InExecutionContext) override;
	//~ End FDebugExecutor

private:
	/** Checks whether the state has all pending tasks completed and calls exit if so */
	void ConditionallyExit(const FSceneStateExecutionContext& InExecutionContext, const FSceneState& InState);
};

} // UE::SceneState::Editor
