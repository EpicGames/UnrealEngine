// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateDebugTaskExecutor.h"
#include "Nodes/SceneStateMachineStateNode.h"
#include "Nodes/SceneStateMachineTaskNode.h"
#include "SceneStateMachineGraphSchema.h"
#include "SceneStateTemplateData.h"
#include "Tasks/SceneStateTaskInstance.h"

namespace UE::SceneState::Editor
{

const FSceneStateTask* FDebugTaskExecutor::GetTask(const FSceneStateExecutionContext& InExecutionContext) const
{
	if (const USceneStateTemplateData* TemplateData = InExecutionContext.GetTemplateData())
	{
		return TemplateData->FindTaskFromNode(GetNodeKey());
	}
	return nullptr;
}

const FSceneState* FDebugTaskExecutor::GetState(TNotNull<const USceneStateTemplateData*> InTemplateData) const
{
	if (const USceneStateMachineTaskNode* const TaskNode = Cast<USceneStateMachineTaskNode>(GetNodeKey().ResolveObjectPtr()))
	{
		if (const USceneStateMachineStateNode* const StateNode = USceneStateMachineGraphSchema::FindConnectedStateNode(TaskNode))
		{
			return InTemplateData->FindStateFromNode(StateNode);
		}
	}
	return nullptr;
}

void FDebugTaskExecutor::Setup(const FSceneStateExecutionContext& InExecutionContext, const FSceneStateTask& InTask)
{
	const USceneStateTemplateData* TemplateData = InExecutionContext.GetTemplateData();
	if (!TemplateData)
	{
		return;
	}

	const FSceneState* ParentState = GetState(TemplateData);
	if (!ParentState)
	{
		return;
	}

	const FSceneStateRange TaskRange = ParentState->GetTaskRange(); 

	const uint16 TaskAbsoluteIndex = InTask.GetTaskIndex();
	const uint16 TaskRelativeIndex = TaskAbsoluteIndex - TaskRange.Index;

	// Make an empty task instance array that matches the state task count,
	// and only set the task instance template for the task this executor cares about
	TArray<FConstStructView> TemplateTaskInstances;
	TemplateTaskInstances.SetNum(TaskRange.Count);
	TemplateTaskInstances[TaskRelativeIndex] = TemplateData->TaskInstances[TaskAbsoluteIndex];

	ParentState->AllocateTaskInstances(InExecutionContext, TemplateTaskInstances);
}

void FDebugTaskExecutor::OnStart(const FSceneStateExecutionContext& InExecutionContext)
{
	if (const FSceneStateTask* Task = GetTask(InExecutionContext))
	{
		Setup(InExecutionContext, *Task);

		const FStructView TaskInstance = InExecutionContext.FindTaskInstance(*Task);
		Task->Setup(InExecutionContext, TaskInstance);
		Task->Start(InExecutionContext, TaskInstance);
		ConditionallyExit(*Task, InExecutionContext);
	}
}

void FDebugTaskExecutor::OnTick(const FSceneStateExecutionContext& InExecutionContext, float InDeltaSeconds)
{
	if (const FSceneStateTask* Task = GetTask(InExecutionContext))
	{
		const FStructView TaskInstance = InExecutionContext.FindTaskInstance(*Task);
		Task->Tick(InExecutionContext, TaskInstance, InDeltaSeconds);
		ConditionallyExit(*Task, InExecutionContext);
	}
}

void FDebugTaskExecutor::OnExit(const FSceneStateExecutionContext& InExecutionContext)
{
	if (const FSceneStateTask* Task = GetTask(InExecutionContext))
	{
		const FStructView TaskInstance = InExecutionContext.FindTaskInstance(*Task);
		Task->Stop(InExecutionContext, TaskInstance, ESceneStateTaskStopReason::State);
	}
}

void FDebugTaskExecutor::ConditionallyExit(const FSceneStateTask& InTask, const FSceneStateExecutionContext& InExecutionContext)
{
	const TOptional<UE::SceneState::EExecutionStatus> TaskStatus = InExecutionContext.GetTaskStatus(InTask);
	if (TaskStatus.IsSet() && *TaskStatus == EExecutionStatus::Finished)
	{
		Exit();
	}
}

} // UE::SceneState::Editor
