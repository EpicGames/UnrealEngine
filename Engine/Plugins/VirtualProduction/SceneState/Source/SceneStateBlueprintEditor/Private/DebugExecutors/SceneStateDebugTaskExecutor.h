// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneStateDebugExecutor.h"

namespace UE::SceneState::Editor
{

/** Debug executes the task corresponding to a given task node */
class FDebugTaskExecutor : public FDebugExecutor
{
public:
	explicit FDebugTaskExecutor(USceneStateObject* InRootObject, const USceneStateMachineNode* InTaskNode)
		: FDebugExecutor(InRootObject, InTaskNode)
	{
	}

protected:
	/** Retrieves the task corresponding to the stored task node */
	const FSceneStateTask* GetTask(const FSceneStateExecutionContext& InExecutionContext) const;

	/** Retrieves the state corresponding to the stored state node */
	const FSceneState* GetState(TNotNull<const USceneStateTemplateData*> InTemplateData) const;

	/** Called to allocate the task instance */
	void Setup(const FSceneStateExecutionContext& InExecutionContext, const FSceneStateTask& InTask);

	//~ Begin FDebugExecutor
	virtual void OnStart(const FSceneStateExecutionContext& InExecutionContext) override;
	virtual void OnTick(const FSceneStateExecutionContext& InExecutionContext, float InDeltaSeconds) override;
	virtual void OnExit(const FSceneStateExecutionContext& InExecutionContext) override;
	//~ End FDebugExecutor

private:
	/** Checks whether the task completed and calls exit if so */
	void ConditionallyExit(const FSceneStateTask& InTask, const FSceneStateExecutionContext& InExecutionContext);
};

} // UE::SceneState::Editor
