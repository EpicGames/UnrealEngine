// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSceneStateDebugView.h"
#include "ISceneStateContextEditor.h"
#include "SceneStateBlueprint.h"
#include "SceneStateBlueprintDelegates.h"
#include "SceneStateBlueprintEditor.h"
#include "SceneStateBlueprintEditorModule.h"
#include "SceneStateObject.h"
#include "Styling/SlateStyleMacros.h"

#define LOCTEXT_NAMESPACE "SSceneStateDebugView"

namespace UE::SceneState::Editor
{

void SDebugView::Construct(const FArguments& InArgs, const TSharedRef<FSceneStateBlueprintEditor>& InBlueprintEditor)
{
	BlueprintEditorWeak = InBlueprintEditor;

	OnBlueprintDebugObjectChangedHandle = Graph::OnBlueprintDebugObjectChanged.AddSP(this, &SDebugView::OnBlueprintDebugObjectChanged);

	ChildSlot
	[
		SAssignNew(ViewContainer, SBorder)
		.BorderImage(FAppStyle::GetBrush("WhiteBorder"))
		.BorderBackgroundColor(FLinearColor::Black)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.Padding(0)
		[
			SAssignNew(PlaceholderWidget, SBox)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("PlaceholderTitle", "Select a debug object supporting debug view"))
				.Font(DEFAULT_FONT("Italic", 10))
				.ColorAndOpacity(FLinearColor::White)
			]
		]
	];

	Refresh();
}

SDebugView::~SDebugView()
{
	Graph::OnBlueprintDebugObjectChanged.Remove(OnBlueprintDebugObjectChangedHandle);
	OnBlueprintDebugObjectChangedHandle.Reset();
}

void SDebugView::Refresh()
{
	if (TSharedPtr<SWidget> ViewWidget = GetViewWidget())
	{
		ViewContainer->SetContent(ViewWidget.ToSharedRef());
	}
	else
	{
		ViewContainer->SetContent(PlaceholderWidget.ToSharedRef());
	}
}

TSharedPtr<SWidget> SDebugView::GetViewWidget() const
{
	USceneStateObject* const DebuggedObject = GetDebuggedObject();
	if (!DebuggedObject)
	{
		return nullptr;
	}

	UObject* const ContextObject = DebuggedObject->GetContextObject();
	if (!ContextObject)
	{
		return nullptr;
	}

	const FContextEditorRegistry& ContextEditorRegistry = FBlueprintEditorModule::GetInternal().GetContextEditorRegistry();

	const TSharedPtr<IContextEditor> ContextEditor = ContextEditorRegistry.FindContextEditor(ContextObject);
	if (!ContextEditor.IsValid())
	{
		return nullptr;
	}

	IContextEditor::FContextParams ContextParams;
	ContextParams.ContextObject = ContextObject;
	return ContextEditor->CreateViewWidget(ContextParams);
}

void SDebugView::OnBlueprintDebugObjectChanged(const Graph::FBlueprintDebugObjectChange& InChange)
{
	TSharedPtr<FSceneStateBlueprintEditor> BlueprintEditor = BlueprintEditorWeak.Pin();
	if (BlueprintEditor.IsValid() && BlueprintEditor->GetBlueprintObj() == InChange.Blueprint)
	{
		Refresh();
	}
}

USceneStateObject* SDebugView::GetDebuggedObject() const
{
	if (TSharedPtr<FSceneStateBlueprintEditor> BlueprintEditor = BlueprintEditorWeak.Pin())
	{
		if (UBlueprint* Blueprint = BlueprintEditor->GetBlueprintObj())
		{
			return Cast<USceneStateObject>(Blueprint->GetObjectBeingDebugged());
		}
	}
	return nullptr;
}

} // UE::SceneState::Editor

#undef LOCTEXT_NAMESPACE
