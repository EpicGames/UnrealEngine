// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "Widgets/SCompoundWidget.h"

class USceneStateObject;
class SBorder;

namespace UE::SceneState
{
	namespace Editor
	{
		class FSceneStateBlueprintEditor;
	}

	namespace Graph
	{
		struct FBlueprintDebugObjectChange;
	}
}

namespace UE::SceneState::Editor
{

/** Widget for displaying the currently debugged object view */
class SDebugView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDebugView) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FSceneStateBlueprintEditor>& InBlueprintEditor);

	virtual ~SDebugView() override;

	/** Updates to the latest blueprint's object being debugged */
	void Refresh();

private:
	/** Gets the widget to display */
	TSharedPtr<SWidget> GetViewWidget() const;

	/** Called when the blueprint debug object has changed */
	void OnBlueprintDebugObjectChanged(const Graph::FBlueprintDebugObjectChange& InChange);

	/** Retrieves the currently debugged object */
	USceneStateObject* GetDebuggedObject() const;

	/** The blueprint editor owning this widget */
	TWeakPtr<FSceneStateBlueprintEditor> BlueprintEditorWeak;

	/** The placeholder widget to use when no valid view widget is found */
	TSharedPtr<SWidget> PlaceholderWidget;

	/** Widget containing the view */
	TSharedPtr<SBorder> ViewContainer;

	/** Handle to the blueprint debug object changed delegate */
	FDelegateHandle OnBlueprintDebugObjectChangedHandle;
};

} // UE::SceneState::Editor
