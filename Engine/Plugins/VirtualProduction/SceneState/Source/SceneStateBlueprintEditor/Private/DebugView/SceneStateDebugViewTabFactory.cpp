// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateDebugViewTabFactory.h"
#include "SSceneStateDebugView.h"
#include "SceneStateBlueprintEditor.h"
#include "Widgets/SNullWidget.h"

#define LOCTEXT_NAMESPACE "SceneStateDebugViewTabFactory"

namespace UE::SceneState::Editor
{

const FName FDebugViewTabFactory::TabId(TEXT("SceneStateDebugView"));

FDebugViewTabFactory::FDebugViewTabFactory(const TSharedRef<FSceneStateBlueprintEditor>& InEditor)
	: FTabFactory(TabId, InEditor)
{
	TabIcon             = FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Viewports");
	TabLabel            = LOCTEXT("TabLabel", "Debug View");
	ViewMenuTooltip     = LOCTEXT("ViewMenuTooltip", "Debug View");
	ViewMenuDescription = LOCTEXT("ViewMenuDescription", "Debug View");
	bIsSingleton        = true;
}

TSharedRef<SWidget> FDebugViewTabFactory::CreateTabBody(const FWorkflowTabSpawnInfo& InInfo) const
{
	TSharedPtr<FSceneStateBlueprintEditor> Editor = GetEditor();
	if (!Editor.IsValid())
	{
		return SNullWidget::NullWidget;
	}
	return SNew(SDebugView, Editor.ToSharedRef());
}

} // UE::SceneState::Editor

#undef LOCTEXT_NAMESPACE
