// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TabFactories/SceneStateTabFactory.h"

namespace UE::SceneState::Editor
{

class FDebugViewTabFactory : public FTabFactory
{
public:
	static const FName TabId;

	explicit FDebugViewTabFactory(const TSharedRef<FSceneStateBlueprintEditor>& InEditor);

	//~ Begin FWorkflowTabFactory
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& InInfo) const override;
	//~ End FWorkflowTabFactory
};

} // UE::SceneState::Editor
