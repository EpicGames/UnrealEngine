// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateBindingFunctionCustomization.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "InstancedStructDetails.h"
#include "PropertyHandle.h"
#include "SceneStateBindingFunction.h"
#include "SceneStateBlueprintEditorUtils.h"
#include "SceneStateCachedBindingData.h"

namespace UE::SceneState::Editor
{

namespace Private
{
/** Details customization for function instances - hiding the output property and assigning the function id to the rest */
class FFunctionInstanceDetails : public FInstancedStructDataDetails
{
public:
	FFunctionInstanceDetails(TSharedPtr<IPropertyHandle> InStructProperty, const FGuid& InFunctionId)
		: FInstancedStructDataDetails(InStructProperty)
		, FunctionId(InFunctionId)
	{
	}

	//~ Begin FInstancedStructDataDetails
	virtual void OnChildRowAdded(IDetailPropertyRow& InChildRow) override
	{
		TSharedPtr<IPropertyHandle> ChildPropHandle = InChildRow.GetPropertyHandle();
		check(ChildPropHandle.IsValid());
		if (UE::SceneState::Editor::IsOutputProperty(ChildPropHandle->GetProperty()))
		{
			InChildRow.Visibility(EVisibility::Collapsed);
		}
		else
		{
			AssignBindingId(ChildPropHandle.ToSharedRef(), FunctionId);
		}
	}
	//~ End FInstancedStructDataDetails

private:
	FGuid FunctionId;
};

} // Private
	
void FBindingFunctionCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
}

void FBindingFunctionCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	const TSharedRef<IPropertyHandle> FunctionIdHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FSceneStateBindingFunction, FunctionId)).ToSharedRef();
	const TSharedRef<IPropertyHandle> FunctionHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FSceneStateBindingFunction, Function)).ToSharedRef();
	const TSharedRef<IPropertyHandle> FunctionInstanceHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FSceneStateBindingFunction, FunctionInstance)).ToSharedRef();

	FunctionIdHandle->MarkHiddenByCustomization();
	FunctionHandle->MarkHiddenByCustomization();
	FunctionInstanceHandle->MarkHiddenByCustomization();

	FGuid FunctionId;
	GetGuid(FunctionIdHandle, FunctionId);

	InChildBuilder.AddCustomBuilder(MakeShared<FInstancedStructDataDetails>(FunctionHandle));
	InChildBuilder.AddCustomBuilder(MakeShared<Private::FFunctionInstanceDetails>(FunctionInstanceHandle, FunctionId));
}

} // UE::SceneState::Editor
