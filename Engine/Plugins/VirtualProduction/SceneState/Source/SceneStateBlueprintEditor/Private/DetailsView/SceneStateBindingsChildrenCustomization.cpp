// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateBindingsChildrenCustomization.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyBindingExtension.h"
#include "PropertyBindingPath.h"
#include "PropertyHandle.h"
#include "SceneStateBindingCollection.h"
#include "SceneStateBindingCollectionOwner.h"
#include "SceneStateBindingFunction.h"

namespace UE::SceneState::Editor
{

namespace Private
{
/** Finds the bindings owner that this property handle is outered to */
ISceneStateBindingCollectionOwner* GetBindingsOwner(const TSharedPtr<IPropertyHandle>& InPropertyHandle)
{
	if (!InPropertyHandle.IsValid())
	{
		return nullptr;
	}

	TArray<UObject*> OuterObjects;
	InPropertyHandle->GetOuterObjects(OuterObjects);

	auto FindBindingOwner =
		[](UObject* InObject)->ISceneStateBindingCollectionOwner*
		{
			for (UObject* Outer = InObject; Outer; Outer = Outer->GetOuter())
			{
				if (ISceneStateBindingCollectionOwner* BindingOwner = Cast<ISceneStateBindingCollectionOwner>(Outer))
				{
					return BindingOwner;
				}
			}
			return nullptr;
		};

	ISceneStateBindingCollectionOwner* CommonBindingsOwner = nullptr;

	for (UObject* Object : OuterObjects)
	{
		ISceneStateBindingCollectionOwner* const BindingsOwner = FindBindingOwner(Object);
		if (CommonBindingsOwner && CommonBindingsOwner != BindingsOwner)
		{
			return nullptr;
		}
		CommonBindingsOwner = BindingsOwner;
	}

	return CommonBindingsOwner;
}

} // UE::SceneState::Editor::Private

FBindingFunctionStructProvider::FBindingFunctionStructProvider(ISceneStateBindingCollectionOwner* InBindingsOwnerChecked, FPropertyBindingPath InTargetPath)
	: BindingsOwnerWeak(InBindingsOwnerChecked)
	, TargetPath(MoveTemp(InTargetPath))
{
}

bool FBindingFunctionStructProvider::IsValid() const
{
	return GetBindingFunctionView(BindingsOwnerWeak.Get(), TargetPath).IsValid();
}

const UStruct* FBindingFunctionStructProvider::GetBaseStructure() const
{
	return FSceneStateBindingFunction::StaticStruct();
}

void FBindingFunctionStructProvider::GetInstances(TArray<TSharedPtr<FStructOnScope>>& OutInstances, const UStruct* InExpectedBaseStructure) const
{
	if (InExpectedBaseStructure)
	{
		const FStructView BindingFunctionView = GetBindingFunctionView(BindingsOwnerWeak.Get(), TargetPath);

		if (BindingFunctionView.IsValid() && BindingFunctionView.GetScriptStruct()->IsChildOf(InExpectedBaseStructure))
		{
			OutInstances.Add(MakeShared<FStructOnScope>(BindingFunctionView.GetScriptStruct(), BindingFunctionView.GetMemory()));
		}
	}
}

bool FBindingFunctionStructProvider::HasValidBindingFunction(ISceneStateBindingCollectionOwner* InBindingsOwner, const FPropertyBindingPath& InTargetPath)
{
	return GetBindingFunctionView(InBindingsOwner, InTargetPath).IsValid();
}

FStructView FBindingFunctionStructProvider::GetBindingFunctionView(ISceneStateBindingCollectionOwner* InBindingsOwner, const FPropertyBindingPath& InTargetPath)
{
	if (!InBindingsOwner || InTargetPath.IsPathEmpty())
	{
		return FStructView();
	}

	FPropertyBindingBinding* const FoundBinding = InBindingsOwner->GetBindingCollection().GetMutableBindings().FindByPredicate(
		[&InTargetPath](const FSceneStateBinding& InBinding)
		{
			return InBinding.GetTargetPath() == InTargetPath;
		});

	if (!FoundBinding)
	{
		return FStructView();
	}

	const FStructView BindingFunctionView = FoundBinding->GetMutablePropertyFunctionNode();

	const FSceneStateBindingFunction* BindingFunction = BindingFunctionView.GetPtr<FSceneStateBindingFunction>();
	if (BindingFunction && BindingFunction->IsValid())
	{
		return BindingFunctionView;
	}
	return FStructView();
}

bool FBindingsChildrenCustomization::ShouldCustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle)
{
	const ISceneStateBindingCollectionOwner* BindingsOwner = Private::GetBindingsOwner(InPropertyHandle);
	if (!BindingsOwner)
	{
		return false;
	}

	// Bound property's children composition gets overridden.
	FPropertyBindingPath TargetPath;
	UE::PropertyBinding::MakeStructPropertyPathFromPropertyHandle(InPropertyHandle, TargetPath);
	if (TargetPath.IsPathEmpty())
	{
		return false;
	}

	return BindingsOwner->GetBindingCollection().HasBinding(TargetPath);
}

void FBindingsChildrenCustomization::CustomizeChildren(IDetailChildrenBuilder& InChildrenBuilder, TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	ISceneStateBindingCollectionOwner* BindingsOwner = Private::GetBindingsOwner(InPropertyHandle);
	if (!BindingsOwner)
	{
		return;
	}

	FPropertyBindingPath TargetPath;
	UE::PropertyBinding::MakeStructPropertyPathFromPropertyHandle(InPropertyHandle, TargetPath);
	if (!FBindingFunctionStructProvider::HasValidBindingFunction(BindingsOwner, TargetPath))
	{
		return;
	}

	// BindingFunction takes control over property's children composition.
	const TSharedPtr<FBindingFunctionStructProvider> StructProvider = MakeShared<FBindingFunctionStructProvider>(BindingsOwner, TargetPath);

	// Create unique name to persists expansion state.
	const FName UniqueName(LexToString(TargetPath.GetStructID()) + TargetPath.ToString());
	InChildrenBuilder.AddChildStructure(InPropertyHandle.ToSharedRef(), StructProvider, UniqueName);
}

} // UE::SceneState::Editor
