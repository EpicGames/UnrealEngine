// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailPropertyChildrenCustomizationHandler.h"
#include "IStructureDataProvider.h"
#include "PropertyBindingPath.h"
#include "UObject/WeakInterfacePtr.h"

class ISceneStateBindingCollectionOwner;

namespace UE::SceneState::Editor
{

/* Provides the binding function data view for a given target path */
class FBindingFunctionStructProvider : public IStructureDataProvider
{
public:
	explicit FBindingFunctionStructProvider(ISceneStateBindingCollectionOwner* InBindingsOwnerChecked, FPropertyBindingPath InTargetPath);

	//~ Begin IStructureDataProvider
	virtual bool IsValid() const override;;
	virtual const UStruct* GetBaseStructure() const override;
	virtual void GetInstances(TArray<TSharedPtr<FStructOnScope>>& OutInstances, const UStruct* InExpectedBaseStructure) const override;
	//~ End IStructureDataProvider

	/** Returns whether there is a valid binding function at the given target path */
	static bool HasValidBindingFunction(ISceneStateBindingCollectionOwner* InBindingsOwner, const FPropertyBindingPath& InTargetPath);

private:
	/** Retrieves the binding function at the given target path if any*/
	static FStructView GetBindingFunctionView(ISceneStateBindingCollectionOwner* InBindingsOwner, const FPropertyBindingPath& InTargetPath);

	TWeakInterfacePtr<ISceneStateBindingCollectionOwner> BindingsOwnerWeak;
	FPropertyBindingPath TargetPath;
};

/* Overrides bound property's children composition. */
class FBindingsChildrenCustomization : public IDetailPropertyChildrenCustomizationHandler
{
public:
	//~ Begin IDetailPropertyChildrenCustomizationHandler
	virtual bool ShouldCustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle) override;
	virtual void CustomizeChildren(IDetailChildrenBuilder& InChildrenBuilder, TSharedPtr<IPropertyHandle> InPropertyHandle) override;
	//~ End IDetailPropertyChildrenCustomizationHandler
};

} // UE::SceneState::Editor
