// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateBlueprintPropertyReferenceCustomization.h"
#include "DetailWidgetRow.h"
#include "Nodes/SceneStateMachineTaskNode.h"
#include "SPinTypeSelector.h"
#include "SceneStateBlueprintPropertyReference.h"
#include "SceneStatePropertyReferenceUtils.h"
#include "ScopedTransaction.h"
#include "Tasks/SceneStateBlueprintableTask.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SceneStateBlueprintPropertyReferenceCustomization"

bool USceneStatePropertyReferenceSchema::SupportsPinTypeContainer(TWeakPtr<const FEdGraphSchemaAction> InSchemaAction, const FEdGraphPinType& InPinType, const EPinContainerType& InContainerType) const
{
	return InContainerType == EPinContainerType::None || InContainerType == EPinContainerType::Array;
}

namespace UE::SceneState::Editor
{

namespace Private
{

/** Determines if the outer object of the property handle is a Task Node in a State Machine */
bool IsInTaskNode(const TSharedRef<IPropertyHandle>& InPropertyHandle)
{
	TArray<UObject*> OuterObjects;
	InPropertyHandle->GetOuterObjects(OuterObjects);
	for (UObject* OuterObject : OuterObjects)
	{
		if (!OuterObject)
		{
			continue;
		}
		if (OuterObject->IsA<USceneStateMachineTaskNode>())
		{
			return true;
		}
		// Only allow Blueprintable Tasks to show the edit properties when it's the Template (i.e. in its Task Blueprint Editor)
		if (OuterObject->IsA<USceneStateBlueprintableTask>() && !OuterObject->IsTemplate())
		{
			return true;
		}
	}
	return false;
}

} // UE::SceneState::Editor::Private

void FBlueprintPropertyReferenceCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	PropertyHandle = InPropertyHandle;

	// Hide value if the property handle is being shown in a Task node.
	// If there's ever a need to edit reference type for a given reference property, a meta-data (e.g. TaskEditable) can be added to address that specific situation. 
	if (Private::IsInTaskNode(InPropertyHandle))
	{
		InHeaderRow
			.NameContent()
			[
				InPropertyHandle->CreatePropertyNameWidget()
			];
		return;
	}

	InHeaderRow
		.NameContent()
		[
			InPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(FMargin(0, 0, 6, 0))
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ReferenceTo", "Reference to"))
				.Font(IPropertyTypeCustomizationUtils::GetRegularFont())
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				.ToolTipText(LOCTEXT("ReferenceToTooltip", "Specifies the type of the referenced property. The referenced property is bound using property binding in Scene State."))
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SPinTypeSelector, FGetPinTypeTree::CreateUObject(GetDefault<USceneStatePropertyReferenceSchema>(), &USceneStatePropertyReferenceSchema::GetVariableTypeTree))
				.TargetPinType(this, &FBlueprintPropertyReferenceCustomization::GetPinType)
				.OnPinTypeChanged(this, &FBlueprintPropertyReferenceCustomization::OnPinTypeChanged)
				.Schema(GetDefault<USceneStatePropertyReferenceSchema>())
				.Font(IPropertyTypeCustomizationUtils::GetRegularFont())
				.bAllowArrays(true)
			]
		];
}

void FBlueprintPropertyReferenceCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
}

bool FBlueprintPropertyReferenceCustomization::GetPropertyReference(FSceneStateBlueprintPropertyReference& OutPropertyReference) const
{
	if (!PropertyHandle.IsValid())
	{
		return false;
	}

	TOptional<FSceneStateBlueprintPropertyReference> Result;

	PropertyHandle->EnumerateConstRawData(
		[&Result](const void* InRawData, const int32, const int32)->bool
		{
			if (!InRawData)
			{
				Result.Reset();
				return false; // break
			}

			const FSceneStateBlueprintPropertyReference& PropertyReference = *static_cast<const FSceneStateBlueprintPropertyReference*>(InRawData);
			if (Result.IsSet())
			{
				if (*Result != PropertyReference)
				{
					Result.Reset();
					return false; // break, multiple values
				}
			}
			else
			{
				Result = PropertyReference;
			}
			return true; // continue
		});

	if (Result.IsSet())
	{
		OutPropertyReference = *Result;
		return true;
	}
	return false;
}

FEdGraphPinType FBlueprintPropertyReferenceCustomization::GetPinType() const
{
	FSceneStateBlueprintPropertyReference PropertyReference;
	if (!GetPropertyReference(PropertyReference))
	{
		return FEdGraphPinType();
	}

	const TArray<FEdGraphPinType, TInlineAllocator<1>> PinTypes = UE::SceneState::GetPropertyReferencePinTypes(PropertyReference);
	return !PinTypes.IsEmpty()
		? PinTypes[0]
		: FEdGraphPinType();
}

void FBlueprintPropertyReferenceCustomization::OnPinTypeChanged(const FEdGraphPinType& InPinType) const
{
	if (!PropertyHandle.IsValid())
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("SetPropertyReferenceType", "Set Property Reference Type"));

	const FSceneStateBlueprintPropertyReference NewValue = FSceneStateBlueprintPropertyReference::BuildFromPinType(InPinType);

	PropertyHandle->NotifyPreChange();
	PropertyHandle->EnumerateRawData(
		[&NewValue](void* InRawData, const int32, const int32)->bool
		{
			if (InRawData)
			{
				*static_cast<FSceneStateBlueprintPropertyReference*>(InRawData) = NewValue;
			}
			return true; // continue
		});
	PropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	PropertyHandle->NotifyFinishedChangingProperties();
}

} // UE::SceneState::Editor

#undef LOCTEXT_NAMESPACE
