// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraphSchema_K2.h"
#include "IPropertyTypeCustomization.h"
#include "SceneStateBlueprintPropertyReferenceCustomization.generated.h"

struct FSceneStateBlueprintPropertyReference;

/** Property Reference Schema to allow customizing the requirements (e.g. supported containers). */
UCLASS()
class USceneStatePropertyReferenceSchema : public UEdGraphSchema_K2
{
	GENERATED_BODY()

public:
	//~ Begin UEdGraphSchema_K2
	virtual bool SupportsPinTypeContainer(TWeakPtr<const FEdGraphSchemaAction> InSchemaAction, const FEdGraphPinType& InPinType, const EPinContainerType& InContainerType) const override;
	//~ End UEdGraphSchema_K2
};

namespace UE::SceneState::Editor
{

/** Customization for FSceneStateBlueprintPropertyReference */
class FBlueprintPropertyReferenceCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FBlueprintPropertyReferenceCustomization>();
	}

	//~ Begin IPropertyTypeCustomization
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils) override;
	//~ End IPropertyTypeCustomization

private:
	bool GetPropertyReference(FSceneStateBlueprintPropertyReference& OutPropertyReference) const;

	FEdGraphPinType GetPinType() const;

	void OnPinTypeChanged(const FEdGraphPinType& InPinType) const;

	TSharedPtr<IPropertyHandle> PropertyHandle;
};

} // UE::SceneState::Editor
