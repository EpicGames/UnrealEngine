// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateBlueprintableTaskInstanceCustomization.h"
#include "PropertyHandle.h"
#include "SceneStateBlueprintEditorUtils.h"
#include "Tasks/SceneStateBlueprintableTaskWrapper.h"

namespace UE::SceneState::Editor
{

void FBlueprintableTaskInstanceCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
}

void FBlueprintableTaskInstanceCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	TSharedPtr<IPropertyHandle> TaskHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FSceneStateBlueprintableTaskInstance, Task));
	if (TaskHandle.IsValid())
	{
		CustomizeTask(TaskHandle.ToSharedRef(), InChildBuilder);
	}
}

void FBlueprintableTaskInstanceCustomization::CustomizeTask(const TSharedRef<IPropertyHandle>& InTaskHandle, IDetailChildrenBuilder& InChildBuilder)
{
	InTaskHandle->MarkHiddenByCustomization();
	AssignBindingId(InTaskHandle, FindTaskId(InTaskHandle));
	AddObjectProperties(InTaskHandle, InChildBuilder);
}

} // UE::SceneState::Editor
