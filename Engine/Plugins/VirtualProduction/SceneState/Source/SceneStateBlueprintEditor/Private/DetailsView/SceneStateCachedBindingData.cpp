// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateCachedBindingData.h"
#include "EdGraphSchema_K2.h"
#include "IPropertyAccessEditor.h"
#include "Kismet2/StructureEditorUtils.h"
#include "Misc/EnumerateRange.h"
#include "PropertyBindingBindableStructDescriptor.h"
#include "PropertyBindingDataView.h"
#include "SceneStateBindingCollection.h"
#include "SceneStateBindingCollectionOwner.h"
#include "SceneStateBindingDesc.h"
#include "SceneStateBindingFunction.h"
#include "SceneStateBlueprintPropertyReference.h"
#include "SceneStatePropertyReference.h"
#include "SceneStatePropertyReferenceMetadata.h"
#include "SceneStatePropertyReferenceUtils.h"
#include "StructUtils/UserDefinedStruct.h"

#define LOCTEXT_NAMESPACE "SceneStateCachedBindingData"

namespace UE::SceneState::Editor
{

bool IsOutputProperty(const TSharedPtr<IPropertyHandle>& InPropertyHandle)
{
	return InPropertyHandle.IsValid() && InPropertyHandle->HasMetaData(TEXT("Output"));
}

bool IsOutputProperty(const FProperty* InProperty)
{
	return InProperty && InProperty->HasMetaData(TEXT("Output"));
}

const FProperty* FindSingleOutputProperty(const UStruct* InStruct)
{
	const FProperty* OutputProperty = nullptr;

	for (const FProperty* Property : TFieldRange<FProperty>(InStruct, EFieldIterationFlags::IncludeSuper))
	{
		if (IsOutputProperty(Property))
		{
			if (OutputProperty)
			{
				return nullptr;
			}
			OutputProperty = Property;
		}
	}

	return OutputProperty;
}

const FProperty* FSceneStateCachedBindingData::GetProperty() const
{
	if (const IPropertyHandle* PropertyHandlePtr = GetPropertyHandle())
	{
		return PropertyHandlePtr->GetProperty();
	}
	return nullptr;
}

void FSceneStateCachedBindingData::UpdatePropertyReferenceTooltip(const FProperty& InProperty, FTextBuilder& InOutTextBuilder) const
{
	if (InProperty.HasMetaData(Metadata::IsRefToArray))
	{
		InOutTextBuilder.AppendLineFormat(LOCTEXT("PropertyRefBindingTooltipArray", "Supported types are Array of {0}")
			, FText::FromString(InProperty.GetMetaData(Metadata::RefType)));
	}
	else
	{
		InOutTextBuilder.AppendLineFormat(LOCTEXT("PropertyRefBindingTooltip", "Supported types are {0}")
			, FText::FromString(InProperty.GetMetaData(Metadata::RefType)));

		if (InProperty.HasMetaData(Metadata::CanRefToArray))
		{
			InOutTextBuilder.AppendLine(LOCTEXT("PropertyRefBindingTooltipCanSupportArray", "Supports Arrays"));
		}
	}
}
FSceneStateCachedBindingData::FPinInfo FSceneStateCachedBindingData::GetPinInfoForProperty(TNotNull<const FProperty*> InProperty, FPropertyBindingDataView InTargetDataView) const
{
	FPinInfo Result = Super::GetPinInfoForProperty(InProperty, InTargetDataView);
	if (UE::SceneState::IsPropertyReference(InProperty) && InTargetDataView.IsValid())
	{
		// Use internal type to construct PinType if it's property of PropertyRef type.
		TArray<FPropertyBindingPathIndirection> TargetIndirections;
		if (ensure(GetTargetPath().ResolveIndirectionsWithValue(InTargetDataView, TargetIndirections)))
		{
			Result.Type = GetPropertyReferencePinType(InProperty, TargetIndirections.Last().GetPropertyAddress());
		}
		Result.IconBrush = FAppStyle::GetBrush("Kismet.Tabs.Variables");
	}
	return Result;
}

bool FSceneStateCachedBindingData::IsPropertyReference(const FProperty& InProperty)
{
	return UE::SceneState::IsPropertyReference(&InProperty);
}

void FSceneStateCachedBindingData::UpdateSourcePropertyPath(TConstStructView<FPropertyBindingBindableStructDescriptor> InDescriptor, const FPropertyBindingPath& InSourcePath, FString& OutString)
{
	const FSceneStateBindingDesc* Descriptor = InDescriptor.GetPtr<FSceneStateBindingDesc>();
	if (!Descriptor || !Descriptor->Struct)
	{
		return;
	}

	TArray<FPropertyBindingPathIndirection> Indirections;
	if (!InSourcePath.ResolveIndirections(Descriptor->Struct, Indirections))
	{
		return;
	}

	int32 FirstSegment = 0;

	// Making first segment of the path invisible for the user if it's property function's single output property.
	if (Descriptor->DataHandle.IsDataType(ESceneStateDataType::Function) && FindSingleOutputProperty(Descriptor->Struct))
	{
		FirstSegment = 1;
	}

	TStringBuilder<256> Result;
	for (TEnumerateRef<const FPropertyBindingPathSegment> Segment : EnumerateRange(InSourcePath.GetSegments().RightChop(FirstSegment)))
	{
		const int32 SegmentIndex = Segment.GetIndex() + FirstSegment;

		const FPropertyBindingPathIndirection* Indirection = Indirections.FindByPredicate(
			[SegmentIndex](const FPropertyBindingPathIndirection& InIndirection)
			{
				return InIndirection.GetPathSegmentIndex() == SegmentIndex;
			});

		if (!ensure(Indirection))
		{
			return;
		}

		if (SegmentIndex > FirstSegment)
		{
			Result += TEXT(".");
		}

		if (const UUserDefinedStruct* CurrentUserDefinedStruct = Cast<const UUserDefinedStruct>(Indirection->GetContainerStruct()))
		{
			const FString FriendlyName = FStructureEditorUtils::GetVariableFriendlyNameForProperty(CurrentUserDefinedStruct, Indirection->GetProperty());
			if (!FriendlyName.IsEmpty())
			{
				Result += FriendlyName;
			}
			else
			{
				Result += Segment->GetName().ToString();
			}
		}
		else
		{
			Result += Segment->GetName().ToString();
		}

		if (Segment->GetArrayIndex() >= 0)
		{
			Result += FString::Printf(TEXT("[%d]"), Segment->GetArrayIndex());
		}
	}

	OutString = Result;
}

bool FSceneStateCachedBindingData::CanBindToContextStructInternal(const UStruct* InStruct, const int32 InStructIndex)
{
	const FSceneStateBindingDesc& BindingDesc = GetBindableStructDescriptor(InStructIndex).Get<FSceneStateBindingDesc>();

	// Binding directly into function's struct is allowed if it contains a compatible single output property.
	if (BindingDesc.DataHandle.IsDataType(ESceneStateDataType::Function))
	{
		const ISceneStateBindingCollectionOwner* BindingOwner = Cast<ISceneStateBindingCollectionOwner>(GetOwner());
		FPropertyBindingDataView DataView;

		// If DataView exists, struct is an instance of already bound function.
		if (!BindingOwner || BindingOwner->GetBindingDataViewByID(BindingDesc.ID, DataView))
		{
			return false;
		}

		if (const FStructProperty* StructProperty = CastField<FStructProperty>(GetProperty()))
		{
			// Disallow PropertyRef binding to property functions
			if (StructProperty->Struct && StructProperty->Struct->IsChildOf<FSceneStatePropertyReference>())
			{
				return false;
			}
		}

		if (const FProperty* OutputProperty = FindSingleOutputProperty(BindingDesc.Struct))
		{
			return CanBindToProperty(OutputProperty, { FBindingChainElement(nullptr, InStructIndex), FBindingChainElement(const_cast<FProperty*>(OutputProperty)) });
		}
	}

	return Super::CanBindToContextStructInternal(InStruct, InStructIndex);
}

bool FSceneStateCachedBindingData::CanAcceptPropertyOrChildrenInternal(const FProperty& InSourceProperty, TConstArrayView<FBindingChainElement, int> InBindingChain)
{
	if (InBindingChain.IsEmpty())
	{
		return Super::CanAcceptPropertyOrChildrenInternal(InSourceProperty, InBindingChain);
	}

	const int32 StructIndex = InBindingChain[0].ArrayIndex;
	const FSceneStateBindingDesc& BindingDesc = GetBindableStructDescriptor(StructIndex).Get<FSceneStateBindingDesc>();

	if (BindingDesc.DataHandle.IsDataType(ESceneStateDataType::Function))
	{
		const ISceneStateBindingCollectionOwner* BindingOwner = Cast<ISceneStateBindingCollectionOwner>(GetOwner());
		FPropertyBindingDataView DataView;

		// If DataView exists, struct is an instance of already bound function.
		if (!BindingOwner || BindingOwner->GetBindingDataViewByID(BindingDesc.ID, DataView))
		{
			return false;
		}

		// To avoid duplicates, PropertyFunction struct's children are not allowed to be bound if it contains a compatible single output property.
		if (const FProperty* OutputProperty = FindSingleOutputProperty(BindingDesc.Struct))
		{
			if (CanBindToProperty(OutputProperty, { FBindingChainElement(nullptr, StructIndex), FBindingChainElement(const_cast<FProperty*>(OutputProperty)) }))
			{
				return false;
			}
		}

		// Binding to non-output PropertyFunctions properties is not allowed.
		if (InBindingChain.Num() == 1 && !IsOutputProperty(&InSourceProperty))
		{
			return false;
		}
	}

	return Super::CanAcceptPropertyOrChildrenInternal(InSourceProperty, InBindingChain);
}

void FSceneStateCachedBindingData::GetSourceDataViewForNewBinding(TNotNull<IPropertyBindingBindingCollectionOwner*> InBindingsOwner, TConstStructView<FPropertyBindingBindableStructDescriptor> InDescriptor, FPropertyBindingDataView& OutSourceDataView)
{
	const FSceneStateBindingDesc& BindingDesc = InDescriptor.Get<FSceneStateBindingDesc>();

	if (BindingDesc.DataHandle.IsDataType(ESceneStateDataType::Function))
	{
		OutSourceDataView = FPropertyBindingDataView(BindingDesc.Struct, nullptr);
	}
	else
	{
		Super::GetSourceDataViewForNewBinding(InBindingsOwner, InDescriptor, OutSourceDataView);
	}
}

bool FSceneStateCachedBindingData::AddBindingInternal(TConstStructView<FPropertyBindingBindableStructDescriptor> InDescriptor, FPropertyBindingPath& InOutSourcePath, const FPropertyBindingPath& InTargetPath)
{
	const FSceneStateBindingDesc& BindingDesc = InDescriptor.Get<FSceneStateBindingDesc>();

	// Only functions have custom add binding logic
	if (!BindingDesc.DataHandle.IsDataType(ESceneStateDataType::Function))
	{
		return false;
	}

	ISceneStateBindingCollectionOwner* BindingsOwner = Cast<ISceneStateBindingCollectionOwner>(GetOwner());
	if (!ensure(BindingsOwner))
	{
		return false;
	}

	UE::SceneState::FBindingFunctionInfo FunctionInfo;

	BindingsOwner->ForEachBindableFunction(
		[ID = BindingDesc.ID, &FunctionInfo](const FSceneStateBindingDesc& InBindingDesc, const UE::SceneState::FBindingFunctionInfo& InFunctionInfo)->bool
		{
			if (InBindingDesc.ID == ID)
			{
				FunctionInfo = InFunctionInfo;
				return false; // break
			}
			return true; // continue
		});

	if (!ensure(FunctionInfo.FunctionTemplate.GetScriptStruct()))
	{
		return false;
	}

	FSceneStateBindingCollection& BindingCollection = BindingsOwner->GetBindingCollection();

	// If there are no segments, bindings leads directly into source struct's single output property. Its path has to be recovered.
	if (InOutSourcePath.NumSegments() == 0)
	{
		const FProperty* OutputProperty = FindSingleOutputProperty(BindingDesc.Struct);
		if (!ensure(OutputProperty))
		{
			return false;
		}

		FPropertyBindingPathSegment OutputPropertySegment = FPropertyBindingPathSegment(OutputProperty->GetFName());
		InOutSourcePath = BindingCollection.AddBindingFunction(FunctionInfo, MakeArrayView(&OutputPropertySegment, 1), InTargetPath);
	}
	else
	{
		InOutSourcePath = BindingCollection.AddBindingFunction(FunctionInfo, InOutSourcePath.GetSegments(), InTargetPath);
	}
	return true;
}

void FSceneStateCachedBindingData::AddPropertyInfoOverride(const FProperty& InProperty, TArray<TSharedPtr<const UE::PropertyBinding::FPropertyInfoOverride>>& OutPropertyInfoOverrides) const
{
	const FStructProperty* StructProperty = CastField<const FStructProperty>(&InProperty);
	if (!StructProperty || !StructProperty->Struct)
	{
		return;
	}

	// Add the PropertyRef property type with its RefType
	if (StructProperty->Struct->IsChildOf<FSceneStatePropertyReference>())
	{
		void* PropertyRefAddress = nullptr;

		// Passed in InProperty comes from GetPropertyHandle
		if (const IPropertyHandle* PropertyRefHandle = GetPropertyHandle())
		{
			PropertyRefHandle->GetValueData(PropertyRefAddress);
		}

		TArray<FEdGraphPinType, TInlineAllocator<1>> PropertyReferenceInfo = GetPropertyReferencePinTypes(&InProperty, PropertyRefAddress);
		OutPropertyInfoOverrides.Reserve(OutPropertyInfoOverrides.Num() + PropertyReferenceInfo.Num());

		for (const FEdGraphPinType& PinType : PropertyReferenceInfo)
		{
			TSharedRef<UE::PropertyBinding::FPropertyInfoOverride> PropertyInfoOverride = MakeShared<UE::PropertyBinding::FPropertyInfoOverride>();
			PropertyInfoOverride->PinType = PinType;
			PropertyInfoOverride->TypeNameText = UEdGraphSchema_K2::TypeToText(PinType);
			OutPropertyInfoOverrides.Emplace(MoveTemp(PropertyInfoOverride));
		}

		TSharedRef<UE::PropertyBinding::FPropertyInfoOverride> PropertyInfoOverride = MakeShared<UE::PropertyBinding::FPropertyInfoOverride>();
		PropertyInfoOverride->PinType.PinCategory = UEdGraphSchema_K2::PC_Struct;
		PropertyInfoOverride->PinType.PinSubCategoryObject = StructProperty->Struct;
		PropertyInfoOverride->TypeNameText = LOCTEXT("PromotePropertyReferenceDisplayName", "Promote as Reference");
		OutPropertyInfoOverrides.Emplace(MoveTemp(PropertyInfoOverride));
	}
}

bool FSceneStateCachedBindingData::DeterminePropertiesCompatibilityInternal(const FProperty* InSourceProperty, const FProperty* InTargetProperty, const void* InSourcePropertyValue, const void* InTargetPropertyValue, bool& bOutAreCompatible) const
{
	if (UE::SceneState::IsPropertyReference(InTargetProperty))
	{
		bOutAreCompatible = UE::SceneState::IsPropertyReferenceCompatible(InTargetProperty, InTargetPropertyValue, InSourceProperty, InSourcePropertyValue);
		return true;
	}
	return false;
}

bool FSceneStateCachedBindingData::GetPropertyFunctionText(FConstStructView InPropertyFunctionStructView, FText& OutText) const
{
	// Todo: consider having each function set their own descriptions while also being able to determine binding names (via passing the binding owner)
	const FSceneStateBindingFunction* BindingFunction = InPropertyFunctionStructView.GetPtr<const FSceneStateBindingFunction>();
	if (BindingFunction && BindingFunction->Function.GetScriptStruct())
	{
		OutText = BindingFunction->Function.GetScriptStruct()->GetDisplayNameText();
		return true;
	}
	return FCachedBindingData::GetPropertyFunctionText(InPropertyFunctionStructView, OutText);
}

bool FSceneStateCachedBindingData::GetPropertyFunctionTooltipText(FConstStructView InPropertyFunctionStructView, FText& OutText) const
{
	const FSceneStateBindingFunction* BindingFunction = InPropertyFunctionStructView.GetPtr<const FSceneStateBindingFunction>();
	if (BindingFunction && BindingFunction->Function.GetScriptStruct())
	{
		OutText = BindingFunction->Function.GetScriptStruct()->GetToolTipText();
		return true;
	}
	return Super::GetPropertyFunctionTooltipText(InPropertyFunctionStructView, OutText);
}

bool FSceneStateCachedBindingData::GetPropertyFunctionIconColor(FConstStructView InPropertyFunctionStructView, FLinearColor& OutColor) const
{
	if (const FSceneStateBindingFunction* BindingFunction = InPropertyFunctionStructView.GetPtr<const FSceneStateBindingFunction>())
	{
		if (const FProperty* OutputProperty = UE::SceneState::Editor::FindSingleOutputProperty(BindingFunction->FunctionInstance.GetScriptStruct()))
		{
			const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
			check(Schema);

			FEdGraphPinType PinType;
			if (Schema->ConvertPropertyToPinType(OutputProperty, PinType))
			{
				OutColor = Schema->GetPinTypeColor(PinType);
				return true;
			}
		}
	}
	return false;
}

bool FSceneStateCachedBindingData::GetPropertyFunctionImage(FConstStructView InPropertyFunctionStructView, const FSlateBrush*& OutImage) const
{
	OutImage = FAppStyle::GetBrush(TEXT("GraphEditor.Function_16x"));
	return true;
}

} // UE::SceneState::Editor

#undef LOCTEXT_NAMESPACE
