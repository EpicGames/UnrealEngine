// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyBindingExtension.h"

namespace UE::SceneState::Editor
{
	/** Determines whether the given property handle is deemed as an 'output' for a function */
	bool IsOutputProperty(const TSharedPtr<IPropertyHandle>& InPropertyHandle);

	/** Determines whether the given property is deemed as an 'output' for a function */
	bool IsOutputProperty(const FProperty* InProperty);

	/** Returns the single property deemed as 'output' for a function. Returns null if none or multiple found. */
	const FProperty* FindSingleOutputProperty(const UStruct* InStruct);

} // UE::SceneState::Editor

namespace UE::SceneState::Editor
{
struct FSceneStateCachedBindingData : UE::PropertyBinding::FCachedBindingData, TSharedFromThis<FSceneStateCachedBindingData>
{
	using Super = UE::PropertyBinding::FCachedBindingData;
	using Super::Super;

	/** Gets the property from the property handle if valid */
	const FProperty* GetProperty() const;

	//~ Begin UE::PropertyBinding::FCachedBindingData
	virtual void UpdatePropertyReferenceTooltip(const FProperty& InProperty, FTextBuilder& InOutTextBuilder) const override;
	virtual FPinInfo GetPinInfoForProperty(TNotNull<const FProperty*> InProperty, FPropertyBindingDataView InTargetDataView) const override;
	virtual bool IsPropertyReference(const FProperty& InProperty) override;
	virtual void UpdateSourcePropertyPath(TConstStructView<FPropertyBindingBindableStructDescriptor> InDescriptor, const FPropertyBindingPath& InSourcePath, FString& OutString);
	virtual bool CanBindToContextStructInternal(const UStruct* InStruct, const int32 InStructIndex) override;
	virtual bool CanAcceptPropertyOrChildrenInternal(const FProperty& InSourceProperty, TConstArrayView<FBindingChainElement, int> InBindingChain) override;
	virtual void GetSourceDataViewForNewBinding(TNotNull<IPropertyBindingBindingCollectionOwner*> InBindingsOwner, TConstStructView<FPropertyBindingBindableStructDescriptor> InDescriptor, FPropertyBindingDataView& OutSourceDataView) override;
	virtual bool AddBindingInternal(TConstStructView<FPropertyBindingBindableStructDescriptor> InDescriptor, FPropertyBindingPath& InOutSourcePath, const FPropertyBindingPath& InTargetPath) override;
	virtual void AddPropertyInfoOverride(const FProperty& InProperty, TArray<TSharedPtr<const UE::PropertyBinding::FPropertyInfoOverride>>& OutPropertyInfoOverrides) const override;
	virtual bool DeterminePropertiesCompatibilityInternal(const FProperty* InSourceProperty,const FProperty* InTargetProperty,const void* InSourcePropertyValue,const void* InTargetPropertyValue,bool& bOutAreCompatible) const override;
	virtual bool GetPropertyFunctionText(FConstStructView InPropertyFunctionStructView, FText& OutText) const;
	virtual bool GetPropertyFunctionTooltipText(FConstStructView InPropertyFunctionStructView, FText& OutText) const;
	virtual bool GetPropertyFunctionIconColor(FConstStructView InPropertyFunctionStructView, FLinearColor& OutColor) const;
	virtual bool GetPropertyFunctionImage(FConstStructView InPropertyFunctionStructView, const FSlateBrush*& OutImage) const;
	//~ End UE::PropertyBinding::FCachedBindingData
};

} // UE::SceneState::Editor
