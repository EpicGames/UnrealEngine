// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateInstancedStructDataProvider.h"
#include "PropertyHandle.h"
#include "StructUtils/InstancedStruct.h"

namespace UE::SceneState::Editor
{

FInstancedStructDataProvider::FInstancedStructDataProvider(const TSharedPtr<IPropertyHandle>& InControllerValuesHandle)
	: InstancedStructHandle(InControllerValuesHandle)
{
}

bool FInstancedStructDataProvider::IsValid() const
{
	bool bHasValidData = false;

	EnumerateInstances(
		[&bHasValidData](const UScriptStruct* InScriptStruct, uint8* InMemory, UPackage*)
		{
			if (InScriptStruct && InMemory)
			{
				bHasValidData = true;
				return false; // Break
			}
			return true; // Continue
		});

	return bHasValidData;
}

const UStruct* FInstancedStructDataProvider::GetBaseStructure() const
{
	// Taken from UClass::FindCommonBase
	auto FindCommonBaseStruct =
		[](const UScriptStruct* InStructA, const UScriptStruct* InStructB)
		{
			const UScriptStruct* CommonBaseStruct = InStructA;
			while (CommonBaseStruct && InStructB && !InStructB->IsChildOf(CommonBaseStruct))
			{
				CommonBaseStruct = Cast<UScriptStruct>(CommonBaseStruct->GetSuperStruct());
			}
			return CommonBaseStruct;
		};

	const UScriptStruct* CommonStruct = nullptr;

	EnumerateInstances(
		[&CommonStruct, &FindCommonBaseStruct](const UScriptStruct* InScriptStruct, uint8*, UPackage*)
		{
			if (InScriptStruct)
			{
				CommonStruct = FindCommonBaseStruct(InScriptStruct, CommonStruct);
			}
			return true; // Continue
		});

	return CommonStruct;
}

void FInstancedStructDataProvider::GetInstances(TArray<TSharedPtr<FStructOnScope>>& OutInstances, const UStruct* InExpectedBaseStructure) const
{
	// The returned instances need to be compatible with base structure.
	// This function returns empty instances in case they are not compatible, with the idea that we have as many instances as we have outer objects.
	EnumerateInstances(
		[&OutInstances, InExpectedBaseStructure](const UScriptStruct* InScriptStruct, uint8* InMemory, UPackage* InPackage)
		{
			TSharedPtr<FStructOnScope> Result;

			if (InExpectedBaseStructure && InScriptStruct && InScriptStruct->IsChildOf(InExpectedBaseStructure))
			{
				Result = MakeShared<FStructOnScope>(InScriptStruct, InMemory);
				Result->SetPackage(InPackage);
			}

			OutInstances.Add(Result);
			return true; // Continue
		});
}

bool FInstancedStructDataProvider::IsPropertyIndirection() const
{
	return true;
}

uint8* FInstancedStructDataProvider::GetValueBaseAddress(uint8* InParentValueAddress, const UStruct* InExpectedBaseStructure) const
{
	if (!InParentValueAddress)
	{
		return nullptr;
	}

	FInstancedStruct& InstancedStruct = *reinterpret_cast<FInstancedStruct*>(InParentValueAddress);
	if (InExpectedBaseStructure && InstancedStruct.GetScriptStruct() && InstancedStruct.GetScriptStruct()->IsChildOf(InExpectedBaseStructure))
	{
		return InstancedStruct.GetMutableMemory();
	}
	return nullptr;
}

void FInstancedStructDataProvider::EnumerateInstances(TFunctionRef<bool(const UScriptStruct*, uint8*, UPackage*)> InFunctor) const
{
	if (!InstancedStructHandle.IsValid())
	{
		return;
	}

	TArray<UPackage*> Packages;
	InstancedStructHandle->GetOuterPackages(Packages);

	InstancedStructHandle->EnumerateRawData(
		[&InFunctor, &Packages](void* InRawData, const int32 InDataIndex, const int32 InDataNum)
		{
			const UScriptStruct* ScriptStruct = nullptr;
			uint8* Memory = nullptr;
			UPackage* Package = nullptr;
			if (FInstancedStruct* InstancedStruct = static_cast<FInstancedStruct*>(InRawData))
			{
				ScriptStruct = InstancedStruct->GetScriptStruct();
				Memory = InstancedStruct->GetMutableMemory();
				if (ensureMsgf(Packages.IsValidIndex(InDataIndex), TEXT("Expecting packages count (%d) and raw data count (%d) to match."), Packages.Num(), InDataNum))
				{
					Package = Packages[InDataIndex];
				}
			}
			return InFunctor(ScriptStruct, Memory, Package);
		});
}

} // UE::SceneState::Editor
