// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IStructureDataProvider.h"
#include "Templates/SharedPointer.h"

class IPropertyHandle;
class UPackage;
class UScriptStruct;

namespace UE::SceneState::Editor
{

/** Structure Data Provider for Instanced Struct properties */
class FInstancedStructDataProvider : public IStructureDataProvider
{
public:
	FInstancedStructDataProvider() = default;

	explicit FInstancedStructDataProvider(const TSharedPtr<IPropertyHandle>& InInstancedStructHandle);

	virtual ~FInstancedStructDataProvider() override = default;

	//~ Begin IStructureDataProvider
	virtual bool IsValid() const override;
	virtual const UStruct* GetBaseStructure() const override;
	virtual void GetInstances(TArray<TSharedPtr<FStructOnScope>>& OutInstances, const UStruct* InExpectedBaseStructure) const override;
	virtual bool IsPropertyIndirection() const override;
	virtual uint8* GetValueBaseAddress(uint8* InParentValueAddress, const UStruct* InExpectedBaseStructure) const override;
	//~ End IStructureDataProvider

protected:
	/** Calls the given functor for every instanced struct instance that this handle manages */
	void EnumerateInstances(TFunctionRef<bool(const UScriptStruct*, uint8*, UPackage*)> InFunctor) const;

	/** Handle to the Instanced Struct property */
	TSharedPtr<IPropertyHandle> InstancedStructHandle;
};

} // UE::SceneState::Editor
