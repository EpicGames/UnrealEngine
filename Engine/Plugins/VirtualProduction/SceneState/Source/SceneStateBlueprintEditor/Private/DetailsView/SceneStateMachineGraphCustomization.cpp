// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateMachineGraphCustomization.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "IPropertyUtilities.h"
#include "PropertyBagDetails.h"
#include "SceneStateBlueprintEditorUtils.h"
#include "SceneStateMachineGraph.h"
#include "SceneStateParameterDetails.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

namespace UE::SceneState::Editor
{

void FStateMachineGraphCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	const TArray<TWeakObjectPtr<USceneStateMachineGraph>> Graphs = InDetailBuilder.GetObjectsOfTypeBeingCustomized<USceneStateMachineGraph>();

	const TSharedRef<IPropertyHandle> CategoryHandle = InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(USceneStateMachineGraph, Category));
	const TSharedRef<IPropertyHandle> StateMachineIdHandle = InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(USceneStateMachineGraph, ParametersId));
	const TSharedRef<IPropertyHandle> ParametersHandle = InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(USceneStateMachineGraph, Parameters));
	const TSharedRef<IPropertyHandle> RunModeHandle = InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(USceneStateMachineGraph, RunMode));

	CategoryHandle->MarkHiddenByCustomization();
	StateMachineIdHandle->MarkHiddenByCustomization();
	ParametersHandle->MarkHiddenByCustomization();
	RunModeHandle->MarkHiddenByCustomization();

	// Listen to Parameter Changes to broadcast change
	ParametersHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSPLambda(this,
		[Graphs]()
		{
			for (TWeakObjectPtr<USceneStateMachineGraph> GraphWeak : Graphs)
			{
				if (USceneStateMachineGraph* Graph = GraphWeak.Get())
				{
					Graph->NotifyParametersChanged();
				}
			}
		}));

	// State Machine Category
	IDetailCategoryBuilder& StateMachineCategory = InDetailBuilder.EditCategory(TEXT("State Machine"));

	// Show category if there's at least one top-level graph
	const bool bHasTopLevelGraph = Graphs.ContainsByPredicate(
		[](TWeakObjectPtr<USceneStateMachineGraph> InGraphWeak)
		{
			USceneStateMachineGraph* Graph = InGraphWeak.Get();
			return Graph && Graph->IsRootStateMachine();
		});

	if (bHasTopLevelGraph)
	{
		StateMachineCategory.AddProperty(CategoryHandle);

		// Todo: at the moment run mode is not supported in sub-graphs
		StateMachineCategory.AddProperty(RunModeHandle);
	}

	FGuid StateMachineId;
	GetGuid(StateMachineIdHandle, StateMachineId);

	// Parameters Category
	IDetailCategoryBuilder& ParametersCategory = InDetailBuilder.EditCategory(TEXT("Parameters"));
	ParametersCategory.HeaderContent(FParameterDetails::BuildHeader(InDetailBuilder, ParametersHandle), /*bWholeRowContent*/true);

	ParametersCategory.AddCustomBuilder(MakeShared<FParameterDetails>(ParametersHandle
		, InDetailBuilder.GetPropertyUtilities()
		, StateMachineId
		, /*bFixedLayout*/false));
}

} // UE::SceneState::Editor
