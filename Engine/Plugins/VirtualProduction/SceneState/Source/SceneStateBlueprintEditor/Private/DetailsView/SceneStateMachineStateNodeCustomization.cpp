// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateMachineStateNodeCustomization.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "Nodes/SceneStateMachineStateNode.h"
#include "SceneStateBlueprintEditorUtils.h"
#include "SceneStateParameterDetails.h"

namespace UE::SceneState::Editor
{

void FStateMachineStateNodeCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	const TArray<TWeakObjectPtr<USceneStateMachineStateNode>> StateNodes = InDetailBuilder.GetObjectsOfTypeBeingCustomized<USceneStateMachineStateNode>();

	const TSharedRef<IPropertyHandle> ParametersIdHandle = InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(USceneStateMachineStateNode, ParametersId));
	const TSharedRef<IPropertyHandle> ParametersHandle = InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(USceneStateMachineStateNode, Parameters));
	const TSharedRef<IPropertyHandle> EventHandlersHandle = InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(USceneStateMachineStateNode, EventHandlers));

	ParametersIdHandle->MarkHiddenByCustomization();
	ParametersHandle->MarkHiddenByCustomization();
	EventHandlersHandle->MarkHiddenByCustomization();

	// Listen to Parameter Changes to broadcast change
	ParametersHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSPLambda(this,
		[StateNodes]()
		{
			for (TWeakObjectPtr<USceneStateMachineStateNode> StateNodeWeak : StateNodes)
			{
				if (USceneStateMachineStateNode* StateNode = StateNodeWeak.Get())
				{
					StateNode->NotifyParametersChanged();
				}
			}
		}));

	FGuid ParametersId;
	GetGuid(ParametersIdHandle, ParametersId);

	// Parameters Category
	IDetailCategoryBuilder& ParametersCategory = InDetailBuilder.EditCategory(TEXT("Parameters"));
	ParametersCategory.HeaderContent(FParameterDetails::BuildHeader(InDetailBuilder, ParametersHandle), /*bWholeRowContent*/true);

	ParametersCategory.AddCustomBuilder(MakeShared<FParameterDetails>(ParametersHandle
		, InDetailBuilder.GetPropertyUtilities()
		, ParametersId
		, /*bFixedLayout*/false));

	// Event Handlers Category
	IDetailCategoryBuilder& EventsCategory = InDetailBuilder.EditCategory(TEXT("Events"));
	EventsCategory.AddProperty(EventHandlersHandle);
}

} // UE::SceneState::Editor
