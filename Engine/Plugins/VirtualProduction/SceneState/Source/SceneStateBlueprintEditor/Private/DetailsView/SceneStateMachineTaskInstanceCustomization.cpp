// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateMachineTaskInstanceCustomization.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "SceneStateBlueprintEditorUtils.h"
#include "SceneStateParameterDetails.h"
#include "Tasks/SceneStateMachineTask.h"
#include "Widgets/SSceneStateMachinePicker.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SceneStateMachineTaskInstanceCustomization"

namespace UE::SceneState::Editor
{

void FStateMachineTaskInstanceCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
}

void FStateMachineTaskInstanceCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	const FName TargetIdName = GET_MEMBER_NAME_CHECKED(FSceneStateMachineTaskInstance, TargetId);
	const FName ParametersIdName = GET_MEMBER_NAME_CHECKED(FSceneStateMachineTaskInstance, ParametersId);
	const FName ParametersName = GET_MEMBER_NAME_CHECKED(FSceneStateMachineTaskInstance, Parameters);

	const TSharedRef<IPropertyHandle> TargetIdHandle = InPropertyHandle->GetChildHandle(TargetIdName).ToSharedRef();
	const TSharedRef<IPropertyHandle> ParametersIdHandle = InPropertyHandle->GetChildHandle(ParametersIdName).ToSharedRef();
	const TSharedRef<IPropertyHandle> ParametersHandle = InPropertyHandle->GetChildHandle(ParametersName).ToSharedRef();

	ParametersIdHandle->MarkHiddenByCustomization();
	TargetIdHandle->MarkHiddenByCustomization();

	InChildBuilder
		.AddProperty(TargetIdHandle)
		.CustomWidget()
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("StateMachineIdDisplayName", "State Machine"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SStateMachinePicker, TargetIdHandle, ParametersHandle)
		];

	FGuid ParametersId;
	GetGuid(ParametersIdHandle, ParametersId);

	IDetailCategoryBuilder& TasksCategory = InChildBuilder.GetParentCategory();
	TasksCategory.SetSortOrder(0);

	IDetailCategoryBuilder& ParametersCategory = TasksCategory.GetParentLayout().EditCategory(TEXT("Parameters"));
	ParametersCategory.SetSortOrder(1);

	ParametersCategory.AddCustomBuilder(MakeShared<FParameterDetails>(ParametersHandle
		, InCustomizationUtils.GetPropertyUtilities().ToSharedRef()
		, ParametersId
		, /*bFixedLayout*/true));
}

} // UE::SceneState::Editor

#undef LOCTEXT_NAMESPACE
