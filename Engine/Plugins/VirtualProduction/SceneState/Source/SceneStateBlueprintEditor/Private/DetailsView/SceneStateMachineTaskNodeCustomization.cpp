// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateMachineTaskNodeCustomization.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "InstancedStructDetails.h"
#include "Nodes/SceneStateMachineTaskNode.h"
#include "PropertyBindingExtension.h"
#include "SceneStateBlueprintEditorUtils.h"

namespace UE::SceneState::Editor
{

namespace Private
{

class FTaskInstanceDetails : public FInstancedStructDataDetails
{
public:
	FTaskInstanceDetails(TSharedPtr<IPropertyHandle> InStructProperty, const FGuid& InTaskId)
		: FInstancedStructDataDetails(InStructProperty)
		, TaskId(InTaskId)
	{
	}

	//~ Begin FInstancedStructDataDetails
	virtual void OnChildRowAdded(IDetailPropertyRow& InChildRow) override
	{
		TSharedPtr<IPropertyHandle> ChildPropHandle = InChildRow.GetPropertyHandle();
		check(ChildPropHandle.IsValid());
		AssignBindingId(ChildPropHandle.ToSharedRef(), TaskId);
	}
	//~ End FInstancedStructDataDetails

private:
	FGuid TaskId;
};

} // Private
	
void FStateMachineTaskNodeCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	const TSharedRef<IPropertyHandle> TaskHandle = InDetailBuilder.GetProperty(USceneStateMachineTaskNode::GetTaskPropertyName());
	const TSharedRef<IPropertyHandle> TaskInstanceHandle = InDetailBuilder.GetProperty(USceneStateMachineTaskNode::GetTaskInstancePropertyName());

	TaskHandle->MarkHiddenByCustomization();
	TaskInstanceHandle->MarkHiddenByCustomization();

	const FGuid TaskId = FindTaskId(TaskInstanceHandle);

	IDetailCategoryBuilder& TaskCategory = InDetailBuilder.EditCategory(TEXT("Task"));
	TaskCategory.InitiallyCollapsed(false);
	TaskCategory.RestoreExpansionState(true);
	TaskCategory.AddCustomBuilder(MakeShared<FInstancedStructDataDetails>(TaskHandle));
	TaskCategory.AddCustomBuilder(MakeShared<Private::FTaskInstanceDetails>(TaskInstanceHandle, TaskId));
}

} // UE::SceneState::Editor
