// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateMachineTransitionNodeCustomization.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "IPropertyUtilities.h"
#include "Nodes/SceneStateMachineTransitionNode.h"
#include "PropertyBagDetails.h"
#include "SceneStateBlueprintEditorUtils.h"
#include "SceneStateParameterDetails.h"

namespace UE::SceneState::Editor
{

void FStateMachineTransitionNodeCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	TSharedRef<IPropertyHandle> ParametersIdHandle = InDetailBuilder.GetProperty(USceneStateMachineTransitionNode::GetParametersIdName());
	TSharedRef<IPropertyHandle> ParametersHandle = InDetailBuilder.GetProperty(USceneStateMachineTransitionNode::GetParametersName());

	ParametersIdHandle->MarkHiddenByCustomization();
	ParametersHandle->MarkHiddenByCustomization();

	// Transitions Category
	IDetailCategoryBuilder& TransitionsCategory = InDetailBuilder.EditCategory(TEXT("Transitions"));
	TransitionsCategory.SetSortOrder(0);

	FGuid ParametersId;
	GetGuid(ParametersIdHandle, ParametersId);

	// Parameters Category
	IDetailCategoryBuilder& ParametersCategory = InDetailBuilder.EditCategory(TEXT("Parameters"));
	ParametersCategory.SetSortOrder(1);
	ParametersCategory.HeaderContent(FParameterDetails::BuildHeader(InDetailBuilder, ParametersHandle), /*bWholeRowContent*/true);

	ParametersCategory.AddCustomBuilder(MakeShared<FParameterDetails>(ParametersHandle
		, InDetailBuilder.GetPropertyUtilities()
		, ParametersId
		, /*bFixedLayout*/false));
}

} // UE::SceneState::Editor
