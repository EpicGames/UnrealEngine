// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateParameterDetails.h"
#include "DetailLayoutBuilder.h"
#include "IDetailPropertyRow.h"
#include "IPropertyUtilities.h"
#include "SceneStateBlueprintEditorUtils.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

namespace UE::SceneState::Editor
{

TSharedRef<SWidget> FParameterDetails::BuildHeader(IDetailLayoutBuilder& InDetailBuilder, const TSharedRef<IPropertyHandle>& InParametersHandle)
{
	const TSharedRef<IPropertyUtilities> PropUtils = InDetailBuilder.GetPropertyUtilities();

	return SNew(SHorizontalBox)
		.IsEnabled(PropUtils, &IPropertyUtilities::IsPropertyEditingEnabled)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(4.f, 0.f, 0.f, 0.f)
		[
			SNew(STextBlock)
			.Text(InParametersHandle->GetPropertyDisplayName())
			.Font(FAppStyle::Get().GetFontStyle(TEXT("DetailsView.CategoryFontStyle")))
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(4.f, 0.f, 0.f, 0.f)
		[
			FPropertyBagDetails::MakeAddPropertyWidget(InParametersHandle, PropUtils, EPropertyBagPropertyType::String).ToSharedRef()
		];
}

FParameterDetails::FParameterDetails(const TSharedRef<IPropertyHandle>& InStructProperty, const TSharedRef<IPropertyUtilities>& InPropUtils, const FGuid& InParametersId, bool bInFixedLayout)
	: FPropertyBagInstanceDataDetails(InStructProperty, InPropUtils, bInFixedLayout)
	, ParametersId(InParametersId)
{
}

void FParameterDetails::OnChildRowAdded(IDetailPropertyRow& InChildRow)
{
	FPropertyBagInstanceDataDetails::OnChildRowAdded(InChildRow);

	TSharedPtr<IPropertyHandle> ChildPropHandle = InChildRow.GetPropertyHandle();
	check(ChildPropHandle.IsValid());
	AssignBindingId(ChildPropHandle.ToSharedRef(), ParametersId);
}

} // UE::SceneState::Editor
