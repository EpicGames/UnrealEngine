// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyBagDetails.h"
#include "Misc/Guid.h"

class IDetailLayoutBuilder;

namespace UE::SceneState::Editor
{

class FParameterDetails : public FPropertyBagInstanceDataDetails
{
public:
	static TSharedRef<SWidget> BuildHeader(IDetailLayoutBuilder& InDetailBuilder, const TSharedRef<IPropertyHandle>& InParametersHandle);

	explicit FParameterDetails(const TSharedRef<IPropertyHandle>& InStructProperty
		, const TSharedRef<IPropertyUtilities>& InPropUtils
		, const FGuid& InParametersId
		, bool bInFixedLayout);

	//~ Begin FPropertyBagInstanceDataDetails
	virtual void OnChildRowAdded(IDetailPropertyRow& InChildRow) override;
	//~ End FPropertyBagInstanceDataDetails

private:
	FGuid ParametersId;
};

} // UE::SceneState::Editor
