// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStatePlayerTaskInstanceCustomization.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyHandle.h"
#include "SceneStateBlueprintEditorUtils.h"
#include "SceneStateObject.h"
#include "SceneStatePlayer.h"
#include "Tasks/SceneStatePlayerTask.h"

namespace UE::SceneState::Editor
{

void FPlayerTaskInstanceCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
}

void FPlayerTaskInstanceCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	TSharedPtr<IPropertyHandle> PlayerHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FSceneStatePlayerTaskInstance, Player));
	if (PlayerHandle.IsValid())
	{
		CustomizePlayer(PlayerHandle.ToSharedRef(), InChildBuilder);
	}
}

void FPlayerTaskInstanceCustomization::CustomizePlayer(const TSharedRef<IPropertyHandle>& InPlayerHandle, IDetailChildrenBuilder& InChildBuilder)
{
	TSharedPtr<IPropertyHandle> RootObjectHandle = InPlayerHandle->GetChildHandle(USceneStatePlayer::GetRootStateName());
	TSharedPtr<IPropertyHandle> SceneStateClassHandle = InPlayerHandle->GetChildHandle(USceneStatePlayer::GetSceneStateClassName());

	InPlayerHandle->MarkHiddenByCustomization();
	AssignBindingId(InPlayerHandle, FindTaskId(InPlayerHandle));

	if (SceneStateClassHandle.IsValid())
	{
		InChildBuilder.AddProperty(SceneStateClassHandle.ToSharedRef());
	}

	if (RootObjectHandle.IsValid())
	{
		AddObjectProperties(RootObjectHandle.ToSharedRef(), InChildBuilder);
	}
}

} // UE::SceneState::Editor
