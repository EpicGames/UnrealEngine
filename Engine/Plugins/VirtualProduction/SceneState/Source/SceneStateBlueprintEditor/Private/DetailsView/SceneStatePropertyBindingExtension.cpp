// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStatePropertyBindingExtension.h"
#include "EdGraphSchema_K2.h"
#include "GameFramework/Actor.h"
#include "IPropertyAccessEditor.h"
#include "SceneStateBindingDesc.h"
#include "SceneStateBindingMetadata.h"
#include "SceneStateBlueprintEditorUtils.h"
#include "SceneStateCachedBindingData.h"
#include "SceneStatePropertyReference.h"

namespace UE::SceneState::Editor
{

void FBindingExtension::ExtendWidgetRow(FDetailWidgetRow& InWidgetRow, const IDetailLayoutBuilder& InDetailBuilder, const UClass* InObjectClass, TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	// Disable Actor references from being set from within the editor
	// This aligns with actor ref properties which are disabled in template objects (e.g. a bp variable of actor ref type will be disabled within the blueprint)
	// This does not disable the entire row though, to allow the binding extension to still work.
	if (IsObjectPropertyOfClass(InPropertyHandle->GetProperty(), AActor::StaticClass()))
	{
		InWidgetRow.IsValueEnabled(false);
	}
	FPropertyBindingExtension::ExtendWidgetRow(InWidgetRow, InDetailBuilder, InObjectClass, InPropertyHandle);
}

bool FBindingExtension::IsPropertyExtendable(const UClass* InObjectClass, const IPropertyHandle& InPropertyHandle) const
{
	// Disallow if property has 'NoBindingSelfOnly'.
	// Unlike 'NoBinding', this only checks 'self' and does not recurse check parent handles.
	if (InPropertyHandle.HasMetaData(Metadata::NoBindingSelfOnly))
	{
		return false;
	}

	// Version of 'NoBindingSelfOnly' for containers to only apply it on the container property, not its inner properties.
	if (InPropertyHandle.HasMetaData(Metadata::NoBindingContainerSelfOnly))
	{
		// Container and its Inner properties carry this same meta-data so check if the property is the container, and not the inner property.
		const FProperty* Property = InPropertyHandle.GetProperty();
		if (!Property || Property->IsA<FArrayProperty>() || Property->IsA<FSetProperty>() || Property->IsA<FMapProperty>())
		{
			return false;
		}
	}

	return FPropertyBindingExtension::IsPropertyExtendable(InObjectClass, InPropertyHandle);
}

TSharedPtr<UE::PropertyBinding::FCachedBindingData> FBindingExtension::CreateCachedBindingData(IPropertyBindingBindingCollectionOwner* InBindingsOwner
	, const FPropertyBindingPath& InTargetPath
	, const TSharedPtr<IPropertyHandle>& InPropertyHandle
	, TConstArrayView<TInstancedStruct<FPropertyBindingBindableStructDescriptor>> InAccessibleStructs) const
{
	return MakeShared<FSceneStateCachedBindingData>(InBindingsOwner, InTargetPath, InPropertyHandle, InAccessibleStructs);
}

bool FBindingExtension::GetPromotionToParameterOverrideInternal(const FProperty& InProperty, bool& bOutOverride) const
{
	if (const FStructProperty* StructProperty = CastField<FStructProperty>(&InProperty))
	{
		// Support Property Refs as even though these aren't bp types, the actual types that would be added are the ones in the meta-data RefType
		if (StructProperty->Struct && StructProperty->Struct->IsChildOf<FSceneStatePropertyReference>())
		{
			bOutOverride = true;
			return true;
		}
	}
	return false;
}

void FBindingExtension::UpdateContextStruct(TConstStructView<FPropertyBindingBindableStructDescriptor> InStructDesc, FBindingContextStruct& InOutContextStruct, TMap<FString, FText>& InOutSectionNames) const
{
	const FSceneStateBindingDesc& BindingDesc = InStructDesc.Get<FSceneStateBindingDesc>();
	InOutContextStruct.Section = BindingDesc.Section;

	// Function overrides it's struct's icon color.
	if (BindingDesc.DataHandle.IsDataType(ESceneStateDataType::Function))
	{
		if (const FProperty* OutputProperty = FindSingleOutputProperty(BindingDesc.Struct))
		{
			const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

			FEdGraphPinType PinType;
			if (ensure(Schema) && Schema->ConvertPropertyToPinType(OutputProperty, PinType))
			{
				InOutContextStruct.Color = Schema->GetPinTypeColor(PinType);
			}
		}
	}
}

bool FBindingExtension::CanBindToProperty(const FPropertyBindingPath& InTargetPath, const IPropertyHandle& InPropertyHandle) const
{
	const FProperty* PropertyToBind = InPropertyHandle.GetProperty();
	return PropertyToBind && !IsOutputProperty(PropertyToBind);
}

bool FBindingExtension::CanBindToArrayElements() const
{
	return true;
}
} // UE::SceneState::Editor
