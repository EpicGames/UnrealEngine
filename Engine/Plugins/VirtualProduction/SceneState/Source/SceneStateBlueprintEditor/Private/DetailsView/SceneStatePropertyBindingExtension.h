// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyBindingExtension.h"

namespace UE::SceneState::Editor
{

class FBindingExtension : public FPropertyBindingExtension
{
public:
	//~ Begin IDetailPropertyExtensionHandler
	virtual void ExtendWidgetRow(FDetailWidgetRow& InWidgetRow, const IDetailLayoutBuilder& InDetailBuilder, const UClass* InObjectClass, TSharedPtr<IPropertyHandle> InPropertyHandle) override;
	//~ End IDetailPropertyExtensionHandler

	//~ Begin FPropertyBindingExtension
	virtual bool IsPropertyExtendable(const UClass* InObjectClass, const IPropertyHandle& InPropertyHandle) const override;
	virtual TSharedPtr<UE::PropertyBinding::FCachedBindingData> CreateCachedBindingData(IPropertyBindingBindingCollectionOwner* InBindingsOwner, const FPropertyBindingPath& InTargetPath, const TSharedPtr<IPropertyHandle>& InPropertyHandle, TConstArrayView<TInstancedStruct<FPropertyBindingBindableStructDescriptor>> InAccessibleStructs) const override;
	virtual bool GetPromotionToParameterOverrideInternal(const FProperty& InProperty, bool& bOutOverride) const override;
	virtual void UpdateContextStruct(TConstStructView<FPropertyBindingBindableStructDescriptor> InStructDesc, FBindingContextStruct& InOutContextStruct, TMap<FString, FText>& InOutSectionNames) const override;
	virtual bool CanBindToProperty(const FPropertyBindingPath& InTargetPath, const IPropertyHandle& InPropertyHandle) const override;
	virtual bool CanBindToArrayElements() const override;
	//~ End FPropertyBindingExtension
};

} // UE::SceneState::Editor
