// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSceneStateMachinePicker.h"
#include "DetailLayoutBuilder.h"
#include "GuidStructCustomization.h"
#include "SceneStateBlueprint.h"
#include "SceneStateBlueprintEditorUtils.h"
#include "SceneStateMachineGraph.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SSceneStateMachinePicker"

namespace UE::SceneState::Editor
{

namespace Private
{

template<typename T>
T* GetTypedOuter(TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	if (!InPropertyHandle.IsValid())
	{
		return nullptr;
	}

	TArray<UObject*> OuterObjects;
	InPropertyHandle->GetOuterObjects(OuterObjects);

	// Find the first valid object of type T from the outer objects
	for (UObject* OuterObject : OuterObjects)
	{
		if (OuterObject)
		{
			if (T* Result = Cast<T>(OuterObject))
			{
				return Result;
			}

			if (T* Result = OuterObject->GetTypedOuter<T>())
			{
				return Result;
			}
		}
	}
	return nullptr;
}

} // Private

void SStateMachinePicker::Construct(const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InStateMachineIdHandle, const TSharedRef<IPropertyHandle>& InParametersHandle)
{
	StateMachineIdHandle = InStateMachineIdHandle;
	ParametersHandle = InParametersHandle;

	// Add all the ids of outer state machine graphs to the ignore set
	StateMachineIdsToIgnore.Reset();
	{
		const USceneStateMachineGraph* StateMachineGraph = Private::GetTypedOuter<USceneStateMachineGraph>(ParametersHandle);
		while (StateMachineGraph)
		{
			StateMachineIdsToIgnore.Add(StateMachineGraph->ParametersId);
			StateMachineGraph = StateMachineGraph->GetTypedOuter<USceneStateMachineGraph>();
		}
	}

	OnParametersChangedHandle = USceneStateMachineGraph::OnParametersChanged().AddSP(this
		, &SStateMachinePicker::OnParametersChanged);

	ChildSlot
	[
		SAssignNew(Picker, SComboBox<TSharedRef<FStateMachinePickerOption>>)
		.OptionsSource(&PickerOptions)
		.InitiallySelectedItem(SelectedOption)
		.OnGenerateWidget(this, &SStateMachinePicker::GenerateOptionWidget)
		.OnComboBoxOpening(this, &SStateMachinePicker::RefreshOptions)
		.OnSelectionChanged(this, &SStateMachinePicker::OnOptionSelectionChanged)
		[
			SNew(STextBlock)
			.Text(this, &SStateMachinePicker::GetStateMachineName)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
	];

	RefreshOptions();
	RefreshParameters();
}

SStateMachinePicker::~SStateMachinePicker()
{
	USceneStateMachineGraph::OnParametersChanged().Remove(OnParametersChangedHandle);
	OnParametersChangedHandle.Reset();
}

void SStateMachinePicker::OnParametersChanged(USceneStateMachineGraph* InGraph)
{
	if (!SelectedOption.IsValid() || SelectedOption->Id != InGraph->ParametersId)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("ParametersChanged", "Parameters Changed"));

	WriteGuidToProperty(StateMachineIdHandle, InGraph->ParametersId);
	RefreshOptions();

	ParametersHandle->NotifyPreChange();

	RefreshParameters();

	ParametersHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	ParametersHandle->NotifyFinishedChangingProperties();
}

FText SStateMachinePicker::GetStateMachineName() const
{
	if (SelectedOption.IsValid())
	{
		return FText::FromName(SelectedOption->Name);
	}
	return FText::GetEmpty();
}

void SStateMachinePicker::RefreshOptions()
{
	SelectedOption.Reset();
	PickerOptions.Reset();

	USceneStateBlueprint* const Blueprint = Private::GetTypedOuter<USceneStateBlueprint>(ParametersHandle);
	if (!Blueprint)
	{
		return;
	}

	FGuid CurrentStateMachineId;
	GetGuid(StateMachineIdHandle.ToSharedRef(), CurrentStateMachineId);

	// Todo: this currently only considers the top level state machines, which are the only ones that can be set to 'On-Demand'
	PickerOptions.Reserve(Blueprint->StateMachineGraphs.Num());

	for (UEdGraph* Graph : Blueprint->StateMachineGraphs)
	{
		USceneStateMachineGraph* StateMachineGraph = Cast<USceneStateMachineGraph>(Graph);

		// Skip the state machines in the ignore set
		if (!StateMachineGraph || StateMachineIdsToIgnore.Contains(StateMachineGraph->ParametersId))
		{
			continue;
		}

		TSharedRef<FStateMachinePickerOption> Option = MakeShared<FStateMachinePickerOption>();
		Option->Name = StateMachineGraph->GetFName();
		Option->Id = StateMachineGraph->ParametersId;
		Option->GraphWeak = StateMachineGraph;

		PickerOptions.Add(Option);

		if (CurrentStateMachineId == StateMachineGraph->ParametersId)
		{
			SelectedOption = Option;
		}
	}

	// Update the picker's selected option (can be null)
	if (Picker.IsValid())
	{
		Picker->RefreshOptions();
		Picker->SetSelectedItem(SelectedOption);
	}
}

void SStateMachinePicker::RefreshParameters()
{
	USceneStateBlueprint* const Blueprint = Private::GetTypedOuter<USceneStateBlueprint>(ParametersHandle);
	if (!Blueprint)
	{
		return;
	}

	USceneStateMachineGraph* Graph = nullptr;
	if (SelectedOption.IsValid())
	{
		Graph = SelectedOption->GraphWeak.Get();
	}

	if (Graph)
	{
		ForEachInstancedPropertyBag(
			[Graph](FInstancedPropertyBag& InstancedPropertyBag)
			{
				if (!CompareParametersLayout(InstancedPropertyBag, Graph->Parameters))
				{
					FInstancedPropertyBag OldPropertyBag = InstancedPropertyBag;
					InstancedPropertyBag = Graph->Parameters;
					InstancedPropertyBag.CopyMatchingValuesByID(OldPropertyBag);
				}
				return true; //continue
			});
	}
	else
	{
		ForEachInstancedPropertyBag(
			[](FInstancedPropertyBag& InstancedPropertyBag)
			{
				InstancedPropertyBag.Reset();
				return true; //continue
			});
	}
}

void SStateMachinePicker::ForEachInstancedPropertyBag(TFunctionRef<bool(FInstancedPropertyBag&)> InFunctor)
{
	ParametersHandle->EnumerateRawData(
		[&InFunctor](void* InStructRawData, const int32, const int32)->bool
		{
			if (FInstancedPropertyBag* InstancedPropertyBag = static_cast<FInstancedPropertyBag*>(InStructRawData))
			{
				return InFunctor(*InstancedPropertyBag);
			}
			return true; //continue
		});
}

TSharedRef<SWidget> SStateMachinePicker::GenerateOptionWidget(TSharedRef<FStateMachinePickerOption> InOption)
{
	return SNew(STextBlock)
	   .Text(FText::FromName(InOption->Name))
	   .Font(IDetailLayoutBuilder::GetDetailFont());
}

void SStateMachinePicker::OnOptionSelectionChanged(TSharedPtr<FStateMachinePickerOption> InSelectedOption, ESelectInfo::Type InSelectInfo)
{
	if (InSelectedOption.IsValid() && SelectedOption != InSelectedOption)
	{
		FScopedTransaction Transaction(LOCTEXT("SetStateMachine", "Set State Machine"));
		SelectedOption = InSelectedOption;
		WriteGuidToProperty(StateMachineIdHandle.ToSharedRef(), SelectedOption->Id);

		ParametersHandle->NotifyPreChange();

		RefreshParameters();

		ParametersHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		ParametersHandle->NotifyFinishedChangingProperties();
	}
}

} // UE::SceneState::Editor

#undef LOCTEXT_NAMESPACE
