// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Widgets/SCompoundWidget.h"

class IPropertyHandle;
class USceneStateBlueprint;
class USceneStateMachineGraph;
struct FInstancedPropertyBag;
template<typename OptionType> class SComboBox;

namespace UE::SceneState::Editor
{

struct FStateMachinePickerOption
{
	FName Name;
	FGuid Id;
	TWeakObjectPtr<USceneStateMachineGraph> GraphWeak;
};

/**
 * Widget that shows the available State Machine Graphs that are set to 'Manual' as names,
 * but underneath handles them / saves these as Guids
 */
class SStateMachinePicker : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SStateMachinePicker) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InStateMachineIdHandle, const TSharedRef<IPropertyHandle>& InParametersHandle);

	virtual ~SStateMachinePicker() override;

private:
	void OnParametersChanged(USceneStateMachineGraph* InGraph);

	FText GetStateMachineName() const;

	void RefreshOptions();

	void RefreshParameters();

	void ForEachInstancedPropertyBag(TFunctionRef<bool(FInstancedPropertyBag&)> InFunctor);

	TSharedRef<SWidget> GenerateOptionWidget(TSharedRef<FStateMachinePickerOption> InOption);

	void OnOptionSelectionChanged(TSharedPtr<FStateMachinePickerOption> InSelectedOption, ESelectInfo::Type InSelectInfo);

	TSharedPtr<IPropertyHandle> StateMachineIdHandle;
	TSharedPtr<IPropertyHandle> ParametersHandle;

	TSharedPtr<SComboBox<TSharedRef<FStateMachinePickerOption>>> Picker;
	TArray<TSharedRef<FStateMachinePickerOption>> PickerOptions;

	TSharedPtr<FStateMachinePickerOption> SelectedOption;
	FDelegateHandle OnParametersChangedHandle;

	/** State machine ids that this picker should not display as options */
	TSet<FGuid> StateMachineIdsToIgnore;
};

} // UE::SceneState::Editor
