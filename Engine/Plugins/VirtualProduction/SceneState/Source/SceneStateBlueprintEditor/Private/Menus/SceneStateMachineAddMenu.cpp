// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateMachineAddMenu.h"
#include "SceneStateBlueprintEditorCommands.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "SceneStateMachineAddMenu"

namespace UE::SceneState::Editor
{

FStateMachineAddMenu::FStateMachineAddMenu()
	: CommandList(MakeShared<FUICommandList>())
{
}

void FStateMachineAddMenu::BindCommands(const TSharedRef<FUICommandList>& InCommandList)
{
	CommandList->Append(InCommandList);
}

TSharedRef<SWidget> FStateMachineAddMenu::GenerateWidget()
{
	UToolMenus* const ToolMenus = UToolMenus::Get();
	check(ToolMenus);

	const FName MenuName = GetMenuName();

	if (!ToolMenus->IsMenuRegistered(MenuName))
	{
		UToolMenu* const Menu = ToolMenus->RegisterMenu(MenuName, NAME_None, EMultiBoxType::Menu);

		FToolMenuSection& Section = Menu->AddSection(TEXT("AddNewItem"), LOCTEXT("AddOperations", "Add New"));

		const FBlueprintEditorCommands& SceneStateBlueprintEditorCommands = FBlueprintEditorCommands::Get();
		Section.AddMenuEntry(SceneStateBlueprintEditorCommands.AddStateMachine, LOCTEXT("AddStateMachine", "State Machine"));
	}

	FToolMenuContext Context(CommandList);
	return ToolMenus->GenerateWidget(MenuName, Context);
}

} // UE::SceneState::Editor

#undef LOCTEXT_NAMESPACE
