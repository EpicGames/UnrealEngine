// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class FUICommandList;
class SWidget;

namespace UE::SceneState::Editor
{

class FStateMachineAddMenu : public TSharedFromThis<FStateMachineAddMenu>
{
public:
	FStateMachineAddMenu();

	void BindCommands(const TSharedRef<FUICommandList>& InCommandList);

	static FName GetMenuName()
	{
		return TEXT("SceneStateMachineAddMenu");
	}

	TSharedRef<SWidget> GenerateWidget(); 

private:
	TSharedRef<FUICommandList> CommandList;
};

} // UE::SceneState::Editor
