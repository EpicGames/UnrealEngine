// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateMachineContextMenu.h"
#include "Actions/SceneStateBlueprintAction_Graph.h"
#include "Framework/Commands/GenericCommands.h"
#include "K2Node_Composite.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "SGraphActionMenu.h"
#include "SceneStateBlueprint.h"
#include "SceneStateBlueprintEditor.h"
#include "SceneStateMachineGraphUtils.h"
#include "SceneStateUtils.h"
#include "ScopedTransaction.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "SceneStateMachineContextMenu"

namespace UE::SceneState::Editor
{

FStateMachineContextMenu::FStateMachineContextMenu(const TSharedRef<FSceneStateBlueprintEditor>& InBlueprintEditor, const TSharedRef<SGraphActionMenu>& InGraphActionMenu)
	: BlueprintEditorWeak(InBlueprintEditor)
	, GraphActionMenuWeak(InGraphActionMenu)
	, CommandList(MakeShared<FUICommandList>())
{
}

void FStateMachineContextMenu::BindCommands(const TSharedRef<FUICommandList>& InCommandList)
{
	CommandList->Append(InCommandList);

	const FGenericCommands& GenericCommands = FGenericCommands::Get();

	CommandList->MapAction(GenericCommands.Rename
		, FExecuteAction::CreateSP(this, &FStateMachineContextMenu::Rename)
		, FCanExecuteAction::CreateSP(this, &FStateMachineContextMenu::CanRename));

	CommandList->MapAction(GenericCommands.Delete
		, FExecuteAction::CreateSP(this, &FStateMachineContextMenu::Delete)
		, FCanExecuteAction::CreateSP(this, &FStateMachineContextMenu::CanDelete));
}

TSharedRef<SWidget> FStateMachineContextMenu::GenerateWidget()
{
	UToolMenus* const ToolMenus = UToolMenus::Get();
	check(ToolMenus);

	const FName MenuName = GetMenuName();

	if (!ToolMenus->IsMenuRegistered(MenuName))
	{
		UToolMenu* const Menu = ToolMenus->RegisterMenu(MenuName, NAME_None, EMultiBoxType::Menu);

		FToolMenuSection& Section = Menu->AddSection(TEXT("GraphActions"));

		const FGenericCommands& GenericCommands = FGenericCommands::Get();

		Section.AddMenuEntry(GenericCommands.Rename);
		Section.AddSeparator(NAME_None);
		Section.AddMenuEntry(GenericCommands.Delete);
	}

	FToolMenuContext Context(CommandList);
	return ToolMenus->GenerateWidget(MenuName, Context);
}

TSharedPtr<Graph::FBlueprintAction_Graph> FStateMachineContextMenu::GetSelectedGraphAction() const
{
	TSharedPtr<SGraphActionMenu> GraphActionMenu = GraphActionMenuWeak.Pin();
	if (!GraphActionMenu.IsValid())
	{
		return nullptr;
	}

	TArray<TSharedPtr<FEdGraphSchemaAction>> SelectedActions;
	GraphActionMenu->GetSelectedActions(SelectedActions);

	if (SelectedActions.IsEmpty())
	{
		return nullptr;
	}

	TSharedPtr<FEdGraphSchemaAction> SelectedAction(SelectedActions[0]);

	if (SelectedAction.IsValid() && SelectedAction->GetTypeId() == Graph::FBlueprintAction_Graph::StaticGetTypeId())
	{
		return StaticCastSharedPtr<Graph::FBlueprintAction_Graph>(SelectedAction);
	}

	return nullptr;
}

bool FStateMachineContextMenu::CanRename() const
{
	TSharedPtr<SGraphActionMenu> GraphActionMenu = GraphActionMenuWeak.Pin();
	return GraphActionMenu.IsValid() && GraphActionMenu->CanRequestRenameOnActionNode();
}

void FStateMachineContextMenu::Rename()
{
	TSharedPtr<SGraphActionMenu> GraphActionMenu = GraphActionMenuWeak.Pin();
	if (GraphActionMenu.IsValid() && GraphActionMenu->CanRequestRenameOnActionNode())
	{
		GraphActionMenu->OnRequestRenameOnActionNode();
	}
}

bool FStateMachineContextMenu::CanDelete() const
{
	TSharedPtr<FSceneStateBlueprintEditor> BlueprintEditor = BlueprintEditorWeak.Pin();
	if (!BlueprintEditor.IsValid() || !BlueprintEditor->InEditingMode())
	{
		return false;
	}

	TSharedPtr<Graph::FBlueprintAction_Graph> GraphAction = GetSelectedGraphAction();
	return GraphAction.IsValid() && Graph::CanDirectlyRemoveGraph(GraphAction->EdGraph);
}

void FStateMachineContextMenu::Delete()
{
	TSharedPtr<FSceneStateBlueprintEditor> BlueprintEditor = BlueprintEditorWeak.Pin();
	if (!BlueprintEditor.IsValid())
	{
		return;
	}

	USceneStateBlueprint* Blueprint = Cast<USceneStateBlueprint>(BlueprintEditor->GetBlueprintObj());
	if (!Blueprint)
	{
		return;
	}

	TSharedPtr<Graph::FBlueprintAction_Graph> GraphAction = GetSelectedGraphAction();
	if (!GraphAction.IsValid() || !Graph::CanDirectlyRemoveGraph(GraphAction->EdGraph))
	{
		return;
	}

	const UEdGraphSchema* Schema = GraphAction->EdGraph->GetSchema();
	if (Schema && Schema->TryDeleteGraph(GraphAction->EdGraph))
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("RemoveGraph", "Remove Graph"));
	Blueprint->Modify();
	GraphAction->EdGraph->Modify();

	if (GraphAction->GraphType == EEdGraphSchemaAction_K2Graph::Subgraph)
	{
		// Remove any composite nodes bound to this graph
		TArray<UK2Node_Composite*> CompositeNodes;
		FBlueprintEditorUtils::GetAllNodesOfClass<UK2Node_Composite>(Blueprint, CompositeNodes);

		for (UK2Node_Composite* CompositeNode : CompositeNodes)
		{
			if (CompositeNode->BoundGraph == GraphAction->EdGraph)
			{
				FBlueprintEditorUtils::RemoveNode(Blueprint, CompositeNode, /*bDontRecompile*/true);
			}
		}
	}

	Blueprint->StateMachineGraphs.Remove(GraphAction->EdGraph);
	Graph::RemoveGraph(GraphAction->EdGraph);

	BlueprintEditor->CloseDocumentTab(GraphAction->EdGraph);
	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);

	DiscardObject(GraphAction->EdGraph);
}

} // UE::SceneState::Editor

#undef LOCTEXT_NAMESPACE
