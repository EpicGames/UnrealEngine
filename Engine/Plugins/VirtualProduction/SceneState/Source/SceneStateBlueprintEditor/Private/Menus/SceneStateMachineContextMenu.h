// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class FUICommandList;
class SGraphActionMenu;
class SWidget;

namespace UE::SceneState
{
	namespace Editor
	{
		class FSceneStateBlueprintEditor;
	}

	namespace Graph
	{
		struct FBlueprintAction_Graph;	
	}
}

namespace UE::SceneState::Editor
{

class FStateMachineContextMenu : public TSharedFromThis<FStateMachineContextMenu>
{
public:
	explicit FStateMachineContextMenu(const TSharedRef<FSceneStateBlueprintEditor>& InBlueprintEditor, const TSharedRef<SGraphActionMenu>& InGraphActionMenu);

	void BindCommands(const TSharedRef<FUICommandList>& InCommandList);

	static FName GetMenuName()
	{
		return TEXT("SceneStateMachineContextMenu");
	}

	TSharedRef<SWidget> GenerateWidget(); 

private:
	TSharedPtr<Graph::FBlueprintAction_Graph> GetSelectedGraphAction() const;

	bool CanRename() const;
	void Rename();

	bool CanDelete() const;
	void Delete();

	TWeakPtr<FSceneStateBlueprintEditor> BlueprintEditorWeak;

	TWeakPtr<SGraphActionMenu> GraphActionMenuWeak;

	TSharedRef<FUICommandList> CommandList;
};

} // UE::SceneState::Editor
