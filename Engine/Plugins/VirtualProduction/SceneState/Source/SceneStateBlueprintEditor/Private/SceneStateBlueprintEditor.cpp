// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateBlueprintEditor.h"
#include "AppModes/SceneStateAppModes.h"
#include "AppModes/SceneStateBlueprintEditorMode.h"
#include "Debug/SceneStateDebugger.h"
#include "DetailsView/SceneStateBindingsChildrenCustomization.h"
#include "DetailsView/SceneStateMachineTaskNodeCustomization.h"
#include "DetailsView/SceneStatePropertyBindingExtension.h"
#include "Framework/Application/SlateApplication.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Nodes/SceneStateMachineTaskNode.h"
#include "SceneStateBlueprint.h"
#include "SceneStateBlueprintDelegates.h"
#include "SceneStateBlueprintEditorCommands.h"
#include "SceneStateBlueprintExtension.h"
#include "SceneStateBlueprintFactory.h"
#include "SceneStateMachineGraph.h"
#include "SceneStateObject.h"
#include "SceneStateTransitionGraph.h"
#include "ScopedTransaction.h"
#include "Tools/BaseAssetToolkit.h"
#include "Widgets/SSceneStateStateMachineMenu.h"

#define LOCTEXT_NAMESPACE "SceneStateBlueprintEditor"

namespace UE::SceneState::Editor
{

const FLazyName FSceneStateBlueprintEditor::SelectionState_StateMachine(TEXT("StateMachine"));

FSceneStateBlueprintEditor::FSceneStateBlueprintEditor()
	: Debugger(MakeShared<FDebugger>())
{
}

FSceneStateBlueprintEditor::~FSceneStateBlueprintEditor()
{
	Graph::OnBlueprintDebugObjectChanged.Remove(OnDebugObjectChangedHandle);
	OnDebugObjectChangedHandle.Reset();
}

void FSceneStateBlueprintEditor::Init(USceneStateBlueprint* InBlueprint, const FAssetOpenArgs& InOpenArgs)
{
	check(InBlueprint);

	AddBlueprintExtensions(*InBlueprint);

	OnDebugObjectChangedHandle = Graph::OnBlueprintDebugObjectChanged.AddSP(this, &FSceneStateBlueprintEditor::OnDebugObjectChanged);

	Super::InitBlueprintEditor(InOpenArgs.GetToolkitMode()
		, InOpenArgs.ToolkitHost
		, { InBlueprint }
		, /*bShouldOpenInDefaultsMode*/false);
}

void FSceneStateBlueprintEditor::AddStateMachine()
{
	USceneStateBlueprint* Blueprint = Cast<USceneStateBlueprint>(GetBlueprintObj());
	if (!Blueprint || !InEditingMode())
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("AddStateMachine", "Add State Machine"));
	Blueprint->Modify();
	USceneStateBlueprintFactory::AddStateMachine(Blueprint);
}

TSharedRef<SWidget> FSceneStateBlueprintEditor::CreateStateMachineMenu()
{
	return SAssignNew(StateMachineMenu, SStateMachineMenu, SharedThis(this));
}

void FSceneStateBlueprintEditor::AddBlueprintExtensions(USceneStateBlueprint& InBlueprint)
{
	if (!InBlueprint.FindExtension<USceneStateBlueprintExtension>())
	{
		USceneStateBlueprintExtension* BlueprintExtension = NewObject<USceneStateBlueprintExtension>(&InBlueprint);
		InBlueprint.AddExtension(BlueprintExtension);
	}
}

void FSceneStateBlueprintEditor::OnDebugObjectChanged(const Graph::FBlueprintDebugObjectChange& InDebugObjectChange)
{
	Debugger.Reset();

	if (USceneStateObject* Object = Cast<USceneStateObject>(InDebugObjectChange.DebugObject))
	{
		Debugger = MakeShared<FDebugger>();
		Debugger->AttachTo(Object);
	}
}

void FSceneStateBlueprintEditor::RefreshEditors(ERefreshBlueprintEditorReason::Type InReason)
{
	Super::RefreshEditors(InReason);

	if (StateMachineMenu.IsValid())
	{
		StateMachineMenu->RefreshMenu();
	}
}

void FSceneStateBlueprintEditor::RefreshMyBlueprint()
{
	Super::RefreshMyBlueprint();

	if (StateMachineMenu.IsValid())
	{
		StateMachineMenu->RefreshMenu();
	}
}

void FSceneStateBlueprintEditor::JumpToHyperlink(const UObject* InObjectReference, bool bInRequestRename)
{
	if (!InObjectReference)
	{
		return;
	}

	const bool bIsStateMachineObject = InObjectReference->IsA<USceneStateMachineGraph>()
		|| InObjectReference->IsA<USceneStateTransitionGraph>()
		|| InObjectReference->IsA<USceneStateMachineNode>();

	if (bIsStateMachineObject)
	{
		const USceneStateMachineNode* const TargetNode = Cast<USceneStateMachineNode>(InObjectReference);
		if (TargetNode)
		{
			InObjectReference = TargetNode->GetTypedOuter<UEdGraph>();
		}

		// Default to navigate the same document
		FDocumentTracker::EOpenDocumentCause OpenMode = FDocumentTracker::NavigatingCurrentDocument;

		// Force it to open in a new document if shift is pressed
		if (FSlateApplication::Get().GetModifierKeys().IsShiftDown())
		{
			OpenMode = FDocumentTracker::ForceOpenNewDocument;
		}

		OpenDocument(InObjectReference, OpenMode);

		if (TargetNode)
		{
			JumpToNode(TargetNode);
		}
	}
	else
	{
		Super::JumpToHyperlink(InObjectReference, bInRequestRename);
	}
}

bool FSceneStateBlueprintEditor::InEditingMode() const
{
	// Always in editing mode when not in PIE
	if (!IsPlayInEditorActive())
	{
		return true;
	}

	// Never in editing mode when debugging in bp
	if (FSlateApplication::Get().InKismetDebuggingMode())
	{
		return false;
	}

	// By default, disallow editing while in PIE unless the blueprint allows it. 
	if (const UBlueprint* Blueprint = GetBlueprintObj())
	{
		return Blueprint->CanAlwaysRecompileWhilePlayingInEditor();
	}
	return false;
}

void FSceneStateBlueprintEditor::CreateDefaultTabContents(const TArray<UBlueprint*>& InBlueprints)
{
	Super::CreateDefaultTabContents(InBlueprints);

	check(Inspector.IsValid());
	if (TSharedPtr<IDetailsView> DetailsView = Inspector->GetPropertyView())
	{
		DetailsView->RegisterInstancedCustomPropertyLayout(USceneStateMachineTaskNode::StaticClass()
			, FOnGetDetailCustomizationInstance::CreateStatic(&FStateMachineTaskNodeCustomization::MakeInstance));

		DetailsView->SetExtensionHandler(MakeShared<FBindingExtension>());
		DetailsView->SetChildrenCustomizationHandler(MakeShared<FBindingsChildrenCustomization>());
	}
}

void FSceneStateBlueprintEditor::CreateDefaultCommands()
{
	Super::CreateDefaultCommands();

	const FBlueprintEditorCommands& StateEditorCommands = FBlueprintEditorCommands::Get();

	ToolkitCommands->MapAction(StateEditorCommands.AddStateMachine
		, FExecuteAction::CreateSP(this, &FSceneStateBlueprintEditor::AddStateMachine)
		, FCanExecuteAction::CreateSP(this, &FSceneStateBlueprintEditor::InEditingMode));
}

void FSceneStateBlueprintEditor::RegisterApplicationModes(const TArray<UBlueprint*>& InBlueprints, bool bInShouldOpenInDefaultsMode, bool bInNewlyCreated)
{
	TSharedRef<FSceneStateBlueprintEditor> This = SharedThis(this);

	AddApplicationMode(FAppModes::Blueprint, MakeShared<FBlueprintAppMode>(This));
	SetCurrentMode(FAppModes::Blueprint);
}

bool FSceneStateBlueprintEditor::IsInAScriptingMode() const
{
	return true;
}

void FSceneStateBlueprintEditor::ClearSelectionStateFor(FName InSelectionOwner)
{
	if (InSelectionOwner == FSceneStateBlueprintEditor::SelectionState_StateMachine)
	{
		if (StateMachineMenu.IsValid())
		{
			StateMachineMenu->ClearSelection();
		}
	}
	else
	{
		Super::ClearSelectionStateFor(InSelectionOwner);
	}
}

FName FSceneStateBlueprintEditor::GetToolkitFName() const
{
	return TEXT("SceneStateBlueprintEditor");
}

FText FSceneStateBlueprintEditor::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Scene State Editor");
}

} // UE::SceneState::Editor

#undef LOCTEXT_NAMESPACE
