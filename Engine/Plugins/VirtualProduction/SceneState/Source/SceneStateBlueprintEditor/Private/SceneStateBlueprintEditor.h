// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintEditor.h"

class USceneStateBlueprint;
struct FAssetOpenArgs;

namespace UE::SceneState
{
	class FDebugger;

	namespace Editor
	{
		class SStateMachineMenu;
	}
	namespace Graph
	{
		struct FBlueprintDebugObjectChange;
	}
}

namespace UE::SceneState::Editor
{

/** Blueprint Editor for the Scene State Object */
class FSceneStateBlueprintEditor : public FBlueprintEditor
{
public:
	using Super = FBlueprintEditor;

	static const FLazyName SelectionState_StateMachine;

	FSceneStateBlueprintEditor();
	virtual ~FSceneStateBlueprintEditor() override;

	void Init(USceneStateBlueprint* InBlueprint, const FAssetOpenArgs& InOpenArgs);

	void AddStateMachine();

	TSharedRef<SWidget> CreateStateMachineMenu();

	void AddBlueprintExtensions(USceneStateBlueprint& InBlueprint);

	/** Called when the debug object has changed */
	void OnDebugObjectChanged(const Graph::FBlueprintDebugObjectChange& InDebugObjectChange);

	//~ Begin IBlueprintEditor
	virtual void RefreshEditors(ERefreshBlueprintEditorReason::Type InReason) override;
	virtual void RefreshMyBlueprint();
	virtual void JumpToHyperlink(const UObject* InObjectReference, bool bInRequestRename) override;
	//~ End IBlueprintEditor

	//~ Begin FBlueprintEditor
	using Super::InitBlueprintEditor;
	virtual bool InEditingMode() const;
	virtual void CreateDefaultTabContents(const TArray<UBlueprint*>& InBlueprints) override;
	virtual void CreateDefaultCommands() override;
	virtual void RegisterApplicationModes(const TArray<UBlueprint*>& InBlueprints, bool bInShouldOpenInDefaultsMode, bool bInNewlyCreated) override;
	virtual bool IsInAScriptingMode() const override;
	virtual void ClearSelectionStateFor(FName InSelectionOwner);
	//~ End FBlueprintEditor

	//~ Begin IToolkit Interface
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	//~ End IToolkit Interface

private:
	/** The current active debugger, or null if no debugger is attached */
	TSharedPtr<UE::SceneState::FDebugger> Debugger;

	TSharedPtr<SStateMachineMenu> StateMachineMenu;

	/** Handle to the delegate called when the debug object has changed */
	FDelegateHandle OnDebugObjectChangedHandle;
};

} // UE::SceneState::Editor
