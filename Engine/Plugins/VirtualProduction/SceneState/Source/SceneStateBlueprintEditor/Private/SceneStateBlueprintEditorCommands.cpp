// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateBlueprintEditorCommands.h"
#include "SceneStateBlueprintEditorStyle.h"

#define LOCTEXT_NAMESPACE "SceneStateBlueprintEditorCommands"

namespace UE::SceneState::Editor
{

FBlueprintEditorCommands::FBlueprintEditorCommands()
	: TCommands(TEXT("SceneStateBlueprintEditor")
	, LOCTEXT("SceneStateBlueprintEditor", "Scene State Blueprint Editor")
	, NAME_None
	, FBlueprintEditorStyle::Get().GetStyleSetName())
{
}

void FBlueprintEditorCommands::RegisterCommands()
{
	UI_COMMAND(AddStateMachine
		, "Add State Machine"
		, "Adds a new state machine graph"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(DebugRunSelection
		, "Run Selection"
		, "Runs the selected element (state, task) in standalone"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(DebugPushEvent
		, "Push Events"
		, "Pushes the debug events to the currently debugged object"
		, EUserInterfaceActionType::Button
		, FInputChord());
}

} // UE::SceneState::Editor

#undef LOCTEXT_NAMESPACE
