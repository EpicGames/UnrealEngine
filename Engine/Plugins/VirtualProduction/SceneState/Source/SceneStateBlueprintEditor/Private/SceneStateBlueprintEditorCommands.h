// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

namespace UE::SceneState::Editor
{

class FBlueprintEditorCommands : public TCommands<FBlueprintEditorCommands>
{
public:
	FBlueprintEditorCommands();

	//~ Begin TCommands
	virtual void RegisterCommands() override;
	//~ End TCommands

	TSharedPtr<FUICommandInfo> AddStateMachine;

	TSharedPtr<FUICommandInfo> DebugPushEvent;
	TSharedPtr<FUICommandInfo> DebugRunSelection;
};

} // UE::SceneState::Editor
