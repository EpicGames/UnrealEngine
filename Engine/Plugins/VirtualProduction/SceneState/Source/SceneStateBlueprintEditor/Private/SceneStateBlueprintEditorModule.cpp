// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateBlueprintEditorModule.h"
#include "AssetToolsModule.h"
#include "BlueprintEditorModule.h"
#include "Compiler/SceneStateBlueprintCompilerContext.h"
#include "DetailsView/SceneStateBindingFunctionCustomization.h"
#include "DetailsView/SceneStateBlueprintPropertyReferenceCustomization.h"
#include "DetailsView/SceneStateBlueprintableTaskInstanceCustomization.h"
#include "DetailsView/SceneStateMachineGraphCustomization.h"
#include "DetailsView/SceneStateMachineStateNodeCustomization.h"
#include "DetailsView/SceneStateMachineTaskInstanceCustomization.h"
#include "DetailsView/SceneStateMachineTransitionNodeCustomization.h"
#include "DetailsView/SceneStatePlayerTaskInstanceCustomization.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "KismetCompiler.h"
#include "KismetCompilerModule.h"
#include "Nodes/SceneStateMachineStateNode.h"
#include "Nodes/SceneStateMachineTaskNode.h"
#include "Nodes/SceneStateMachineTransitionNode.h"
#include "PropertyEditorModule.h"
#include "SceneStateBindingFunction.h"
#include "SceneStateBlueprint.h"
#include "SceneStateBlueprintEditorCommands.h"
#include "SceneStateBlueprintEditorLog.h"
#include "SceneStateBlueprintPropertyReference.h"
#include "SceneStateMachineGraph.h"
#include "SceneStateMachineGraphSchema.h"
#include "SceneStateObject.h"
#include "SceneStateTaskBlueprint.h"
#include "Tasks/SceneStateBlueprintableTask.h"
#include "Tasks/SceneStateBlueprintableTaskWrapper.h"
#include "Tasks/SceneStateMachineTask.h"
#include "Tasks/SceneStatePlayerTask.h"

#define LOCTEXT_NAMESPACE "SceneStateBlueprintEditorModule"

DEFINE_LOG_CATEGORY(LogSceneStateBlueprintEditor);

IMPLEMENT_MODULE(UE::SceneState::Editor::FBlueprintEditorModule, SceneStateBlueprintEditor)

namespace UE::SceneState::Editor
{

namespace Private
{

class FEmptyCustomization : public IDetailCustomization
{
public:
	static TSharedPtr<IDetailCustomization> MakeInstance(TSharedPtr<IBlueprintEditor>)
	{
		return MakeShared<FEmptyCustomization>();
	}

	//~ Begin IDetailCustomization
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override {}
	//~ End IDetailCustomization
};

void RegisterGraphCustomization()
{
	::FBlueprintEditorModule& BlueprintEditorModule = FModuleManager::LoadModuleChecked<::FBlueprintEditorModule>("Kismet");

	// Register empty customization to remove the "graph is not editable" message in the "Graph" category
	// @see FBlueprintGraphActionDetails::CustomizeDetails, FBlueprintEditorModule::CustomizeGraph
	BlueprintEditorModule.RegisterGraphCustomization(GetDefault<USceneStateMachineGraphSchema>()
		, FOnGetGraphCustomizationInstance::CreateStatic(&FEmptyCustomization::MakeInstance));
}

void UnregisterGraphCustomization()
{
	if (UObjectInitialized())
	{
		if (::FBlueprintEditorModule* BlueprintEditorModule = FModuleManager::GetModulePtr<::FBlueprintEditorModule>("Kismet"))
		{
			BlueprintEditorModule->UnregisterGraphCustomization(GetDefault<USceneStateMachineGraphSchema>());
		}
	}
}

} // Private

const FBlueprintEditorModule& FBlueprintEditorModule::GetInternal()
{
	return FModuleManager::Get().GetModuleChecked<FBlueprintEditorModule>(UE_MODULE_NAME);
}

void FBlueprintEditorModule::StartupModule()
{
	FBlueprintEditorCommands::Register();

	RegisterCompiler();
	RegisterDefaultEvents();
	RegisterDetailCustomizations();

	PostEngineInitHandle = FCoreDelegates::GetOnPostEngineInit().AddStatic(&Private::RegisterGraphCustomization);
}

void FBlueprintEditorModule::ShutdownModule()
{
	FBlueprintEditorCommands::Unregister();
	UnregisterDefaultEvents();
	UnregisterDetailCustomizations();
	Private::UnregisterGraphCustomization();

	FCoreDelegates::GetOnPostEngineInit().Remove(PostEngineInitHandle);
	PostEngineInitHandle.Reset();
}

void FBlueprintEditorModule::RegisterCompiler(TSubclassOf<USceneStateBlueprint> InBlueprintClass)
{
	FKismetCompilerContext::RegisterCompilerForBP(InBlueprintClass,
		[](UBlueprint* InBlueprint, FCompilerResultsLog& InMessageLog, const FKismetCompilerOptions& InCompileOptions)
		{
			return MakeShared<FBlueprintCompilerContext>(CastChecked<USceneStateBlueprint>(InBlueprint), InMessageLog, InCompileOptions);
		});
}

void FBlueprintEditorModule::RegisterContextEditor(const TSharedPtr<IContextEditor>& InContextEditor)
{
	ContextEditorRegistry.RegisterContextEditor(InContextEditor);
}

void FBlueprintEditorModule::UnregisterContextEditor(const TSharedPtr<IContextEditor>& InContextEditor)
{
	ContextEditorRegistry.UnregisterContextEditor(InContextEditor);
}

void FBlueprintEditorModule::RegisterCompiler()
{
	IKismetCompilerInterface& KismetCompilerModule = FModuleManager::LoadModuleChecked<IKismetCompilerInterface>("KismetCompiler");
	KismetCompilerModule.OverrideBPTypeForClass(USceneStateObject::StaticClass(), USceneStateBlueprint::StaticClass());
	KismetCompilerModule.OverrideBPTypeForClass(USceneStateBlueprintableTask::StaticClass(), USceneStateTaskBlueprint::StaticClass());

	for (UClass* Class : TObjectRange<UClass>())
	{
		if (Class && Class->IsChildOf<USceneStateBlueprint>())
		{
			FBlueprintEditorModule::RegisterCompiler(Class);
		}
	}
}

void FBlueprintEditorModule::RegisterDefaultEvents()
{
	UClass* const StateClass = USceneStateObject::StaticClass();
	FKismetEditorUtilities::RegisterAutoGeneratedDefaultEvent(this, StateClass, GET_FUNCTION_NAME_CHECKED(USceneStateObject, ReceiveEnter));
	FKismetEditorUtilities::RegisterAutoGeneratedDefaultEvent(this, StateClass, GET_FUNCTION_NAME_CHECKED(USceneStateObject, ReceiveTick));
	FKismetEditorUtilities::RegisterAutoGeneratedDefaultEvent(this, StateClass, GET_FUNCTION_NAME_CHECKED(USceneStateObject, ReceiveExit));

	UClass* const TaskClass = USceneStateBlueprintableTask::StaticClass();
	FKismetEditorUtilities::RegisterAutoGeneratedDefaultEvent(this, TaskClass, GET_FUNCTION_NAME_CHECKED(USceneStateBlueprintableTask, ReceiveStart));
	FKismetEditorUtilities::RegisterAutoGeneratedDefaultEvent(this, TaskClass, GET_FUNCTION_NAME_CHECKED(USceneStateBlueprintableTask, ReceiveTick));
	FKismetEditorUtilities::RegisterAutoGeneratedDefaultEvent(this, TaskClass, GET_FUNCTION_NAME_CHECKED(USceneStateBlueprintableTask, ReceiveStop));
}

void FBlueprintEditorModule::UnregisterDefaultEvents()
{
	FKismetEditorUtilities::UnregisterAutoBlueprintNodeCreation(this);
}

void FBlueprintEditorModule::RegisterDetailCustomizations()
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	PropertyEditorModule.RegisterCustomPropertyTypeLayout(CustomizedTypes.Add_GetRef(FSceneStateBlueprintableTaskInstance::StaticStruct()->GetFName())
		, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FBlueprintableTaskInstanceCustomization::MakeInstance));

	PropertyEditorModule.RegisterCustomPropertyTypeLayout(CustomizedTypes.Add_GetRef(FSceneStateMachineTaskInstance::StaticStruct()->GetFName())
		, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FStateMachineTaskInstanceCustomization::MakeInstance));

	PropertyEditorModule.RegisterCustomPropertyTypeLayout(CustomizedTypes.Add_GetRef(FSceneStatePlayerTaskInstance::StaticStruct()->GetFName())
		, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FPlayerTaskInstanceCustomization::MakeInstance));

	PropertyEditorModule.RegisterCustomPropertyTypeLayout(CustomizedTypes.Add_GetRef(FSceneStateBlueprintPropertyReference::StaticStruct()->GetFName())
		, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FBlueprintPropertyReferenceCustomization::MakeInstance));

	PropertyEditorModule.RegisterCustomPropertyTypeLayout(CustomizedTypes.Add_GetRef(FSceneStateBindingFunction::StaticStruct()->GetFName())
		, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FBindingFunctionCustomization::MakeInstance));

	PropertyEditorModule.RegisterCustomClassLayout(CustomizedClasses.Add_GetRef(USceneStateMachineGraph::StaticClass()->GetFName())
		, FOnGetDetailCustomizationInstance::CreateStatic(&FStateMachineGraphCustomization::MakeInstance));

	PropertyEditorModule.RegisterCustomClassLayout(CustomizedClasses.Add_GetRef(USceneStateMachineStateNode::StaticClass()->GetFName())
		, FOnGetDetailCustomizationInstance::CreateStatic(&FStateMachineStateNodeCustomization::MakeInstance));

	PropertyEditorModule.RegisterCustomClassLayout(CustomizedClasses.Add_GetRef(USceneStateMachineTransitionNode::StaticClass()->GetFName())
		, FOnGetDetailCustomizationInstance::CreateStatic(&FStateMachineTransitionNodeCustomization::MakeInstance));
}

void FBlueprintEditorModule::UnregisterDetailCustomizations()
{
	if (FPropertyEditorModule* PropertyEditorModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
	{
		for (FName CustomizedType : CustomizedTypes)
		{
			PropertyEditorModule->UnregisterCustomPropertyTypeLayout(CustomizedType);
		}
		CustomizedTypes.Reset();

		for (FName CustomizedClass : CustomizedClasses)
		{
			PropertyEditorModule->UnregisterCustomClassLayout(CustomizedClass);
		}
		CustomizedClasses.Reset();
	}
}

} // UE::SceneState::Editor

#undef LOCTEXT_NAMESPACE
