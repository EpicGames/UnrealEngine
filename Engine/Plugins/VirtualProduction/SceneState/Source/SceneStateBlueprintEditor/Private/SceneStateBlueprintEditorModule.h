// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISceneStateBlueprintEditorModule.h"
#include "SceneStateContextEditorRegistry.h"

namespace UE::SceneState::Editor
{

class FBlueprintEditorModule : public IBlueprintEditorModule
{
public:
	static const FBlueprintEditorModule& GetInternal();

	const FContextEditorRegistry& GetContextEditorRegistry() const
	{
		return ContextEditorRegistry;
	}

protected:
	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface

	//~ Begin IBlueprintEditorModule
	virtual void RegisterCompiler(TSubclassOf<USceneStateBlueprint> InBlueprintClass) override;
	virtual void RegisterContextEditor(const TSharedPtr<IContextEditor>& InContextEditor) override;
	virtual void UnregisterContextEditor(const TSharedPtr<IContextEditor>& InContextEditor) override;
	//~ End IBlueprintEditorModule

	void RegisterCompiler();

	void RegisterDefaultEvents();
	void UnregisterDefaultEvents();

	void RegisterDetailCustomizations();
	void UnregisterDetailCustomizations();

	FContextEditorRegistry ContextEditorRegistry;

	TArray<FName> CustomizedTypes;
	TArray<FName> CustomizedClasses;

	FDelegateHandle PostEngineInitHandle;
};

} // UE::SceneState::Editor
