// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateBlueprintEditorStyle.h"
#include "EdGraphSchema_K2.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Templates/SubclassOf.h"
#include "Textures/SlateIcon.h"

namespace UE::SceneState::Editor
{

FBlueprintEditorStyle::FBlueprintEditorStyle()
	: FSlateStyleSet(TEXT("SceneStateBlueprintEditor"))
{
	ContentRootDir = FPaths::EngineContentDir() / TEXT("Editor/Slate");
	CoreContentRootDir = FPaths::EngineContentDir() / TEXT("Slate");

	// Commands
	Set("SceneStateBlueprintEditor.AddStateMachine", new IMAGE_BRUSH_SVG("Starship/GraphEditors/StateMachine", CoreStyleConstants::Icon16x16));
	Set("SceneStateBlueprintEditor.DebugRunSelection", new IMAGE_BRUSH_SVG("Starship/MainToolbar/Simulate", CoreStyleConstants::Icon16x16));
	Set("SceneStateBlueprintEditor.DebugPushEvent", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Export", CoreStyleConstants::Icon16x16));

	// Schema Icons
	Set("SchemaIcon.SceneStateMachineGraphSchema", new IMAGE_BRUSH_SVG("Starship/GraphEditors/StateMachine", CoreStyleConstants::Icon16x16));

	// Class Thumbnails/Icons
	Set("ClassThumbnail.SceneStateObject", new IMAGE_BRUSH_SVG("Starship/GraphEditors/StateMachine", CoreStyleConstants::Icon64x64));
	Set("ClassIcon.SceneStateObject", new IMAGE_BRUSH_SVG("Starship/GraphEditors/StateMachine", CoreStyleConstants::Icon16x16));

	// Task Icons
	Set("TaskIcon.SceneStateTask", new IMAGE_BRUSH("Icons/AssetIcons/Default_16x", CoreStyleConstants::Icon16x16));
	Set("TaskIcon.SceneStateMachineTask", new IMAGE_BRUSH_SVG("Starship/GraphEditors/StateMachine", CoreStyleConstants::Icon16x16));
	Set("TaskIcon.SceneStateBlueprintableTaskWrapper", new IMAGE_BRUSH_SVG("Starship/AssetIcons/Blueprint_16", CoreStyleConstants::Icon16x16));
	Set("TaskIcon.SceneStateDelayTask", new IMAGE_BRUSH_SVG("Starship/Common/Timecode", CoreStyleConstants::Icon16x16));
	Set("TaskIcon.SceneStatePrintStringTask", new CORE_IMAGE_BRUSH_SVG("Starship/Common/OutputLog", CoreStyleConstants::Icon16x16));
	Set("TaskIcon.SceneStateSpawnActorTask", new IMAGE_BRUSH_SVG("Starship/AssetIcons/Actor_16", CoreStyleConstants::Icon16x16));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FBlueprintEditorStyle::~FBlueprintEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

FSlateIcon FBlueprintEditorStyle::GetGraphSchemaIcon(TSubclassOf<UEdGraphSchema> InSchemaClass) const
{
	if (!InSchemaClass)
	{
		return FSlateIcon();
	}

	const FString BasePath(TEXT("SchemaIcon."));

	while (InSchemaClass)
	{
		const FName IconPath = *(BasePath + InSchemaClass->GetName());
		if (GetOptionalBrush(IconPath, nullptr, nullptr))
		{
			return FSlateIcon(GetStyleSetName(), IconPath);
		}
		InSchemaClass = InSchemaClass->GetSuperClass();
	}

	return FSlateIcon();
}

} // UE::SceneState::Editor
