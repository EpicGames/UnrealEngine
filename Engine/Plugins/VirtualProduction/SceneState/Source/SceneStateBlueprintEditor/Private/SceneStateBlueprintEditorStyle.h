// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class UEdGraphSchema;
struct FSlateIcon;
template <typename T> class TSubclassOf;

namespace UE::SceneState::Editor
{

class FBlueprintEditorStyle : public FSlateStyleSet
{
public:
	static FBlueprintEditorStyle& Get()
	{
		static FBlueprintEditorStyle Instance;
		return Instance;
	}

	FBlueprintEditorStyle();
	virtual ~FBlueprintEditorStyle() override;

	FSlateIcon GetGraphSchemaIcon(TSubclassOf<UEdGraphSchema> InSchemaClass) const;
};

} // UE::SceneState::Editor
