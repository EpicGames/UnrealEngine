// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateBlueprintEditorUtils.h"
#include "DetailsView/SceneStateInstancedStructDataProvider.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailGroup.h"
#include "Misc/Guid.h"
#include "Nodes/SceneStateMachineTaskNode.h"
#include "PropertyBindingExtension.h"
#include "PropertyHandle.h"
#include "StructUtils/PropertyBag.h"

namespace UE::SceneState::Editor
{

FGuid FindTaskId(const TSharedRef<IPropertyHandle>& InPropertyHandle)
{
	TArray<UObject*> OuterObjects;
	InPropertyHandle->GetOuterObjects(OuterObjects);
	if (OuterObjects.Num() != 1)
	{
		return FGuid();
	}

	if (USceneStateMachineTaskNode* TaskNode = Cast<USceneStateMachineTaskNode>(OuterObjects[0]))
	{
		return TaskNode->GetTaskId();
	}
	return FGuid();
}

bool IsObjectPropertyOfClass(const FProperty* InProperty, const UClass* InClass)
{
	const FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(InProperty);
	return ObjectProperty && ObjectProperty->PropertyClass && ObjectProperty->PropertyClass->IsChildOf(InClass);
}

bool IsStruct(const FProperty* InProperty, const UScriptStruct* InStruct)
{
	const FStructProperty* StructProperty = CastField<FStructProperty>(InProperty);
	return StructProperty && StructProperty->Struct->IsChildOf(InStruct);
}

FPropertyAccess::Result GetGuid(const TSharedRef<IPropertyHandle>& InGuidPropertyHandle, FGuid& OutGuid)
{
	const FStructProperty* StructProperty = CastField<FStructProperty>(InGuidPropertyHandle->GetProperty());
	if (!StructProperty || StructProperty->Struct != TBaseStructure<FGuid>::Get())
	{
		return FPropertyAccess::Fail;
	}

	FPropertyAccess::Result Result = FPropertyAccess::Fail;

	InGuidPropertyHandle->EnumerateConstRawData(
		[&OutGuid, &Result](const void* InRawData, const int32 InDataIndex, const int32)->bool
		{
			const FGuid& CurrentGuid = *static_cast<const FGuid*>(InRawData);
			if (InDataIndex == 0)
			{
				OutGuid = CurrentGuid;
				Result = FPropertyAccess::Success;
			}
			else if (OutGuid != CurrentGuid)
			{
				OutGuid = FGuid();
				Result = FPropertyAccess::MultipleValues;
				return false; // break
			}
			return true; // continue
		});

	return Result;
}

TSharedRef<IStructureDataProvider> CreateInstancedStructDataProvider(const TSharedRef<IPropertyHandle>& InPropertyHandle)
{
	check(IsStruct(InPropertyHandle->GetProperty(), FInstancedStruct::StaticStruct()));
	return MakeShared<FInstancedStructDataProvider>(InPropertyHandle);
}

bool CompareParametersLayout(const FInstancedPropertyBag& InParametersA, const FInstancedPropertyBag& InParametersB)
{
	if (InParametersA.GetNumPropertiesInBag() != InParametersB.GetNumPropertiesInBag())
	{
		return false;
	}

	const UPropertyBag* BagA = InParametersA.GetPropertyBagStruct();
	const UPropertyBag* BagB = InParametersB.GetPropertyBagStruct();

	if (!BagA || !BagB)
	{
		return BagA == BagB;
	}

	const TConstArrayView<FPropertyBagPropertyDesc> DescsA = BagA->GetPropertyDescs();
	const TConstArrayView<FPropertyBagPropertyDesc> DescsB = BagB->GetPropertyDescs();

	for (int32 Index = 0; Index < DescsA.Num(); ++Index)
	{
		if (DescsA[Index].Name != DescsB[Index].Name || !DescsA[Index].CompatibleType(DescsB[Index]))
		{
			return false;
		}
	}

	return true;
}

void AssignBindingId(const TSharedRef<IPropertyHandle>& InPropertyHandle, const FGuid& InTaskId)
{
	InPropertyHandle->SetInstanceMetaData(UE::PropertyBinding::MetaDataStructIDName, LexToString(InTaskId));
}

UClass* FindCommonBase(const TSharedRef<IPropertyHandle>& InPropertyHandle)
{
	UClass* CommonBase = nullptr;

	FObjectProperty* ObjectProperty = CastField<FObjectProperty>(InPropertyHandle->GetProperty());
	if (!ObjectProperty)
	{
		return nullptr;
	}

	InPropertyHandle->EnumerateConstRawData(
		[&CommonBase, ObjectProperty](const void* InRawData, const int32, const int32)
		{
			if (InRawData)
			{
				if (const UObject* Object = ObjectProperty->GetObjectPropertyValue(InRawData))
				{
					UClass* ObjectClass = Object->GetClass();
					CommonBase = CommonBase ? UClass::FindCommonBase(CommonBase, ObjectClass) : ObjectClass;
				}
			}
			return true; // Continue
		});

	return CommonBase;
}

bool AddObjectProperties(const TSharedRef<IPropertyHandle>& InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, uint64 InDisallowedFlags)
{
	const UClass* ObjectClass = FindCommonBase(InPropertyHandle);
	if (!ObjectClass)
	{
		return false;
	}

	TMap<FName, IDetailGroup*> PropertyGroups;

	for (const FProperty* Property : TFieldRange<FProperty>(ObjectClass, EFieldIterationFlags::IncludeSuper))
	{
		if (Property->HasAnyPropertyFlags(InDisallowedFlags))
		{
			continue;
		}

		TSharedPtr<IPropertyHandle> PropertyHandle = InPropertyHandle->GetChildHandle(Property->GetFName());
		if (!PropertyHandle.IsValid() || !PropertyHandle->IsValidHandle())
		{
			continue;
		}

		const FName CategoryName = PropertyHandle->GetDefaultCategoryName();

		IDetailGroup* DetailGroup = PropertyGroups.FindRef(CategoryName);
		if (!DetailGroup)
		{
			DetailGroup = &InChildBuilder.AddGroup(CategoryName, PropertyHandle->GetDefaultCategoryText());
			DetailGroup->ToggleExpansion(/*bExpand*/true);
			PropertyGroups.Add(CategoryName, DetailGroup);
		}
		check(DetailGroup);

		DetailGroup->AddPropertyRow(PropertyHandle.ToSharedRef());
	}

	return true;
}

} // UE::SceneState::Editor
