// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateBlueprintFactory.h"
#include "AssetToolsModule.h"
#include "ClassViewerFilter.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Kismet2/SClassPickerDialog.h"
#include "Misc/MessageDialog.h"
#include "SceneStateBlueprint.h"
#include "SceneStateBlueprintEditor.h"
#include "SceneStateGeneratedClass.h"
#include "SceneStateMachineGraph.h"
#include "SceneStateMachineGraphSchema.h"
#include "SceneStateObject.h"
#include "SceneStateSchema.h"

#define LOCTEXT_NAMESPACE "SceneStateBlueprintFactory"

namespace UE::SceneState::Editor
{

class FClassViewFilter : public IClassViewerFilter
{
public:
	FClassViewFilter()
		: AllowedChildrenOfClasses({ USceneStateSchema::StaticClass() })
	{
	}

	virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs) override
	{
		return !InClass->HasAnyClassFlags(EClassFlags::CLASS_Deprecated | EClassFlags::CLASS_Abstract)
			&& InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InClass);
	}

	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef<const IUnloadedBlueprintData> InUnloadedClassData, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs) override
	{
		return !InUnloadedClassData->HasAnyClassFlags(EClassFlags::CLASS_Deprecated | EClassFlags::CLASS_Abstract)
			&& InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InUnloadedClassData);
	}

private:
	/** All children of these classes will be included unless filtered out by another setting. */
	TSet<const UClass*> AllowedChildrenOfClasses;
};

} // UE::SceneState::Editor

USceneStateBlueprintFactory::USceneStateBlueprintFactory()
{
	ParentClass = USceneStateObject::StaticClass();
	SupportedClass = USceneStateBlueprint::StaticClass();
	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

UEdGraph* USceneStateBlueprintFactory::AddStateMachine(USceneStateBlueprint* InBlueprint)
{
	check(InBlueprint);

	const FName GraphName = FBlueprintEditorUtils::FindUniqueKismetName(InBlueprint, TEXT("State Machine"));

	UEdGraph* NewStateMachineGraph = FBlueprintEditorUtils::CreateNewGraph(InBlueprint
		, GraphName
		, USceneStateMachineGraph::StaticClass()
		, USceneStateMachineGraphSchema::StaticClass());

	// Allocate Default State Machine Nodes (i.e. Entry node)
	const UEdGraphSchema* Schema = NewStateMachineGraph->GetSchema();
	check(Schema);
	Schema->CreateDefaultNodesForGraph(*NewStateMachineGraph);

	InBlueprint->StateMachineGraphs.Add(NewStateMachineGraph);

	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(InBlueprint);

	return NewStateMachineGraph;
}

FText USceneStateBlueprintFactory::GetDisplayName() const
{
	return SupportedClass ? SupportedClass->GetDisplayNameText() : Super::GetDisplayName();
}

FString USceneStateBlueprintFactory::GetDefaultNewAssetName() const
{
	// Short name removing "Motion Design" and "SceneState" prefix for new assets
	return TEXT("NewBlueprint");
}

uint32 USceneStateBlueprintFactory::GetMenuCategories() const
{
	IAssetTools& AssetTools = FAssetToolsModule::GetModule().Get();
	return AssetTools.FindAdvancedAssetCategory("MotionDesignCategory");
}

bool USceneStateBlueprintFactory::ConfigureProperties()
{
	SchemaClass = nullptr;

	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

	FClassViewerInitializationOptions Options;
	Options.Mode = EClassViewerMode::ClassPicker;
	Options.DisplayMode = EClassViewerDisplayMode::TreeView;
	Options.NameTypeToDisplay = EClassViewerNameTypeToDisplay::DisplayName;
	Options.ClassFilters.Emplace(MakeShared<UE::SceneState::Editor::FClassViewFilter>());

	const FText TitleText = LOCTEXT("SelectSceneStateSchema", "Select Scene State Schema");

	UClass* ChosenSchema = nullptr;
	if (SClassPickerDialog::PickClass(TitleText, Options, ChosenSchema, USceneStateSchema::StaticClass()))
	{
		SchemaClass = ChosenSchema;
		return true;
	}
	return false;
}

UObject* USceneStateBlueprintFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, FFeedbackContext* InWarn)
{
	return FactoryCreateNew(InClass, InParent, InName, InFlags, InContext, InWarn, NAME_None);
}

UObject* USceneStateBlueprintFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, FFeedbackContext* InWarn, FName InCallingContext)
{
	check(InClass && InClass->IsChildOf<USceneStateBlueprint>());

	if (!FKismetEditorUtilities::CanCreateBlueprintOfClass(ParentClass) || !ParentClass->IsChildOf<USceneStateObject>())
	{
		FMessageDialog::Open(EAppMsgType::Ok
			, FText::Format(LOCTEXT("InvalidParentClassMessage", "Unable to create Scene State Blueprint with parent class '{0}'.")
			, FText::FromString(GetNameSafe(ParentClass))));
		return nullptr;
	}

	// Create Blueprint
	USceneStateBlueprint* Blueprint = CastChecked<USceneStateBlueprint>(FKismetEditorUtilities::CreateBlueprint(ParentClass
		, InParent
		, InName
		, EBlueprintType::BPTYPE_Normal
		, InClass
		, USceneStateGeneratedClass::StaticClass()
		, InCallingContext));

	check(Blueprint && Blueprint->GeneratedClass);

	checkf(Cast<USceneStateGeneratedClass>(Blueprint->GeneratedClass) != nullptr
		, TEXT("Scene State Blueprint generated class is not properly set up for %s.\n"
			"Ensure that this Scene State Blueprint class has a Scene State compiler registered via ISceneStateBlueprintEditorModule")
		, *GetNameSafe(Blueprint->GetClass()));

	Blueprint->SetSceneStateSchema(SchemaClass);

	UEdGraph* StateMachineGraph = USceneStateBlueprintFactory::AddStateMachine(Blueprint);

	Blueprint->LastEditedDocuments.AddUnique(StateMachineGraph);

	return Blueprint;
}

#undef LOCTEXT_NAMESPACE
