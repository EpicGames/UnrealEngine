// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateContextEditorRegistry.h"
#include "ISceneStateContextEditor.h"
#include "Templates/SubclassOf.h"
#include "UObject/Object.h"

namespace UE::SceneState::Editor
{

void FContextEditorRegistry::RegisterContextEditor(const TSharedPtr<IContextEditor>& InContextEditor)
{
	if (InContextEditor.IsValid())
	{
		ContextEditors.Add(InContextEditor);
	}
}

void FContextEditorRegistry::UnregisterContextEditor(const TSharedPtr<IContextEditor>& InContextEditor)
{
	ContextEditors.Remove(InContextEditor);
}

TSharedPtr<IContextEditor> FContextEditorRegistry::FindContextEditor(const UObject* InContextObject) const
{
	if (!InContextObject)
	{
		return nullptr;
	}

	const TSubclassOf<UObject> ExactContextClass = InContextObject->GetClass();

	TSharedPtr<IContextEditor> SelectedEditor;
	TSubclassOf<UObject> SelectedClass;

	TArray<TSubclassOf<UObject>> ContextClasses;

	// Find the context editor closest in match to the context editor
	for (const TSharedPtr<IContextEditor>& ContextEditor : ContextEditors)
	{
		ContextEditor->GetContextClasses(ContextClasses);
		if (ContextClasses.IsEmpty())
		{
			continue;
		}

		for (TSubclassOf<UObject> ContextClass : ContextClasses)
		{
			// Found exact match, return
			if (ContextClass == ExactContextClass)
			{
				return ContextEditor;
			}

			// Select the context class if it's a child of the exact class and if it's a class closer to the exact class than the currently selected class
			if (ExactContextClass->IsChildOf(ContextClass) && (!SelectedClass || ContextClass->IsChildOf(SelectedClass)))
			{
				SelectedClass = ContextClass;
				SelectedEditor = ContextEditor;
			}
		}
	}

	return SelectedEditor;
}

} // UE::SceneState::Editor
