// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Templates/SharedPointer.h"

namespace UE::SceneState::Editor
{
	class IContextEditor;
}

namespace UE::SceneState::Editor
{

/** Holds all the registered context editors */
class FContextEditorRegistry
{
public:
	/** Registers the given context editor if valid */
	void RegisterContextEditor(const TSharedPtr<IContextEditor>& InContextEditor);

	/** Unregisters the given context editor */
	void UnregisterContextEditor(const TSharedPtr<IContextEditor>& InContextEditor);

	/** Finds the most relevant context editor to the given context object */
	TSharedPtr<IContextEditor> FindContextEditor(const UObject* InContextObject) const;

private:
	TArray<TSharedPtr<IContextEditor>> ContextEditors;
};

} // UE::SceneState::Editor
