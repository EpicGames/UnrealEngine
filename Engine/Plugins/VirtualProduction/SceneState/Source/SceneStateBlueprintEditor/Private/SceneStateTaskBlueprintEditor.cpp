// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateTaskBlueprintEditor.h"
#include "SceneState.h"
#include "SceneStateMachine.h"
#include "SceneStateTaskBlueprint.h"
#include "Tasks/SceneStateBlueprintableTask.h"

#define LOCTEXT_NAMESPACE "SceneStateTaskBlueprintEditor"

namespace UE::SceneState::Editor
{

void FSceneStateTaskBlueprintEditor::Init(USceneStateTaskBlueprint* InBlueprint, const FAssetOpenArgs& InOpenArgs)
{
	check(InBlueprint);

	Super::InitBlueprintEditor(InOpenArgs.GetToolkitMode()
		, InOpenArgs.ToolkitHost
		, { InBlueprint }
		, /*bShouldOpenInDefaultsMode*/false);
}

FName FSceneStateTaskBlueprintEditor::GetToolkitFName() const
{
	return TEXT("SceneStateTaskBlueprintEditor");
}

FText FSceneStateTaskBlueprintEditor::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Scene State Task Editor");
}

} // UE::SceneState::Editor

#undef LOCTEXT_NAMESPACE
