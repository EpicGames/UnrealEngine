// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintEditor.h"

class USceneStateTaskBlueprint;

namespace UE::SceneState::Editor
{

class FSceneStateTaskBlueprintEditor : public FBlueprintEditor
{
public:
	using Super = FBlueprintEditor;

	void Init(USceneStateTaskBlueprint* InBlueprint, const FAssetOpenArgs& InOpenArgs);

protected:
	//~ Begin IToolkit Interface
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	//~ End IToolkit Interface
};

} // UE::SceneState::Editor
