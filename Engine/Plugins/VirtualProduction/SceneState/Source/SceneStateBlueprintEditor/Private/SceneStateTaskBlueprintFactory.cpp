// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateTaskBlueprintFactory.h"
#include "AssetToolsModule.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Misc/MessageDialog.h"
#include "SceneStateTaskBlueprint.h"
#include "Tasks/SceneStateBlueprintableTask.h"
#include "Tasks/SceneStateTaskGeneratedClass.h"

#define LOCTEXT_NAMESPACE "SceneStateTaskBlueprintFactory"

USceneStateTaskBlueprintFactory::USceneStateTaskBlueprintFactory()
{
	ParentClass = USceneStateBlueprintableTask::StaticClass();
	SupportedClass = USceneStateTaskBlueprint::StaticClass();
	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

FText USceneStateTaskBlueprintFactory::GetDisplayName() const
{
	return SupportedClass ? SupportedClass->GetDisplayNameText() : Super::GetDisplayName();
}

FString USceneStateTaskBlueprintFactory::GetDefaultNewAssetName() const
{
	// Short name removing "Motion Design" and "SceneState" prefix for new assets
	return TEXT("NewTaskBlueprint");
}

uint32 USceneStateTaskBlueprintFactory::GetMenuCategories() const
{
	IAssetTools& AssetTools = FAssetToolsModule::GetModule().Get();
	return AssetTools.FindAdvancedAssetCategory("MotionDesignCategory");
}

UObject* USceneStateTaskBlueprintFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, FFeedbackContext* InWarn, FName InCallingContext)
{
	check(InClass && InClass->IsChildOf<USceneStateTaskBlueprint>());

	if (!FKismetEditorUtilities::CanCreateBlueprintOfClass(ParentClass) || !ParentClass->IsChildOf<USceneStateBlueprintableTask>())
	{
		FMessageDialog::Open(EAppMsgType::Ok
			, FText::Format(LOCTEXT("InvalidParentClassMessage", "Unable to create Scene State Script Task Blueprint with parent class '{0}'.")
			, FText::FromString(GetNameSafe(ParentClass))));
		return nullptr;
	}

	// Create Blueprint
	USceneStateTaskBlueprint* Blueprint = CastChecked<USceneStateTaskBlueprint>(FKismetEditorUtilities::CreateBlueprint(ParentClass
		, InParent
		, InName
		, EBlueprintType::BPTYPE_Normal
		, USceneStateTaskBlueprint::StaticClass()
		, USceneStateTaskGeneratedClass::StaticClass()
		, InCallingContext));

	return Blueprint;
}

#undef LOCTEXT_NAMESPACE
