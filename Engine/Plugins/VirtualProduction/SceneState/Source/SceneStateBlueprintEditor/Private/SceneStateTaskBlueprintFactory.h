// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "Templates/SubclassOf.h"
#include "SceneStateTaskBlueprintFactory.generated.h"

class USceneStateBlueprintableTask;

UCLASS()
class USceneStateTaskBlueprintFactory : public UFactory
{
	GENERATED_BODY()

public:
	USceneStateTaskBlueprintFactory();

	//~ Begin UFactory
	virtual FText GetDisplayName() const override;
	virtual FString GetDefaultNewAssetName() const override;
	virtual uint32 GetMenuCategories() const override;
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, FFeedbackContext* InWarn, FName InCallingContext) override;
	//~ End UFactory

	UPROPERTY()
	TSubclassOf<USceneStateBlueprintableTask> ParentClass;
};
