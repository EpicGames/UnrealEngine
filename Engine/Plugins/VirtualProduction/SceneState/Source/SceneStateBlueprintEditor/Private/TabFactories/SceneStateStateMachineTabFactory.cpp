// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateStateMachineTabFactory.h"
#include "SceneStateBlueprintEditor.h"
#include "Widgets/SNullWidget.h"

#define LOCTEXT_NAMESPACE "SceneStateStateMachineTabFactory"

namespace UE::SceneState::Editor
{

const FName FStateMachineTabFactory::TabId(TEXT("SceneStateStateMachine"));

FStateMachineTabFactory::FStateMachineTabFactory(const TSharedRef<FSceneStateBlueprintEditor>& InEditor)
	: FTabFactory(TabId, InEditor)
{
	TabIcon             = FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.StateMachine_16x");
	TabLabel            = LOCTEXT("TabLabel", "State Machines");
	ViewMenuTooltip     = LOCTEXT("ViewMenuTooltip", "State Machines");
	ViewMenuDescription = LOCTEXT("ViewMenuDescription", "State Machines");
	bIsSingleton        = true;
}

TSharedRef<SWidget> FStateMachineTabFactory::CreateTabBody(const FWorkflowTabSpawnInfo& InInfo) const
{
	TSharedPtr<FSceneStateBlueprintEditor> Editor = GetEditor();
	if (!Editor.IsValid())
	{
		return SNullWidget::NullWidget;
	}
	return Editor->CreateStateMachineMenu();
}

} // UE::SceneState::Editor

#undef LOCTEXT_NAMESPACE 
