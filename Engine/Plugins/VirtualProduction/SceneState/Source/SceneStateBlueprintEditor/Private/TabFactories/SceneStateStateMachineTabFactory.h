// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneStateTabFactory.h"

namespace UE::SceneState::Editor
{

class FStateMachineTabFactory : public FTabFactory
{
public:
	static const FName TabId;

	explicit FStateMachineTabFactory(const TSharedRef<FSceneStateBlueprintEditor>& InEditor);

	//~ Begin FWorkflowTabFactory
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& InInfo) const override;
	//~ End FWorkflowTabFactory
};

} // UE::SceneState::Editor
