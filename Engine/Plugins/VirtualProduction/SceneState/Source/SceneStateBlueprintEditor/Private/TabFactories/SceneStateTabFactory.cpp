// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateTabFactory.h"
#include "SceneStateBlueprintEditor.h"

namespace UE::SceneState::Editor
{

FTabFactory::FTabFactory(FName InTabId, const TSharedRef<FSceneStateBlueprintEditor>& InEditor)
	: FWorkflowTabFactory(InTabId, InEditor)
{
}

TSharedPtr<FSceneStateBlueprintEditor> FTabFactory::GetEditor() const
{
	return StaticCastSharedPtr<FSceneStateBlueprintEditor>(HostingApp.Pin());
}

} // UE::SceneState::Editor
