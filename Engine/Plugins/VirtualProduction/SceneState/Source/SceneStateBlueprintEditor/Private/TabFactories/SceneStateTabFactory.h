// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorkflowOrientedApp/WorkflowTabFactory.h"

namespace UE::SceneState::Editor
{
	class FSceneStateBlueprintEditor;
}

namespace UE::SceneState::Editor
{

/** Base class for all Tab Factories in Scene State Blueprint Editor */
class FTabFactory : public FWorkflowTabFactory
{
public:
	explicit FTabFactory(FName InTabId, const TSharedRef<FSceneStateBlueprintEditor>& InEditor);

	TSharedPtr<FSceneStateBlueprintEditor> GetEditor() const;
};

} // UE::SceneState::Editor
