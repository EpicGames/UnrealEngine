// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/SceneStateBlueprintableTaskDesc.h"
#include "SceneStateEditorUtils.h"
#include "Tasks/SceneStateBlueprintableTask.h"
#include "Tasks/SceneStateBlueprintableTaskWrapper.h"

FSceneStateBlueprintableTaskDesc::FSceneStateBlueprintableTaskDesc()
{
	SetSupportedTask<FSceneStateBlueprintableTaskWrapper>();
}

bool FSceneStateBlueprintableTaskDesc::OnGetDisplayName(const FSceneStateTaskDescContext& InContext, FText& OutDisplayName) const
{
	const FSceneStateBlueprintableTaskWrapper& Task = InContext.Task.Get<FSceneStateBlueprintableTaskWrapper>();
	if (const UClass* TaskClass = Task.GetTaskClass())
	{
		OutDisplayName = TaskClass->GetDisplayNameText();
		return true;
	}
	return false;
}

bool FSceneStateBlueprintableTaskDesc::OnGetTooltip(const FSceneStateTaskDescContext& InContext, FText& OutDescription) const
{
	const FSceneStateBlueprintableTaskWrapper& Task = InContext.Task.Get<FSceneStateBlueprintableTaskWrapper>();
	if (const UClass* TaskClass = Task.GetTaskClass())
	{
		OutDescription = UE::SceneState::Editor::GetStructTooltip(*TaskClass);
		return true;
	}
	return false;
}

bool FSceneStateBlueprintableTaskDesc::OnGetJumpTarget(const FSceneStateTaskDescContext& InContext, UObject*& OutJumpTarget) const
{
	const FSceneStateBlueprintableTaskWrapper& Task = InContext.Task.Get<FSceneStateBlueprintableTaskWrapper>();
	if (UClass* TaskClass = Task.GetTaskClass())
	{
		OutJumpTarget = TaskClass;
		return true;
	}
	return false;
}
