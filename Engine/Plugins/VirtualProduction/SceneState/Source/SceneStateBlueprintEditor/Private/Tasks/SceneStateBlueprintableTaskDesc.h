// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tasks/SceneStateTaskDesc.h"
#include "SceneStateBlueprintableTaskDesc.generated.h"

/** Task Desc for FSceneStateBlueprintableTaskWrapper */
USTRUCT()
struct FSceneStateBlueprintableTaskDesc : public FSceneStateTaskDesc
{
	GENERATED_BODY()

	FSceneStateBlueprintableTaskDesc();

protected:
	//~ Begin FSceneStateTaskDesc
	virtual bool OnGetDisplayName(const FSceneStateTaskDescContext& InContext, FText& OutDisplayName) const override;
	virtual bool OnGetTooltip(const FSceneStateTaskDescContext& InContext, FText& OutDescription) const override;
	virtual bool OnGetJumpTarget(const FSceneStateTaskDescContext& InContext, UObject*& OutJumpTarget) const override;
	//~ End FSceneStateTaskDesc
};
