// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/SceneStateMachineTaskDesc.h"
#include "SceneStateBindingDelegates.h"
#include "SceneStateBlueprint.h"
#include "SceneStateMachineGraph.h"
#include "Tasks/SceneStateMachineTask.h"

FSceneStateMachineTaskDesc::FSceneStateMachineTaskDesc()
{
	SetSupportedTask<FSceneStateMachineTask>();
}

bool FSceneStateMachineTaskDesc::OnGetJumpTarget(const FSceneStateTaskDescContext& InContext, UObject*& OutJumpTarget) const
{
	USceneStateBlueprint* Blueprint = InContext.ContextObject->GetTypedOuter<USceneStateBlueprint>();
	if (!Blueprint)
	{
		return false;
	}

	const FSceneStateMachineTaskInstance& TaskInstance = InContext.TaskInstance.Get<FSceneStateMachineTaskInstance>();

	// Find state machine graph that matches the target id
	const TObjectPtr<UEdGraph>* StateMachineGraph = Blueprint->StateMachineGraphs.FindByPredicate(
		[&TaskInstance](UEdGraph* InGraph)
		{
			const USceneStateMachineGraph* const StateMachineGraph = Cast<USceneStateMachineGraph>(InGraph);
			return StateMachineGraph && StateMachineGraph->ParametersId == TaskInstance.TargetId;
		});

	if (!StateMachineGraph)
	{
		return false;
	}

	OutJumpTarget = *StateMachineGraph;
	return true;
}

void FSceneStateMachineTaskDesc::OnStructIdsChanged(const FSceneStateTaskDescMutableContext& InContext, const UE::SceneState::FStructIdChange& InChange) const
{
	FSceneStateMachineTaskInstance& TaskInstance = InContext.TaskInstance.Get<FSceneStateMachineTaskInstance>();

	if (const FGuid* NewStructId = InChange.OldToNewStructIdMap.Find(TaskInstance.TargetId))
	{
		InContext.ContextObject->Modify();
		TaskInstance.TargetId = *NewStructId;
	}
}
