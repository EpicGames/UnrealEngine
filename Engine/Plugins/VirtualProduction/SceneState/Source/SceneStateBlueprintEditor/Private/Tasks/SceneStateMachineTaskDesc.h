// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tasks/SceneStateTaskDesc.h"
#include "SceneStateMachineTaskDesc.generated.h"

/** Task Desc for FSceneStateMachineTask */
USTRUCT()
struct FSceneStateMachineTaskDesc : public FSceneStateTaskDesc
{
	GENERATED_BODY()

	FSceneStateMachineTaskDesc();

protected:
	//~ Begin FSceneStateTaskDesc
	virtual bool OnGetJumpTarget(const FSceneStateTaskDescContext& InContext, UObject*& OutJumpTarget) const override;
	virtual void OnStructIdsChanged(const FSceneStateTaskDescMutableContext& InContext, const UE::SceneState::FStructIdChange& InChange) const override;
	//~ End FSceneStateTaskDesc
};
