// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSceneStateBlueprintPaletteItem.h"
#include "Actions/SceneStateBlueprintAction_Graph.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "SceneStateBlueprintEditor.h"
#include "SceneStateBlueprintEditorLog.h"
#include "SceneStateBlueprintEditorStyle.h"
#include "ScopedTransaction.h"
#include "TutorialMetaData.h"

#define LOCTEXT_NAMESPACE "SSceneStateBlueprintPaletteItem"

namespace UE::SceneState::Editor
{

void SBlueprintPaletteItem::Construct(const FArguments& InArgs, FCreateWidgetForActionData* const InCreateData, TWeakPtr<FSceneStateBlueprintEditor> InBlueprintEditorWeak)
{
	check(InCreateData);

	TSharedPtr<FEdGraphSchemaAction> GraphAction = InCreateData->Action;
	ActionPtr = GraphAction;

	check(GraphAction.IsValid());

	auto IsReadOnly = [InBlueprintEditorWeak, GraphActionWeak = TWeakPtr<FEdGraphSchemaAction>(GraphAction), bReadOnly = InCreateData->bIsReadOnly]()
		{
			TSharedPtr<FSceneStateBlueprintEditor> BlueprintEditor = InBlueprintEditorWeak.Pin();
			TSharedPtr<FEdGraphSchemaAction> GraphAction = GraphActionWeak.Pin();
			if (GraphAction.IsValid() && BlueprintEditor.IsValid())
			{
				return bReadOnly || FBlueprintEditorUtils::IsPaletteActionReadOnly(GraphAction, BlueprintEditor);
			}
			return bReadOnly;
		};

	TSharedRef<SWidget> NameWidget = CreateTextSlotWidget(InCreateData, TAttribute<bool>::CreateLambda(IsReadOnly));

	TSharedRef<SWidget> IconWidget = CreateIconWidget(GraphAction->GetTooltipDescription()
		, GraphAction->GetPaletteIcon()
		, FSlateColor::UseForeground());

	// Setup a meta tag for this node
	FTutorialMetaData TagMeta("PaletteItem");
	{	
		TagMeta.Tag = *FString::Printf(TEXT("PaletteItem,%s,%d"), *GraphAction->GetMenuDescription().ToString(), GraphAction->GetSectionID());
		TagMeta.FriendlyName = GraphAction->GetMenuDescription().ToString();
	}

	ChildSlot
	[
		SNew(SHorizontalBox)		
		.AddMetaData<FTutorialMetaData>(TagMeta)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			IconWidget
		]
		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		.VAlign(VAlign_Center)
		.Padding(3.0f)
		[
			NameWidget
		]
	];
}

void SBlueprintPaletteItem::OnNameTextCommitted(const FText& InText, ETextCommit::Type InCommitType)
{
	TSharedPtr<FEdGraphSchemaAction> Action = ActionPtr.Pin();
	if (!Action.IsValid())
	{
		return;
	}

	if (Action->GetTypeId() != Graph::FBlueprintAction_Graph::StaticGetTypeId())
	{
		return;
	}

	TSharedPtr<Graph::FBlueprintAction_Graph> GraphAction = StaticCastSharedPtr<Graph::FBlueprintAction_Graph>(Action);
	if (!GraphAction->EdGraph || !GraphAction->EdGraph->bAllowRenaming)
	{
		return;
	}

	const FString NewName = InText.ToString();

	if (const UEdGraphSchema* Schema = GraphAction->EdGraph->GetSchema())
	{
		FGraphDisplayInfo DisplayInfo;
		Schema->GetGraphDisplayInformation(*GraphAction->EdGraph, DisplayInfo);

		// No changes required
		if (InText.EqualTo(DisplayInfo.PlainName))
		{
			return;
		}

		if (Schema->TryRenameGraph(GraphAction->EdGraph, *NewName))
		{
			return;
		}
	}

	// Make sure we aren't renaming the graph into something that already exists
	UEdGraph* ExistingGraph = FindObject<UEdGraph>(GraphAction->EdGraph->GetOuter(), *NewName);
	if (ExistingGraph && ExistingGraph != GraphAction->EdGraph)
	{
		UE_LOGF(LogSceneStateBlueprintEditor, Error, "Failed renaming graph '%ls'. Trying to rename to a graph '%ls' that already exists."
			, *GraphAction->EdGraph->GetName()
			, *NewName);
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("Rename Graph", "Rename Graph"));
	FBlueprintEditorUtils::RenameGraph(GraphAction->EdGraph, NewName);
}

} // UE::SceneState::Editor

#undef LOCTEXT_NAMESPACE
