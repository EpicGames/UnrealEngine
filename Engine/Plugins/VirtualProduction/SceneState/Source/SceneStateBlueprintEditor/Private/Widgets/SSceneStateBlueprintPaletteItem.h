// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SGraphPalette.h"

namespace UE::SceneState::Editor
{
	class FSceneStateBlueprintEditor;	
}

namespace UE::SceneState::Editor
{

/** Widget for displaying a Blueprint Item  */
class SBlueprintPaletteItem : public SGraphPaletteItem
{
public:
	SLATE_BEGIN_ARGS(SBlueprintPaletteItem) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, FCreateWidgetForActionData* const InCreateData, TWeakPtr<FSceneStateBlueprintEditor> InBlueprintEditorWeak);

private:
	//~ Begin SGraphPaletteItem
	virtual void OnNameTextCommitted(const FText& InText, ETextCommit::Type InCommitType);
	//~ End SGraphPaletteItem
};

} // UE::SceneState::Editor
