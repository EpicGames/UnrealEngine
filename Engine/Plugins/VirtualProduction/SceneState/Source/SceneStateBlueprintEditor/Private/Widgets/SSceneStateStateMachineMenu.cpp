// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSceneStateStateMachineMenu.h"
#include "Actions/SceneStateBlueprintAction_Graph.h"
#include "Actions/SceneStateMachineNodeAction.h"
#include "GraphActionNode.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Menus/SceneStateMachineAddMenu.h"
#include "Menus/SceneStateMachineContextMenu.h"
#include "Nodes/SceneStateMachineNode.h"
#include "SGraphPalette.h"
#include "SPositiveActionButton.h"
#include "SSceneStateBlueprintPaletteItem.h"
#include "SceneStateBlueprint.h"
#include "SceneStateBlueprintEditor.h"
#include "SceneStateBlueprintEditorCommands.h"
#include "SceneStateBlueprintEditorStyle.h"
#include "SceneStateMachineGraph.h"
#include "SceneStateMachineGraphSchema.h"
#include "SceneStateMachineItemDragDropOp.h"
#include "SceneStateTransitionGraphSchema.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "SSceneStateStateMachineMenu"

namespace UE::SceneState::Editor
{

namespace Private
{

bool GShowAllSubGraphs = false;
static FAutoConsoleVariableRef CVarShowAllSubGraphs(
	TEXT("SceneStateMachine.ShowAllSubGraphs"),
	GShowAllSubGraphs,
	TEXT("Shows all the sub graphs under a state machine graph / node")
);

} // Private
	
SStateMachineMenu::SStateMachineMenu()
	: CommandList(MakeShared<FUICommandList>())
{
}

void SStateMachineMenu::Construct(const FArguments& InArgs, const TSharedRef<FSceneStateBlueprintEditor>& InBlueprintEditor)
{
	CommandList->Append(InBlueprintEditor->GetToolkitCommands());

	BlueprintEditorWeak = InBlueprintEditor;

	FCoreUObjectDelegates::OnObjectPropertyChanged.AddSP(this, &SStateMachineMenu::OnObjectPropertyChanged);

	TSharedRef<SGraphActionMenu> GraphActionMenuRef = SNew(SGraphActionMenu, false)
		.OnGetFilterText(this, &SStateMachineMenu::GetSearchText)
		.OnCreateWidgetForAction(this, &SStateMachineMenu::CreateWidgetForAction)
		.OnCollectAllActions(this, &SStateMachineMenu::CollectGraphActions)
		.OnCollectStaticSections(this, &SStateMachineMenu::CollectSections)
		.OnActionSelected(this, &SStateMachineMenu::OnGraphActionSelected)
		.OnActionDragged(this, &SStateMachineMenu::OnGraphActionDragged)
		.OnActionDoubleClicked(this, &SStateMachineMenu::OnGraphActionDoubleClicked)
		.OnContextMenuOpening(this, &SStateMachineMenu::OnContextMenuOpening)
		.OnGetSectionTitle(this, &SStateMachineMenu::GetSectionTitle)
		.OnGetSectionWidget(this, &SStateMachineMenu::CreateSectionWidget)
		.OnCanRenameSelectedAction(this, &SStateMachineMenu::CanRequestRenameOnActionNode)
		.OnCategoryTextCommitted(this, &SStateMachineMenu::OnCategoryTextCommitted)
		.DefaultRowExpanderBaseIndentLevel(1)
		.AlphaSortItems(false)
		.UseSectionStyling(true);

	GraphActionMenu = GraphActionMenuRef;

	ContextMenu = MakeShared<FStateMachineContextMenu>(InBlueprintEditor, GraphActionMenuRef);
	ContextMenu->BindCommands(CommandList);

	AddMenu = MakeShared<FStateMachineAddMenu>();
	AddMenu->BindCommands(CommandList);

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.Padding(4.0f)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0, 0, 4, 0)
				[
					SNew(SPositiveActionButton)
					.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("AddNewStateMachineCombo")))
					.Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
					.Text(LOCTEXT("AddNewLabel", "Add"))
					.ToolTipText(LOCTEXT("AddNewToolTip", "Add a new State Machine"))
					.IsEnabled(InBlueprintEditor, &FSceneStateBlueprintEditor::InEditingMode)
					.OnGetMenuContent(this, &SStateMachineMenu::CreateAddNewMenuWidget)
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				[
					SAssignNew(SearchBox, SSearchBox)
					.OnTextChanged(this, &SStateMachineMenu::OnFilterTextChanged)
				]
			]
		]
		+ SVerticalBox::Slot()
		.FillHeight(1.f)
		[
			GraphActionMenuRef
		]
	];
}

SStateMachineMenu::~SStateMachineMenu()
{
	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
}

void SStateMachineMenu::RefreshMenu()
{
	bPendingRefresh = false;

	if (GraphActionMenu.IsValid())
	{
		GraphActionMenu->RefreshAllActions(/*bPreserveExpansion*/true);
	}
}

void SStateMachineMenu::ClearSelection()
{
	if (GraphActionMenu.IsValid())
	{
		GraphActionMenu->SelectItemByName(NAME_None);
	}
}

UBlueprint* SStateMachineMenu::GetBlueprint() const
{
	TSharedPtr<FSceneStateBlueprintEditor> BlueprintEditor = BlueprintEditorWeak.Pin();
	return BlueprintEditor.IsValid() ? BlueprintEditor->GetBlueprintObj() : nullptr;
}

void SStateMachineMenu::OnObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent)
{
	if (InPropertyChangedEvent.ChangeType == EPropertyChangeType::ValueSet || InPropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayClear)
	{
		return;
	}

	UBlueprint* Blueprint = GetBlueprint();
	bPendingRefresh |= InObject == Blueprint;
}

FText SStateMachineMenu::GetSearchText() const
{
	if (SearchBox.IsValid())
	{
		return SearchBox->GetText();
	}
	return FText::GetEmpty();
}

void SStateMachineMenu::OnFilterTextChanged(const FText& InFilterText)
{
	if (GraphActionMenu.IsValid())
	{
		GraphActionMenu->GenerateFilteredItems(false);
	}
}

TSharedRef<SWidget> SStateMachineMenu::CreateAddNewMenuWidget()
{
	check(AddMenu.IsValid());
	return AddMenu->GenerateWidget();
}

TSharedPtr<SWidget> SStateMachineMenu::OnContextMenuOpening()
{
	TArray<TSharedPtr<FEdGraphSchemaAction>> SelectedActions;
	GraphActionMenu->GetSelectedActions(SelectedActions);

	// If no Selected Actions, default to the Add New Menu
	if (SelectedActions.IsEmpty())
	{
		return CreateAddNewMenuWidget();
	}

	check(ContextMenu.IsValid());
	return ContextMenu->GenerateWidget();
}

TSharedRef<SWidget> SStateMachineMenu::CreateWidgetForAction(FCreateWidgetForActionData* InCreateData)
{
	return SNew(SBlueprintPaletteItem, InCreateData, BlueprintEditorWeak);
}

void SStateMachineMenu::GetGraphActionDetails(const TSharedPtr<FEdGraphSchemaAction>& InAction, UObject*& OutDetailsObject, FText& OutDetailsText) const
{
	if (!InAction.IsValid())
	{
		return;
	}

	const FName GraphActionType = InAction->GetTypeId();

	if (GraphActionType == Graph::FBlueprintAction_Graph::StaticGetTypeId())
	{
		TSharedPtr<Graph::FBlueprintAction_Graph> GraphAction = StaticCastSharedPtr<Graph::FBlueprintAction_Graph>(InAction);
		if (GraphAction->EdGraph)
		{
			const UEdGraphSchema* Schema = GraphAction->EdGraph->GetSchema();
			check(Schema);

			FGraphDisplayInfo DisplayInfo;
			Schema->GetGraphDisplayInformation(*GraphAction->EdGraph, DisplayInfo);

			OutDetailsObject = GraphAction->EdGraph;
			OutDetailsText = DisplayInfo.PlainName;
		}
	}
	else if (GraphActionType == Graph::FStateMachineAction_Node::StaticGetTypeId())
	{
		TSharedPtr<Graph::FStateMachineAction_Node> NodeAction = StaticCastSharedPtr<Graph::FStateMachineAction_Node>(InAction);
		if (USceneStateMachineNode* Node = NodeAction->GetNode())
		{
			OutDetailsObject = Node;
			OutDetailsText = FText::FromName(Node->GetNodeName());
		}
	}
}

void SStateMachineMenu::OnGraphActionSelected(const TArray<TSharedPtr<FEdGraphSchemaAction>>& InActions, ESelectInfo::Type InSelectionType)
{
	TSharedPtr<FSceneStateBlueprintEditor> BlueprintEditor = BlueprintEditorWeak.Pin();
	if (!BlueprintEditor.IsValid())
	{
		return;
	}

	TSharedPtr<FEdGraphSchemaAction> Action;
	if (!InActions.IsEmpty())
	{
		Action = InActions[0];
	}

	BlueprintEditor->SetUISelectionState(FSceneStateBlueprintEditor::SelectionState_StateMachine);

	UObject* DetailsObject = nullptr;
	FText DetailsText;
	GetGraphActionDetails(Action, DetailsObject, DetailsText);

	BlueprintEditor->GetInspector()->ShowDetailsForSingleObject(DetailsObject, SKismetInspector::FShowDetailsOptions(DetailsText));
}

FReply SStateMachineMenu::OnGraphActionDragged(const TArray<TSharedPtr<FEdGraphSchemaAction>>& InActions, const FPointerEvent& InMouseEvent)
{
	if (InActions.IsEmpty() || !InActions[0].IsValid())
	{
		return FReply::Unhandled();
	}

	const TSharedPtr<FEdGraphSchemaAction>& Action = InActions[0];

	if (Action->GetTypeId() == Graph::FBlueprintAction_Graph::StaticGetTypeId())
	{
		return FReply::Handled().BeginDragDrop(FStateMachineItemDragDropOp::New(Action));
	}

	return FReply::Unhandled();
}

bool SStateMachineMenu::ShouldProcessNode(TNotNull<const USceneStateMachineNode*> InGraphNode) const
{
	return InGraphNode->GetBoundGraphs().ContainsByPredicate(
		[This=this](UEdGraph* InBoundGraph)
		{
			return This->ShouldProcessGraph(InBoundGraph);
		});
}

void SStateMachineMenu::CollectNodeActionsRecursive(TNotNull<USceneStateMachineNode*> InNode, FText InCategory, FGraphActionListBuilderBase& OutActions)
{
	if (!ShouldProcessNode(InNode))
	{
		return;
	}

	const Graph::FStateMachineAction_Node::FArguments Arguments
		{
			.Node = InNode,
			.Category = InCategory,
			.SectionID = NodeSectionID::GRAPH,
		};

	TSharedRef<Graph::FStateMachineAction_Node> NodeAction = MakeShared<Graph::FStateMachineAction_Node>(Arguments);
	OutActions.AddAction(NodeAction);

	InCategory = FText::Format(INVTEXT("{0}|{1}"), InCategory, NodeAction->GetMenuDescription());

	for (UEdGraph* BoundGraph : InNode->GetBoundGraphs())
	{
		CollectGraphActionsRecursive(BoundGraph, InCategory, EEdGraphSchemaAction_K2Graph::Type::Subgraph, OutActions);
	}
}

bool SStateMachineMenu::ShouldProcessGraph(UEdGraph* InGraph) const
{
	if (!InGraph)
	{
		return false;
	}

	if (Private::GShowAllSubGraphs)
	{
		return true;	
	}

	const UEdGraphSchema* Schema = InGraph->GetSchema();
	if (!Schema)
	{
		return false;
	}

	// Don't show graphs that aren't State Machines or K2 Graphs
	if (!Schema->IsA<USceneStateMachineGraphSchema>() && !Schema->IsA<UEdGraphSchema_K2>())
	{
		return false;
	}

	// Prevent Transition Graphs from showing up (as they cause a lot of noise)
	if (Schema->IsA<USceneStateTransitionGraphSchema>())
	{
		return false;
	}

	// Always show all top level state machines
	if (Cast<UBlueprint>(InGraph->GetOuter()))
	{
		return true;
	}

	// Check that there's more than 1 node in this graph.
	// There could be further checks here to verify if the state machine is 'meaningful', but keeping simple here for now
	return InGraph->Nodes.Num() > 1;
}

void SStateMachineMenu::CollectGraphActionsRecursive(UEdGraph* InGraph, FText InCategory, int32 InGraphType, FGraphActionListBuilderBase& OutActions)
{
	if (!ShouldProcessGraph(InGraph))
	{
		return;
	}

	const UEdGraphSchema* Schema = InGraph->GetSchema();
	if (!Schema)
	{
		return;
	}

	const Graph::FBlueprintAction_Graph::FArguments Arguments {
			.Graph = InGraph,
			.GraphType = static_cast<EEdGraphSchemaAction_K2Graph::Type>(InGraphType),
			.Category = InCategory,
			.Icon = FBlueprintEditorStyle::Get().GetGraphSchemaIcon(Schema->GetClass()),
			.Grouping = 1,
			.SectionID = NodeSectionID::GRAPH,
		};

	TSharedRef<Graph::FBlueprintAction_Graph> GraphAction = MakeShared<Graph::FBlueprintAction_Graph>(Arguments);
	OutActions.AddAction(GraphAction);

	InCategory = FText::Format(INVTEXT("{0}|{1}"), InCategory, GraphAction->GetMenuDescription());

	for (UEdGraphNode* GraphNode : InGraph->Nodes)
	{
		if (USceneStateMachineNode* Node = Cast<USceneStateMachineNode>(GraphNode))
		{
			CollectNodeActionsRecursive(Node, InCategory, OutActions);
		}
	}
}

void SStateMachineMenu::CollectGraphActions(FGraphActionListBuilderBase& OutActions)
{
	if (USceneStateBlueprint* Blueprint = Cast<USceneStateBlueprint>(GetBlueprint()))
	{
		for (UEdGraph* Graph : Blueprint->StateMachineGraphs)
		{
			if (USceneStateMachineGraph* StateMachineGraph = Cast<USceneStateMachineGraph>(Graph))
			{
				CollectGraphActionsRecursive(StateMachineGraph, StateMachineGraph->Category, EEdGraphSchemaAction_K2Graph::Type::Graph, OutActions);
			}
		}
	}
}

void SStateMachineMenu::CollectSections(TArray<int32>& OutSectionIds)
{
	OutSectionIds.Add(NodeSectionID::GRAPH);
}

FText SStateMachineMenu::GetSectionTitle(int32 InSectionId) const
{
	check(InSectionId == NodeSectionID::GRAPH);
	return LOCTEXT("StateMachineGraphs", "State Machines");
}

TSharedRef<SWidget> SStateMachineMenu::CreateSectionWidget(TSharedRef<SWidget> InRowWidget, int32 InSectionId)
{
	return SNew(SButton)
		.ButtonStyle(FAppStyle::Get(), "SimpleButton")
		.OnClicked(this, &SStateMachineMenu::OnSectionAddButtonClicked, InSectionId)
		.ContentPadding(FMargin(1, 0))
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("AddNewStateMachine")))
		.ToolTipText(LOCTEXT("AddNewStateMachineGraph", "New State Machine"))
		[
			SNew(SImage)
			.Image(FAppStyle::Get().GetBrush("Icons.PlusCircle"))
			.ColorAndOpacity(FSlateColor::UseForeground())
		];
}

FReply SStateMachineMenu::OnSectionAddButtonClicked(int32 InSectionId)
{
	USceneStateBlueprint* Blueprint = Cast<USceneStateBlueprint>(GetBlueprint());
	if (!Blueprint)
	{
		return FReply::Unhandled();
	}

	check(InSectionId == NodeSectionID::GRAPH);
	CommandList->ExecuteAction(FBlueprintEditorCommands::Get().AddStateMachine.ToSharedRef());
	return FReply::Handled();
}

void SStateMachineMenu::OnGraphActionDoubleClicked(const TArray<TSharedPtr<FEdGraphSchemaAction>>& InActions)
{
	if (!InActions.IsEmpty())
	{
		ExecuteGraphAction(InActions[0]);
	}
}

void SStateMachineMenu::ExecuteGraphAction(const TSharedPtr<FEdGraphSchemaAction>& InAction)
{
	TSharedPtr<FSceneStateBlueprintEditor> BlueprintEditor = BlueprintEditorWeak.Pin();
	if (!InAction.IsValid() || !BlueprintEditor.IsValid())
	{
		return;
	}

	const FName ActionTypeId = InAction->GetTypeId();
	if (ActionTypeId == Graph::FBlueprintAction_Graph::StaticGetTypeId())
	{
		TSharedPtr<Graph::FBlueprintAction_Graph> GraphAction = StaticCastSharedPtr<Graph::FBlueprintAction_Graph>(InAction);
		if (GraphAction->EdGraph)
		{
			BlueprintEditor->JumpToHyperlink(GraphAction->EdGraph, /*bRequestRename*/false);
		}
	}
	else if (ActionTypeId == Graph::FStateMachineAction_Node::StaticGetTypeId())
	{
		TSharedPtr<Graph::FStateMachineAction_Node> NodeAction = StaticCastSharedPtr<Graph::FStateMachineAction_Node>(InAction);
		if (USceneStateMachineNode* Node = NodeAction->GetNode())
		{
			BlueprintEditor->JumpToHyperlink(Node, /*bRequestRename*/false);
		}
	}
}

bool SStateMachineMenu::CanRequestRenameOnActionNode(TWeakPtr<FGraphActionNode> InSelectedNodeWeak) const
{
	TSharedPtr<FGraphActionNode> GraphActionNode = InSelectedNodeWeak.Pin();
	if (!GraphActionNode.IsValid())
	{
		return false;
	}

	TSharedPtr<FSceneStateBlueprintEditor> BlueprintEditor = BlueprintEditorWeak.Pin();
	if (!BlueprintEditor.IsValid() || !BlueprintEditor->InEditingMode())
	{
		return false;
	}

	if (GraphActionNode->IsCategoryNode())
	{
		TArray<TSharedPtr<FEdGraphSchemaAction>> Actions;
		GraphActionMenu->GetCategorySubActions(InSelectedNodeWeak, Actions);

		return Actions.ContainsByPredicate(
			[](const TSharedPtr<FEdGraphSchemaAction>& InAction)
			{
				return InAction.IsValid() && InAction->GetPersistentItemDefiningObject().IsPotentiallyEditable();
			});
	}

	if (GraphActionNode->IsActionNode())
	{
		if (!FBlueprintEditorUtils::IsPaletteActionReadOnly(GraphActionNode->Action, BlueprintEditor))
		{
			return GraphActionNode->Action->CanBeRenamed();
		}
	}

	return false;
}

void SStateMachineMenu::OnCategoryTextCommitted(const FText& InText, ETextCommit::Type InCommitType, TWeakPtr<FGraphActionNode> InTargetAction)
{
	const TSharedPtr<FGraphActionNode> TargetAction = InTargetAction.Pin();
	if (!TargetAction.IsValid())
	{
		return;
	}

	UBlueprint* const Blueprint = GetBlueprint();
	if (!Blueprint)
	{
		return;
	}

	// Remove excess whitespace and prevent categories with just spaces
	const FText CategoryName = FText::TrimPrecedingAndTrailing(InText);

	TArray<TSharedPtr<FEdGraphSchemaAction>> Actions;
	GraphActionMenu->GetCategorySubActions(InTargetAction, Actions);

	if (Actions.IsEmpty())
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("RenameCategory", "Rename Category"));
	Blueprint->Modify();

	for (const TSharedPtr<FEdGraphSchemaAction>& Action : Actions)
	{
		if (Action)
		{
			Action->MovePersistentItemToCategory(CategoryName);
		}
	}

	RefreshMenu();
	FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
	GraphActionMenu->SelectItemByName(*CategoryName.ToString(), ESelectInfo::OnMouseClick, TargetAction->SectionID, /*bIsCategory*/true);
}

void SStateMachineMenu::Tick(const FGeometry& InGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (bPendingRefresh)
	{
		RefreshMenu();
	}
}

FReply SStateMachineMenu::OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent)
{
	if (CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

} // UE::SceneState::Editor

#undef LOCTEXT_NAMESPACE
