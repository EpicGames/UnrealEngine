// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Widgets/SCompoundWidget.h"

class FUICommandList;
class SGraphActionMenu;
class SSearchBox;
class UBlueprint;
class UEdGraph;
class UObject;
class USceneStateMachineNode;
struct FCreateWidgetForActionData;
struct FEdGraphSchemaAction;
struct FGraphActionListBuilderBase;
struct FGraphActionNode;
struct FPropertyChangedEvent;

namespace UE::SceneState::Editor
{
	class FSceneStateBlueprintEditor;
	class FStateMachineAddMenu;
	class FStateMachineContextMenu;
}

namespace UE::SceneState::Editor
{

class SStateMachineMenu : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SStateMachineMenu) {}
	SLATE_END_ARGS()

	SStateMachineMenu();

	void Construct(const FArguments& InArgs, const TSharedRef<FSceneStateBlueprintEditor>& InBlueprintEditor);

	virtual ~SStateMachineMenu() override;

	void RefreshMenu();

	void ClearSelection();

private:
	UBlueprint* GetBlueprint() const;

	void OnObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent);

	FText GetSearchText() const;

	void OnFilterTextChanged(const FText& InFilterText);

	TSharedRef<SWidget> CreateAddNewMenuWidget();

	TSharedPtr<SWidget> OnContextMenuOpening();

	TSharedRef<SWidget> CreateWidgetForAction(FCreateWidgetForActionData* InCreateData);

	void GetGraphActionDetails(const TSharedPtr<FEdGraphSchemaAction>& InAction, UObject*& OutDetailsObject, FText& OutDetailsText) const;

	void OnGraphActionSelected(const TArray<TSharedPtr<FEdGraphSchemaAction>>& InActions, ESelectInfo::Type InSelectionType);

	/** Called when dragging the graph action entry */
	FReply OnGraphActionDragged(const TArray<TSharedPtr<FEdGraphSchemaAction>>& InActions, const FPointerEvent& InMouseEvent);

	bool ShouldProcessNode(TNotNull<const USceneStateMachineNode*> InGraphNode) const;
	void CollectNodeActionsRecursive(TNotNull<USceneStateMachineNode*> InNode, FText InCategory, FGraphActionListBuilderBase& OutActions);

	bool ShouldProcessGraph(UEdGraph* InGraph) const;
	void CollectGraphActionsRecursive(UEdGraph* InGraph, FText InCategory, int32 InGraphType, FGraphActionListBuilderBase& OutActions);

	void CollectGraphActions(FGraphActionListBuilderBase& OutActions);

	void CollectSections(TArray<int32>& OutSectionIds);

	FText GetSectionTitle(int32 InSectionId) const;

	TSharedRef<SWidget> CreateSectionWidget(TSharedRef<SWidget> InRowWidget, int32 InSectionId);

	FReply OnSectionAddButtonClicked(int32 InSectionId);

	void OnGraphActionDoubleClicked(const TArray<TSharedPtr<FEdGraphSchemaAction>>& InActions);

	void ExecuteGraphAction(const TSharedPtr<FEdGraphSchemaAction>& InAction);

	bool CanRequestRenameOnActionNode(TWeakPtr<FGraphActionNode> InSelectedNodeWeak) const;

	/** Called when a category target item has been renamed */
	void OnCategoryTextCommitted(const FText& InText, ETextCommit::Type InCommitType, TWeakPtr<FGraphActionNode> InTargetAction);

	//~ Begin SWidget
	virtual void Tick(const FGeometry& InGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual FReply OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) override;
	//~ End SWidget

	TWeakPtr<FSceneStateBlueprintEditor> BlueprintEditorWeak;

	TSharedRef<FUICommandList> CommandList;

	TSharedPtr<SGraphActionMenu> GraphActionMenu;

	TSharedPtr<SSearchBox> SearchBox;

	TSharedPtr<FStateMachineAddMenu> AddMenu;

	TSharedPtr<FStateMachineContextMenu> ContextMenu;

	bool bPendingRefresh = false;
};

} // UE::SceneState::Editor
