// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateMachineItemDragDropOp.h"
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "SceneStateMachineItemDragDropOp"

namespace UE::SceneState::Editor
{

TSharedRef<FGraphSchemaActionDragDropAction> FStateMachineItemDragDropOp::New(const TSharedPtr<FEdGraphSchemaAction>& InActionNode)
{
	TSharedRef<FStateMachineItemDragDropOp> DragDropOp = MakeShared<FStateMachineItemDragDropOp>();
	DragDropOp->SourceAction = InActionNode;
	DragDropOp->Construct();
	return DragDropOp;
}

void FStateMachineItemDragDropOp::HoverTargetChanged()
{
	if (!SourceAction.IsValid())
	{
		return;
	}

	if (!HoveredCategoryName.IsEmpty())
	{
		OnHoverCategory(HoveredCategoryName);
	}
	else if (const TSharedPtr<FEdGraphSchemaAction> TargetAction = HoveredAction.Pin())
	{
		OnHoverTargetAction(TargetAction.ToSharedRef());
	}
}

FReply FStateMachineItemDragDropOp::DroppedOnCategory(FText InCategory)
{
	if (!SourceAction.IsValid())
	{
		return FReply::Unhandled();
	}

	SourceAction->MovePersistentItemToCategory(InCategory);
	return FReply::Handled();
}

FReply FStateMachineItemDragDropOp::DroppedOnAction(TSharedRef<FEdGraphSchemaAction> InAction)
{
	if (!SourceAction.IsValid()
		|| SourceAction->GetTypeId() != InAction->GetTypeId()
		|| SourceAction->GetPersistentItemDefiningObject() != InAction->GetPersistentItemDefiningObject())
	{
		return FReply::Unhandled();
	}

	FScopedTransaction Transaction(LOCTEXT("ReorderStateMachineGraph", "Reorder State Machine Graph"));
	if (!SourceAction->ReorderToBeforeAction(InAction))
	{
		Transaction.Cancel();
	}

	return FReply::Handled();
}

void FStateMachineItemDragDropOp::OnHoverCategory(const FText& InCategory)
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("SourceDisplayName"), SourceAction->GetMenuDescription());
	Args.Add(TEXT("TargetCategoryName"), HoveredCategoryName);

	if (HoveredCategoryName.EqualTo(SourceAction->GetCategory()))
	{
		SetFeedbackMessageError(FText::Format(LOCTEXT("SameCategoryError", "'{SourceDisplayName}' is already in category '{TargetCategoryName}'"), Args));
	}
	else
	{
		SetFeedbackMessageOk(FText::Format(LOCTEXT("SetCategoryOk", "Move '{SourceDisplayName}' to category '{TargetCategoryName}'"), Args));
	}
}

void FStateMachineItemDragDropOp::OnHoverTargetAction(const TSharedRef<FEdGraphSchemaAction>& InTargetAction)
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("SourceDisplayName"), SourceAction->GetMenuDescription());
	Args.Add(TEXT("TargetDisplayName"), InTargetAction->GetMenuDescription());

	if (InTargetAction->GetTypeId() != SourceAction->GetTypeId())
	{
		SetFeedbackMessageError(FText::Format(LOCTEXT("ReorderActionDifferentAction", "Cannot reorder '{SourceDisplayName}' into a different section."), Args));
		return;
	}

	if (SourceAction->GetPersistentItemDefiningObject() != InTargetAction->GetPersistentItemDefiningObject())
	{
		SetFeedbackMessageError(FText::Format(LOCTEXT("ReorderActionDifferentScope", "Cannot reorder '{SourceDisplayName}' into a different scope."), Args));
		return;
	}

	const int32 SourceIndex = SourceAction->GetReorderIndexInContainer();
	const int32 TargetIndex = InTargetAction->GetReorderIndexInContainer();

	if (SourceIndex == INDEX_NONE)
	{
		SetFeedbackMessageError(FText::Format(LOCTEXT("ReorderNonOrderedItem", "Cannot reorder '{SourceDisplayName}'."), Args));
		return;
	}

	if (TargetIndex == INDEX_NONE)
	{
		SetFeedbackMessageError(FText::Format(LOCTEXT("ReorderOntoNonOrderedItem", "Cannot reorder '{SourceDisplayName}' before '{TargetDisplayName}'."), Args));
		return;
	}

	if (InTargetAction == SourceAction)
	{
		SetFeedbackMessageError(FText::Format(LOCTEXT("ReorderOntoSameItem", "Cannot reorder '{SourceDisplayName}' before itself."), Args));
		return;
	}

	SetFeedbackMessageOk(FText::Format(LOCTEXT("ReorderActionOk", "Reorder '{SourceDisplayName}' before '{TargetDisplayName}'"), Args));
}

void FStateMachineItemDragDropOp::SetFeedbackMessageOk(const FText& InMessage)
{
	SetSimpleFeedbackMessage(FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK")), FLinearColor::White, InMessage);
}

void FStateMachineItemDragDropOp::SetFeedbackMessageError(const FText& InMessage)
{
	SetSimpleFeedbackMessage(FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error")), FLinearColor::White, InMessage);
}

} // UE::SceneState::Editor

#undef LOCTEXT_NAMESPACE
