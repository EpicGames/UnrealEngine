// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GraphEditorDragDropAction.h"

namespace UE::SceneState::Graph
{
	struct FBlueprintAction_Graph;
}

namespace UE::SceneState::Editor
{

/** Drag Drop Op for State Machine Menu Entries */
class FStateMachineItemDragDropOp : public FGraphSchemaActionDragDropAction
{
public:
	using Super = FGraphSchemaActionDragDropAction;

	DRAG_DROP_OPERATOR_TYPE(FStateMachineItemDragDropOp, FGraphSchemaActionDragDropAction)

	static TSharedRef<FGraphSchemaActionDragDropAction> New(const TSharedPtr<FEdGraphSchemaAction>& InActionNode);

	//~ Begin FGraphEditorDragDropAction
	virtual void HoverTargetChanged() override;
	virtual FReply DroppedOnCategory(FText InCategory) override;
	virtual FReply DroppedOnAction(TSharedRef<FEdGraphSchemaAction> InAction) override;
	//~ End FGraphEditorDragDropAction

private:
	/** Called when hovering over a category */
	void OnHoverCategory(const FText& InCategory);

	/** Called when hovering over a target action */
	void OnHoverTargetAction(const TSharedRef<FEdGraphSchemaAction>& InTargetAction);

	void SetFeedbackMessageOk(const FText& InMessage);
	void SetFeedbackMessageError(const FText& InMessage);
};

} // UE::SceneState::Editor
