// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class USceneStateBlueprint;
template <typename T> class TSubclassOf;

namespace UE::SceneState::Editor
{
	class IContextEditor;
}

namespace UE::SceneState::Editor
{

class IBlueprintEditorModule : public IModuleInterface
{
	static constexpr const TCHAR* ModuleName = TEXT("SceneStateBlueprintEditor");

public:
	static bool IsLoaded()
	{
		return FModuleManager::Get().IsModuleLoaded(ModuleName);
	}

	static IBlueprintEditorModule& Get()
	{
		return FModuleManager::Get().LoadModuleChecked<IBlueprintEditorModule>(ModuleName);
	}

	static IBlueprintEditorModule* GetPtr()
	{
		return FModuleManager::Get().GetModulePtr<IBlueprintEditorModule>(ModuleName);
	}

	virtual void RegisterCompiler(TSubclassOf<USceneStateBlueprint> InBlueprintClass) = 0;

	/** Registers the context editor that defines editor-only logic for a given context class/object */
	virtual void RegisterContextEditor(const TSharedPtr<IContextEditor>& InContextEditor) = 0;

	/** Unregisters the context editor from the current list */
	virtual void UnregisterContextEditor(const TSharedPtr<IContextEditor>& InContextEditor) = 0;
};

} // UE::SceneState::Editor
