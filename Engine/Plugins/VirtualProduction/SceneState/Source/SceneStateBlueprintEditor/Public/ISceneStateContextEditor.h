// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Templates/SharedPointerFwd.h"
#include "UObject/ObjectPtr.h"

class SWidget;
class UObject;
template<typename T> class TSubclassOf;

namespace UE::SceneState::Editor
{

/** Defines editor-only logic for a given context class/object */
class IContextEditor
{
public:
	struct FContextParams
	{
		TObjectPtr<UObject> ContextObject;
	};

	/** Gets the context object classes to support */
	virtual void GetContextClasses(TArray<TSubclassOf<UObject>>& OutContextClasses) const = 0;

	/** Retrieves the widget to use for the debug viewer */
	virtual TSharedPtr<SWidget> CreateViewWidget(const FContextParams& InContextParams) const = 0;
};

} // UE::SceneState::Editor
