// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyEditorModule.h"
#include "Templates/SharedPointerFwd.h"

#define UE_API SCENESTATEBLUEPRINTEDITOR_API

class IDetailChildrenBuilder;
class IPropertyHandle;
class UClass;
struct FGuid;
struct FInstancedPropertyBag;

namespace UE::SceneState::Editor
{
	/** Finds the Task Id for a given Property Handle by first finding the outer Task Node and returning its Task id. */
	UE_API FGuid FindTaskId(const TSharedRef<IPropertyHandle>& InPropertyHandle);

	/**
	 * Returns whether a given property is an object property of a given class
	 * @param InProperty the property to check
	 * @param InClass the class to check
	 * @return true if the property is an object property of the given class
	 */
	UE_API bool IsObjectPropertyOfClass(const FProperty* InProperty, const UClass* InClass);

	/**
	 * Returns whether a given property is of a given struct type
	 * @param InProperty the property to check
	 * @param InStruct the type to check
	 * @return true if the property is a struct property of the given struct type  
	 */
	UE_API bool IsStruct(const FProperty* InProperty, const UScriptStruct* InStruct);

	/**
	 * Gets the Guid value from the Property Handle.
	 * @param InGuidPropertyHandle the guid property handle to get the guid value from
	 * @param OutGuid the returned guid if the operation was successful
	 * @return whether the guid was obtained successfully, or whether it failed or had multiple values
	 */
	UE_API FPropertyAccess::Result GetGuid(const TSharedRef<IPropertyHandle>& InGuidPropertyHandle, FGuid& OutGuid);

	/**
	 * Tries to create a structure data provider for the given instanced struct property handle
	 * @param InStructHandle the instanced struct property handle to create a struct data provider for
	 * @return the created struct data provider
	 */
	UE_API TSharedRef<IStructureDataProvider> CreateInstancedStructDataProvider(const TSharedRef<IPropertyHandle>& InStructHandle);

	/**
	 * Compares the structure layout of two property bags
	 * @param InParametersA the first property bag comparand
	 * @param InParametersB the second property bag comparand
	 * @return true if both have the same number of properties with same names and compatible types.
	 */
	UE_API bool CompareParametersLayout(const FInstancedPropertyBag& InParametersA, const FInstancedPropertyBag& InParametersB);

	/** Sets the Instance Meta-data of a given property handle to a Task Id guid */
	UE_API void AssignBindingId(const TSharedRef<IPropertyHandle>& InPropertyHandle, const FGuid& InTaskId);

	/** Finds the Common Class for all the Objects within the Object Property Handle */
	UE_API UClass* FindCommonBase(const TSharedRef<IPropertyHandle>& InPropertyHandle);

	/**
	 * Helper function to add the Common properties of the object property handle to the Child Builder
	 * @param InPropertyHandle the object property handle to get the common properties from
	 * @param InChildBuilder the builder to add the properties to
	 * @param InDisallowedFlags filter flags. Any property with any of these flags will be skipped 
	 */
	UE_API bool AddObjectProperties(const TSharedRef<IPropertyHandle>& InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, uint64 InDisallowedFlags = CPF_DisableEditOnInstance);

} // UE::SceneState::Editor

#undef UE_API
