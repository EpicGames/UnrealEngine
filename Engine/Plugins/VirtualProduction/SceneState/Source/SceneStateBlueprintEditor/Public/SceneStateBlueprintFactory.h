// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "Templates/SubclassOf.h"
#include "SceneStateBlueprintFactory.generated.h"

class UEdGraph;
class USceneStateBlueprint;
class USceneStateObject;
class USceneStateSchema;

UCLASS(MinimalAPI)
class USceneStateBlueprintFactory : public UFactory
{
	GENERATED_BODY()

public:
	USceneStateBlueprintFactory();

	static UEdGraph* AddStateMachine(USceneStateBlueprint* InBlueprint);

	//~ Begin UFactory
	virtual FText GetDisplayName() const override;
	virtual FString GetDefaultNewAssetName() const override;
	virtual uint32 GetMenuCategories() const override;
	virtual bool ConfigureProperties() override;
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, FFeedbackContext* InWarn) override;
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, FFeedbackContext* InWarn, FName InCallingContext) override;
	//~ End UFactory

	/** Scene state schema to use*/
	UPROPERTY()
	TSubclassOf<USceneStateSchema> SchemaClass;

	UPROPERTY()
	TSubclassOf<USceneStateObject> ParentClass;
};
