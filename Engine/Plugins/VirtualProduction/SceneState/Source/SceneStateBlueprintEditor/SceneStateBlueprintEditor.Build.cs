// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SceneStateBlueprintEditor : ModuleRules
{
    public SceneStateBlueprintEditor(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "Core",
                "CoreUObject",
                "SceneStateEditor",
                "UnrealEd",
            }
        );

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "AssetDefinition",
                "BlueprintGraph",
                "DetailCustomizations",
                "Engine",
                "GraphEditor",
                "InputCore",
                "Kismet",
                "KismetCompiler",
                "KismetWidgets",
                "PropertyBindingUtils",
                "PropertyBindingUtilsEditor",
                "PropertyEditor",
                "SceneState",
                "SceneStateBinding",
                "SceneStateBlueprint",
                "SceneStateEvent",
                "SceneStateMachineEditor",
                "SceneStateMachineGraph",
                "SceneStateTransitionGraph",
                "Slate",
                "SlateCore",
                "StructUtilsEditor",
                "ToolMenus",
                "ToolWidgets",
            }
        );
    }
}
