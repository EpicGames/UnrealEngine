// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateEditorModule.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"
#include "SceneStateEditorLog.h"
#include "Tasks/SceneStateTaskDescRegistry.h"

DEFINE_LOG_CATEGORY(LogSceneStateEditor);

IMPLEMENT_MODULE(FSceneStateEditorModule, SceneStateEditor)

void FSceneStateEditorModule::StartupModule()
{
	OnPostEngineInitHandle = FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FSceneStateEditorModule::OnPostEngineInit);
}

void FSceneStateEditorModule::ShutdownModule()
{
	FCoreDelegates::GetOnPostEngineInit().Remove(OnPostEngineInitHandle);
	OnPostEngineInitHandle.Reset();
}

void FSceneStateEditorModule::OnPostEngineInit()
{
	FSceneStateTaskDescRegistry::GlobalRegistry.CacheTaskDescs();
}
