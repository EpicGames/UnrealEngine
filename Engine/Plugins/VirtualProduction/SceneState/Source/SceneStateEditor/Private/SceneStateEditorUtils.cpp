// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateEditorUtils.h"
#include "UObject/Class.h"

namespace UE::SceneState::Editor
{

FText GetStructTooltip(const UStruct& InStruct)
{
	return InStruct.GetMetaDataText(TEXT("Tooltip"), TEXT("UObjectToolTips"), InStruct.GetFullGroupName(/*bStartWithOuter*/false));
}

} // UE::SceneState::Editor
