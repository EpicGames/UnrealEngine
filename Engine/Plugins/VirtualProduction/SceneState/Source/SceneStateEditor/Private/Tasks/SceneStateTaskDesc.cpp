// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/SceneStateTaskDesc.h"
#include "SceneStateEditorUtils.h"
#include "Tasks/SceneStateTask.h"

FSceneStateTaskDesc::FSceneStateTaskDesc()
{
	SetSupportedTask<FSceneStateTask>();
}

FText FSceneStateTaskDesc::GetDisplayName(const FSceneStateTaskDescContext& InContext) const
{
	if (!IsValidContext(InContext))
	{
		return FText::GetEmpty();
	}

	FText DisplayName;
	if (OnGetDisplayName(InContext, DisplayName))
	{
		return DisplayName;
	}

	if (const UScriptStruct* TaskStruct = InContext.Task.GetScriptStruct())
	{
		return TaskStruct->GetDisplayNameText();
	}
	return FText::GetEmpty();
}

FText FSceneStateTaskDesc::GetTooltip(const FSceneStateTaskDescContext& InContext) const
{
	if (!IsValidContext(InContext))
	{
		return FText::GetEmpty();
	}

	FText Tooltip;
	if (OnGetTooltip(InContext, Tooltip))
	{
		return Tooltip;
	}

	if (const UScriptStruct* TaskStruct = InContext.Task.GetScriptStruct())
	{
		return UE::SceneState::Editor::GetStructTooltip(*TaskStruct);
	}
	return FText::GetEmpty();
}

UObject* FSceneStateTaskDesc::GetJumpTarget(const FSceneStateTaskDescContext& InContext) const
{
	if (!IsValidContext(InContext))
	{
		return nullptr;
	}

	UObject* JumpTarget;
	if (OnGetJumpTarget(InContext, JumpTarget))
	{
		return JumpTarget;
	}
	return nullptr;
}

void FSceneStateTaskDesc::NotifyStructIdsChanged(const FSceneStateTaskDescMutableContext& InContext, const UE::SceneState::FStructIdChange& InChange) const
{
	if (IsValidContext(InContext))
	{
		OnStructIdsChanged(InContext, InChange);
	}
}

bool FSceneStateTaskDesc::IsValidContext(const FSceneStateTaskDescContext& InContext) const
{
	if (!InContext.ContextObject || !InContext.Task.IsValid() || !InContext.TaskInstance.IsValid())
	{
		return false;
	}

	const UScriptStruct* TaskStruct = InContext.Task.GetScriptStruct();
	check(TaskStruct);
	return TaskStruct->IsChildOf(SupportedTask);
}

bool FSceneStateTaskDesc::IsValidContext(const FSceneStateTaskDescMutableContext& InContext) const
{
	if (!InContext.ContextObject || !InContext.Task.IsValid() || !InContext.TaskInstance.IsValid())
	{
		return false;
	}

	const UScriptStruct* TaskStruct = InContext.Task.GetScriptStruct();
	check(TaskStruct);
	return TaskStruct->IsChildOf(SupportedTask);
}

void FSceneStateTaskDesc::SetSupportedTask(UScriptStruct* InSupportedTask)
{
	checkf(InSupportedTask && InSupportedTask->IsChildOf<FSceneStateTask>(), TEXT("Task struct must derived from FSceneStateTask!"));
	SupportedTask = InSupportedTask;
}
