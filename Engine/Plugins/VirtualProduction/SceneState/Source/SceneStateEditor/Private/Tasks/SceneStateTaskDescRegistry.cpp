// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/SceneStateTaskDescRegistry.h"
#include "SceneStateEditorLog.h"
#include "SceneStateEditorModule.h"
#include "Tasks/SceneStateTask.h"
#include "Tasks/SceneStateTaskDesc.h"
#include "UObject/UObjectIterator.h"

FSceneStateTaskDescRegistry FSceneStateTaskDescRegistry::GlobalRegistry;

const FSceneStateTaskDescRegistry& FSceneStateTaskDescRegistry::Get()
{
	return GlobalRegistry;
}

const FSceneStateTaskDesc& FSceneStateTaskDescRegistry::GetTaskDesc(const UScriptStruct* InTaskStruct) const
{
	const TInstancedStruct<FSceneStateTaskDesc>* TaskDesc = nullptr;

	for (; InTaskStruct; InTaskStruct = Cast<UScriptStruct>(InTaskStruct->GetSuperStruct()))
	{
		TaskDesc = TaskDescs.Find(InTaskStruct);
		if (TaskDesc)
		{
			break;
		}
	}

	if (TaskDesc)
	{
		return TaskDesc->Get();
	}

	return DefaultTaskDesc.Get();
}

void FSceneStateTaskDescRegistry::CacheTaskDescs()
{
	DefaultTaskDesc.InitializeAs<FSceneStateTaskDesc>();

	TaskDescs.Reset();

	for (const UScriptStruct* Struct : TObjectRange<UScriptStruct>())
	{
		if (Struct->IsChildOf<FSceneStateTaskDesc>())
		{
			TInstancedStruct<FSceneStateTaskDesc> NewInstance;
			NewInstance.InitializeAsScriptStruct(Struct);

			const FSceneStateTaskDesc& TaskDesc = NewInstance.Get();

			const UScriptStruct* const SupportedTask = TaskDesc.GetSupportedTask();
			if (!SupportedTask)
			{
				UE_LOGF(LogSceneStateEditor, Error, "Task Desc '%ls' cannot be registered because it does not have a valid supported task!"
					, *GetNameSafe(NewInstance.GetScriptStruct()));
				continue;
			}

			if (const TInstancedStruct<FSceneStateTaskDesc>* ExistingInstance = TaskDescs.Find(SupportedTask))
			{
				UE_LOGF(LogSceneStateEditor, Warning, "Existing Task Desc '%ls' will get replaced by '%ls'"
					, *GetNameSafe(ExistingInstance->GetScriptStruct())
					, *GetNameSafe(NewInstance.GetScriptStruct()));
			}

			TaskDescs.Add(SupportedTask, MoveTemp(NewInstance));
		}
	}
}

FString FSceneStateTaskDescRegistry::GetReferencerName() const
{
	return TEXT("FSceneStateTaskDescRegistry");
}

void FSceneStateTaskDescRegistry::AddReferencedObjects(FReferenceCollector& InCollector)
{
	DefaultTaskDesc.AddReferencedObjects(InCollector);

	for (TPair<TObjectKey<UScriptStruct>, TInstancedStruct<FSceneStateTaskDesc>>& Pair : TaskDescs)
	{
		Pair.Value.AddReferencedObjects(InCollector);
	}
}
