// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#define UE_API SCENESTATEEDITOR_API

class FText;
class UStruct;

namespace UE::SceneState::Editor
{
	/**
	 * Returns the tooltip for a given struct.
	 * This is different from UStruct::GetTooltipText as it will return an empty text if there was no explicit Tooltip found
	 * @return the found tooltip, or empty if not the metadata was not present.
	 */
	UE_API FText GetStructTooltip(const UStruct& InStruct);

} // UE::SceneState::Editor

#undef UE_API
