// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/StructView.h"
#include "SceneStateTaskDesc.generated.h"

#define UE_API SCENESTATEEDITOR_API

struct FSceneStateTask;
struct FSceneStateTaskInstance;

namespace UE::SceneState
{
	struct FStructIdChange;
}

/** Context to pass to the Task Desc objects to get information about the Task and Task Instance */
struct FSceneStateTaskDescContext
{
	/** Data view of the task */
	TConstStructView<FSceneStateTask> Task;

	/** Data view of the task instance */
	TConstStructView<FSceneStateTaskInstance> TaskInstance;

	/** Context object (e.g. graph node) using the task desc */
	TObjectPtr<const UObject> ContextObject;
};

struct FSceneStateTaskDescMutableContext
{
	/** Data view of the task */
	TStructView<FSceneStateTask> Task;

	/** Data view of the task instance */
	TStructView<FSceneStateTaskInstance> TaskInstance;

	/** Context object (e.g. graph node) using the task desc */
	TObjectPtr<UObject> ContextObject;
};

/** Base struct describing a task and its instance for editor-only data and behavior */
USTRUCT()
struct FSceneStateTaskDesc
{
	GENERATED_BODY()

	UE_API FSceneStateTaskDesc();

	virtual ~FSceneStateTaskDesc() = default;

	/** Return the task type this task desc supports */
	const UScriptStruct* GetSupportedTask() const
	{
		return SupportedTask;
	}

	/** Gets the display name of the task */
	UE_API FText GetDisplayName(const FSceneStateTaskDescContext& InContext) const;

	/** Gets the tooltip of the task */
	UE_API FText GetTooltip(const FSceneStateTaskDescContext& InContext) const;

	/** Gets the target object to go to on node double click */
	UE_API UObject* GetJumpTarget(const FSceneStateTaskDescContext& InContext) const;

	/** Called when struct ids have been changed, to be handled by tasks individually */
	UE_API void NotifyStructIdsChanged(const FSceneStateTaskDescMutableContext& InContext, const UE::SceneState::FStructIdChange& InChange) const;

	/** Set the task struct to support */
	template<typename T UE_REQUIRES(std::is_base_of_v<FSceneStateTask, T>)>
	void SetSupportedTask()
	{
		SetSupportedTask(T::StaticStruct());
	}

protected:
	/** Gets the display name for the task */
	virtual bool OnGetDisplayName(const FSceneStateTaskDescContext& InContext, FText& OutDisplayName) const
	{
		return false;
	}

	/** Gets the tooltip for the task */
	virtual bool OnGetTooltip(const FSceneStateTaskDescContext& InContext, FText& OutDescription) const
	{
		return false;
	}

	/** Gets the target object to go to on node double click */
	virtual bool OnGetJumpTarget(const FSceneStateTaskDescContext& InContext, UObject*& OutJumpTarget) const
	{
		return false;
	}

	/** Called when struct ids have been changed */
	virtual void OnStructIdsChanged(const FSceneStateTaskDescMutableContext& InContext, const UE::SceneState::FStructIdChange& InChange) const
	{
	}

private:
	/** Ensures the context is valid against the supported task */
	bool IsValidContext(const FSceneStateTaskDescContext& InContext) const;

	/** Ensures the context is valid against the supported task */
	bool IsValidContext(const FSceneStateTaskDescMutableContext& InContext) const;

	/** Set the task struct to support */
	UE_API void SetSupportedTask(UScriptStruct* InSupportedTask);

	/** The task struct to support */
	UPROPERTY()
	TObjectPtr<const UScriptStruct> SupportedTask = nullptr;
};

#undef UE_API
