// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "SceneStateTaskDesc.h"
#include "StructUtils/InstancedStruct.h"
#include "UObject/GCObject.h"
#include "UObject/ObjectKey.h"

#define UE_API SCENESTATEEDITOR_API

/** Registry holding editor-only task descs for each discovered task struct */
class FSceneStateTaskDescRegistry : public FGCObject
{
public:
	static FSceneStateTaskDescRegistry GlobalRegistry;

	/** Gets a const ref to the global registry */
	UE_API static const FSceneStateTaskDescRegistry& Get();

	/** Gets the task desc for the given task struct */
	UE_API const FSceneStateTaskDesc& GetTaskDesc(const UScriptStruct* InTaskStruct) const;

	/** Gathers all the available Tasks and stores it in this registry */
	void CacheTaskDescs();

protected:
	//~ Begin FGCObject
	virtual FString GetReferencerName() const override;
	virtual void AddReferencedObjects(FReferenceCollector& InCollector) override;
	//~ End FGCObject

private:
	/** Registered task structs to their task descs */
	TMap<TObjectKey<UScriptStruct>, TInstancedStruct<FSceneStateTaskDesc>> TaskDescs;

	/** Default task desc to use */
	TInstancedStruct<FSceneStateTaskDesc> DefaultTaskDesc;
};

#undef UE_API
