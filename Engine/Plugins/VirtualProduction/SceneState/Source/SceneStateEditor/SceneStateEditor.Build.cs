// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SceneStateEditor : ModuleRules
{
    public SceneStateEditor(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "Core",
                "CoreUObject",
                "SceneState",
            }
        );

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "Engine",
            }
        );
    }
}