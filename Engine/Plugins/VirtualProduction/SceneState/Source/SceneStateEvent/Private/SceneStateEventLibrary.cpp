// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateEventLibrary.h"
#include "Blueprint/BlueprintExceptionInfo.h"
#include "Engine/World.h"
#include "ISceneStateEventHandlerProvider.h"
#include "SceneStateEvent.h"
#include "SceneStateEventLog.h"
#include "SceneStateEventSchema.h"
#include "SceneStateEventSchemaHandle.h"
#include "SceneStateEventStream.h"
#include "SceneStateEventSubsystem.h"
#include "SceneStateEventUtils.h"

#define LOCTEXT_NAMESPACE "SceneStateEventLibrary"

bool USceneStateEventLibrary::PushEvent(USceneStateEventStream* InEventStream, FSceneStateEventSchemaHandle InEventSchemaHandle, FInstancedStruct InEventData)
{
	return UE::SceneState::PushEvent(InEventStream, InEventSchemaHandle, MoveTemp(InEventData));
}

bool USceneStateEventLibrary::BroadcastEvent(UObject* WorldContextObject, FSceneStateEventSchemaHandle InEventSchemaHandle, FInstancedStruct InEventData)
{
	if (const UWorld* World = UE::SceneState::GetContextWorld(WorldContextObject))
	{
		return UE::SceneState::BroadcastEvent(World, InEventSchemaHandle, MoveTemp(InEventData));
	}

	UE_LOGF(LogSceneStateEvent, Error, "BroadcastEvent failed. Could not find a valid world from context object '%ls'", *GetFullNameSafe(WorldContextObject));
	return false;
}

bool USceneStateEventLibrary::EventDataToStruct(const FInstancedStruct& InEventData, int32& OutStructValue)
{
	// This should never be called as it has Custom Thunk
	checkNoEntry();
	return false;
}

DEFINE_FUNCTION(USceneStateEventLibrary::execEventDataToStruct)
{
	P_GET_STRUCT_REF(FInstancedStruct, InEventData);

	// Read Out Struct property
	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.StepCompiledIn<FStructProperty>(nullptr);

	const FStructProperty* StructProperty = CastField<FStructProperty>(Stack.MostRecentProperty);
	void* StructAddress = Stack.MostRecentPropertyAddress;

	P_FINISH;

	if (!StructProperty || !StructAddress)
	{
		FBlueprintExceptionInfo ExceptionInfo(EBlueprintExceptionType::AbortExecution, LOCTEXT("InvalidStructWarning", "Failed to resolve the Struct for 'Event Data To Struct'"));
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}
	else
	{
		P_NATIVE_BEGIN;
		if (InEventData.IsValid() && InEventData.GetScriptStruct()->IsChildOf(StructProperty->Struct))
		{
			StructProperty->Struct->CopyScriptStruct(StructAddress, InEventData.GetMemory());
			*(bool*)RESULT_PARAM = true;
		}
		else
		{
			*(bool*)RESULT_PARAM = false;
		}
		P_NATIVE_END;
	}
}

bool USceneStateEventLibrary::TryGetEventHandlerId(UObject* InContextObject
	, const FSceneStateEventSchemaHandle& InEventSchemaHandle
	, const FString& InEventHandlerId
	, FGuid& OutEventHandlerId)
{
	// Prioritize the passed in String value of the Guid
	if (!InEventHandlerId.IsEmpty())
	{
		LexFromString(OutEventHandlerId, *InEventHandlerId);
		return true;
	}

	if (!InContextObject)
	{
		return false;
	}

	// Find the Event Handler Provider from the Context Object

	ISceneStateEventHandlerProvider* EventHandlerProvider = Cast<ISceneStateEventHandlerProvider>(InContextObject);
	if (!EventHandlerProvider)
	{
		EventHandlerProvider = InContextObject->GetImplementingOuter<ISceneStateEventHandlerProvider>();
	}

	return EventHandlerProvider && EventHandlerProvider->FindEventHandlerId(InEventSchemaHandle, OutEventHandlerId);
}

bool USceneStateEventLibrary::FindEvent(UObject* InContextObject
	, USceneStateEventStream* InEventStream
	, FSceneStateEventSchemaHandle InEventSchemaHandle
	, const FString& InEventHandlerId
	, bool bInCapturedEventsOnly
	, FInstancedStruct& OutEventData)
{
	if (!InEventStream)
	{
		return false;
	}

	const USceneStateEventSchemaObject* const EventSchema = InEventSchemaHandle.GetEventSchema();
	if (!EventSchema)
	{
		return false;
	}

	FGuid EventHandlerId;

	// If an Event Handler Id was provided, use it to find the captured event
	if (TryGetEventHandlerId(InContextObject, InEventSchemaHandle, InEventHandlerId, EventHandlerId))
	{
		if (const FSceneStateEvent* Event = InEventStream->FindCapturedEvent(EventHandlerId))
		{
			// The event schema id of the event should match the provided schema
			// else an incorrect event schema or handler id was provided
			if (ensureAlways(Event->GetId() == EventSchema->Id))
			{
				OutEventData = Event->GetDataView();
                return true;
			}
		}
	}

	// Searching for captured events only, no matching captured event was found
	if (bInCapturedEventsOnly)
	{
		return false;
	}

	// No Event Handler Id was provided, return the first event that matches the schema
	// This can happen in cases where this is called outside of a scope capturing the event of interest
	// (e.g. an external blueprint)
	if (const FSceneStateEvent* Event = InEventStream->FindEventBySchema(InEventSchemaHandle))
	{
		OutEventData = Event->GetDataView();
		return true;
	}

	return false;
}

bool USceneStateEventLibrary::HasEvent(UObject* InContextObject
	, USceneStateEventStream* InEventStream
	, FSceneStateEventSchemaHandle InEventSchemaHandle
	, const FString& InEventHandlerId
	, bool bInCapturedEventsOnly)
{
	if (!InEventStream)
	{
		return false;
	}

	FGuid EventHandlerId;
	if (TryGetEventHandlerId(InContextObject, InEventSchemaHandle, InEventHandlerId, EventHandlerId))
	{
		// If an Event Handler Id was provided, use it to find the captured event
		if (InEventStream->FindCapturedEvent(EventHandlerId))
		{ 
			return true;
		}
	}

	// Searching for captured events only, no matching captured event was found
	if (bInCapturedEventsOnly)
	{
		return false;
	}

	// Find an existing not captured event
	return InEventStream->FindEventBySchema(InEventSchemaHandle) != nullptr;
}

#undef LOCTEXT_NAMESPACE
