// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateEventSchema.h"
#include "SceneStateEvent.h"
#include "SceneStateEventLog.h"
#include "SceneStateEventSchemaCollection.h"
#include "StructUtils/UserDefinedStruct.h"

FSharedStruct USceneStateEventSchemaObject::CreateEvent(FInstancedStruct&& InEventData) const
{
	FSharedStruct EventStruct = FSharedStruct::Make<FSceneStateEvent>();

	FSceneStateEvent& Event = EventStruct.Get<FSceneStateEvent>();
	Event.Id = Id;

	if (InEventData.GetScriptStruct() == Struct)
	{
		Event.Data = MoveTemp(InEventData);
	}
	else
	{
		UE_CLOGF(InEventData.IsValid(), LogSceneStateEvent, Error, "Event Data '%ls' does not match Schema struct '%ls'. "
			"Event will be initialized with Schema default values."
			, *GetNameSafe(InEventData.GetScriptStruct())
			, *Name.ToString());

		// Struct ok to be null here, all this will do is reset the event data
		Event.Data.InitializeAs(Struct);
	}

	UE_LOGF(LogSceneStateEvent, Verbose, "Event '%ls' has been created", *Name.ToString());
	return EventStruct;
}

void USceneStateEventSchemaObject::PostInitProperties()
{
	Super::PostInitProperties();

	if (!IsTemplate())
	{
		Id = FGuid::NewGuid();
	}
}
