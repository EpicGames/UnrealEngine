// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateEventSchemaCollection.h"
#include "SceneStateEventSchema.h"

USceneStateEventSchemaObject* USceneStateEventSchemaCollection::GetEventSchema(const FGuid& InEventSchemaId) const
{
	if (!InEventSchemaId.IsValid())
	{
		return nullptr;
	}

	for (USceneStateEventSchemaObject* EventSchema : EventSchemas)
	{
		if (EventSchema && EventSchema->Id == InEventSchemaId)
		{
			return EventSchema;
		}
	}

	return nullptr;
}
