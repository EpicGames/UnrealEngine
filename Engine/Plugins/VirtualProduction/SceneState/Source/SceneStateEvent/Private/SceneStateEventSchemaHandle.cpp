// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateEventSchemaHandle.h"
#include "SceneStateEventSchema.h"
#include "SceneStateEventSchemaCollection.h"
#include "StructUtils/StructView.h"
#include "StructUtils/UserDefinedStruct.h"

UUserDefinedStruct* FSceneStateEventSchemaHandle::GetEventStruct() const
{
	return EventStruct.LoadSynchronous();
}

USceneStateEventSchemaObject* FSceneStateEventSchemaHandle::GetEventSchema() const
{
	return EventSchema.LoadSynchronous();
}

FStructView FSceneStateEventSchemaHandle::GetDefaultDataView() const
{
	if (UUserDefinedStruct* Struct = GetEventStruct())
	{
		return FStructView(Struct, const_cast<uint8*>(Struct->GetDefaultInstance()));
	}
	return FStructView();
}
