// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateEventStream.h"
#include "SceneStateEventHandler.h"
#include "SceneStateEventLog.h"
#include "SceneStateEventSchema.h"
#include "SceneStateEventSubsystem.h"

bool USceneStateEventStream::Register()
{
	if (USceneStateEventSubsystem* const EventSubsystem = USceneStateEventSubsystem::Get())
	{
		EventSubsystem->RegisterEventStream(this);
		return true;
	}
	return false;
}

void USceneStateEventStream::Unregister()
{
	if (USceneStateEventSubsystem* const EventSubsystem = USceneStateEventSubsystem::Get())
	{
		EventSubsystem->UnregisterEventStream(this);
	}
}

void USceneStateEventStream::PushEvent(FSharedStruct&& InEvent)
{
	Events.Add(MoveTemp(InEvent));
}

void USceneStateEventStream::PushEvent(const FSharedStruct& InEvent)
{
	Events.Add(InEvent);
}

bool USceneStateEventStream::ConsumeEventBySchema(const FSceneStateEventSchemaHandle& InEventSchemaHandle)
{
	// Find first Event that matches the Schema Id
	const int32 EventIndex = GetEventIndexBySchema(InEventSchemaHandle);

	if (EventIndex != INDEX_NONE)
	{
		Events.RemoveAt(EventIndex);
		return true;
	}

	return false;
}

const FSceneStateEvent* USceneStateEventStream::FindEventBySchema(const FSceneStateEventSchemaHandle& InEventSchemaHandle) const
{
	const USceneStateEventSchemaObject* const EventSchema = InEventSchemaHandle.GetEventSchema();
	if (!EventSchema)
	{
		return nullptr;
	}

	const FSharedStruct* FoundEvent = Events.FindByPredicate(
		[EventSchema](const FSharedStruct& InEvent)
		{
			return InEvent.Get<FSceneStateEvent>().GetId() == EventSchema->Id;
		});

	return FoundEvent ? &FoundEvent->Get<FSceneStateEvent>() : nullptr;
}

FSceneStateEvent* USceneStateEventStream::FindCapturedEvent(const FGuid& InHandlerId)
{
	const FSharedStruct* FoundEvent = CapturedEvents.Find(InHandlerId);
	return FoundEvent ? &FoundEvent->Get<FSceneStateEvent>() : nullptr;
}

void USceneStateEventStream::CaptureEvents(TConstArrayView<FSceneStateEventHandler> InEventHandlers)
{
	for (const FSceneStateEventHandler& EventHandler : InEventHandlers)
	{
		// Find the first Event that matches the Handler's Schema
		// and move it to handled
		const int32 EventIndex = GetEventIndexBySchema(EventHandler.GetEventSchemaHandle());
		if (EventIndex != INDEX_NONE)
		{
			CapturedEvents.Add(EventHandler.GetHandlerId(), MoveTempIfPossible(Events[EventIndex]));
			Events.RemoveAt(EventIndex);
		}
	}
}

void USceneStateEventStream::ResetCapturedEvents(TConstArrayView<FSceneStateEventHandler> InEventHandlers)
{
	for (const FSceneStateEventHandler& EventHandler : InEventHandlers)
	{
		CapturedEvents.Remove(EventHandler.GetHandlerId());
	}
}

int32 USceneStateEventStream::GetEventIndexBySchema(const FSceneStateEventSchemaHandle& InEventSchemaHandle) const
{
	const USceneStateEventSchemaObject* const EventSchema = InEventSchemaHandle.GetEventSchema();
	if (!EventSchema)
	{
		return INDEX_NONE;
	}

	return Events.IndexOfByPredicate(
		[EventSchema](const FSharedStruct& InEvent)
		{
			return InEvent.Get<FSceneStateEvent>().GetId() == EventSchema->Id;
		});
}
