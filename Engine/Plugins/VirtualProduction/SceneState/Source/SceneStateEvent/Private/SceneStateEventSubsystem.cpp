// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateEventSubsystem.h"
#include "Engine/Engine.h"
#include "SceneStateEventStream.h"

USceneStateEventSubsystem* USceneStateEventSubsystem::Get()
{
	return GEngine ? GEngine->GetEngineSubsystem<USceneStateEventSubsystem>() : nullptr;
}

void USceneStateEventSubsystem::RegisterEventStream(USceneStateEventStream* InEventStream)
{
	EventStreams.AddUnique(InEventStream);
}

void USceneStateEventSubsystem::UnregisterEventStream(USceneStateEventStream* InEventStream)
{
	EventStreams.Remove(InEventStream);
}

void USceneStateEventSubsystem::ForEachEventStream(TFunctionRef<void(USceneStateEventStream*)> InCallable)
{
	for (USceneStateEventStream* EventStream : EventStreams)
	{
		if (EventStream)
		{
			InCallable(EventStream);
		}
	}
}
