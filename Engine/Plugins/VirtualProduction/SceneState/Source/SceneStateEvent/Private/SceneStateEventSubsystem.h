// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Subsystems/EngineSubsystem.h"
#include "Templates/FunctionFwd.h"
#include "SceneStateEventSubsystem.generated.h"

class USceneStateEventStream;

UCLASS()
class USceneStateEventSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()

public:
	static USceneStateEventSubsystem* Get();

	/** Adds the given event stream to the list of actively registered event streams */
	void RegisterEventStream(USceneStateEventStream* InEventStream);

	/** Removes the given event stream from the list of actively registered event streams */
	void UnregisterEventStream(USceneStateEventStream* InEventStream);

	/** Iterates all valid event streams that are actively registered */
	void ForEachEventStream(TFunctionRef<void(USceneStateEventStream*)> InCallable);

private:
	UPROPERTY()
	TArray<TObjectPtr<USceneStateEventStream>> EventStreams;
};
