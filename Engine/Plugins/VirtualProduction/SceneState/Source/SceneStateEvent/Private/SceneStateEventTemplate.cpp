// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateEventTemplate.h"
#include "StructUtils/UserDefinedStruct.h"

#if WITH_EDITOR
void FSceneStateEventTemplate::SyncEventData()
{
	const UUserDefinedStruct* EventStruct = EventSchemaHandle.GetEventStruct();
	if (EventData.GetScriptStruct() != EventStruct)
	{
		EventData.InitializeAs(EventStruct);
	}
}
#endif
