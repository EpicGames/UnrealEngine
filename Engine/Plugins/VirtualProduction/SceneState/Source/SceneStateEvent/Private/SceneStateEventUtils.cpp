// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateEventUtils.h"
#include "Engine/World.h"
#include "SceneStateEventLog.h"
#include "SceneStateEventSchema.h"
#include "SceneStateEventSchemaHandle.h"
#include "SceneStateEventStream.h"
#include "SceneStateEventSubsystem.h"
#include "SceneStateEventTemplate.h"
#include "StructUtils/InstancedStruct.h"

namespace UE::SceneState
{

namespace Private
{

bool IsEventStreamInContext(const USceneStateEventStream* InEventStream, const UObject* InEventContext)
{
	// Special case for Worlds: instead of checking outer chain, check that the world the event stream returns is the same
	if (const UWorld* ContextWorld = Cast<UWorld>(InEventContext))
	{
		return GetContextWorld(InEventStream) == ContextWorld;
	}
	return InEventStream->IsIn(InEventContext);
}

} // Private

bool PushEvent(USceneStateEventStream* InEventStream, const FSceneStateEventSchemaHandle& InEventSchemaHandle, FInstancedStruct&& InEventData)
{
	if (!InEventStream)
	{
		return false;
	}

	const USceneStateEventSchemaObject* EventSchema = InEventSchemaHandle.GetEventSchema();
	if (!EventSchema)
	{
		return false;
	}

	InEventStream->PushEvent(EventSchema->CreateEvent(MoveTemp(InEventData)));
	return true;
}

bool PushEvent(USceneStateEventStream* InEventStream, const FSceneStateEventTemplate& InEventTemplate)
{
	return PushEvent(InEventStream, InEventTemplate.GetEventSchemaHandle(), FInstancedStruct(InEventTemplate.GetEventData()));
}

const UWorld* GetContextWorld(const UObject* InObject)
{
	const UWorld* World = InObject ? InObject->GetWorld() : nullptr;
	return World ? World : GWorld;
}

bool BroadcastEvent(const UObject* InEventContext, const FSceneStateEventSchemaHandle& InEventSchemaHandle, FInstancedStruct&& InEventData)
{
	if (!InEventContext)
	{
		UE_LOGF(LogSceneStateEvent, Error, "Broadcast event failed. Event context is invalid.");
		return false;
	}

	USceneStateEventSubsystem* const EventSubsystem = USceneStateEventSubsystem::Get();
	if (!EventSubsystem)
	{
		UE_LOGF(LogSceneStateEvent, Error, "Broadcast event failed. Event subsystem not found.");
		return false;
	}

	const USceneStateEventSchemaObject* EventSchema = InEventSchemaHandle.GetEventSchema();
	if (!EventSchema)
	{
		UE_LOGF(LogSceneStateEvent, Error, "Broadcast event failed. Event schema not found.");
		return false;
	}

	const FSharedStruct Event = EventSchema->CreateEvent(MoveTemp(InEventData));

	EventSubsystem->ForEachEventStream(
		[&Event, InEventContext](USceneStateEventStream* InEventStream)
		{
			// Only consider event streams that are within the context
			if (Private::IsEventStreamInContext(InEventStream, InEventContext))
			{
				InEventStream->PushEvent(Event);
			}
		});

	return true;
}

bool BroadcastEvent(const UObject* InEventContext, const FSceneStateEventTemplate& InEventTemplate)
{
	return BroadcastEvent(InEventContext, InEventTemplate.GetEventSchemaHandle(), FInstancedStruct(InEventTemplate.GetEventData()));
}

} // UE::SceneState
