// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "ISceneStateEventHandlerProvider.generated.h"

struct FSceneStateEventSchemaHandle;

UINTERFACE(MinimalAPI, meta=(CannotImplementInterfaceInBlueprint))
class USceneStateEventHandlerProvider : public UInterface
{
	GENERATED_BODY()
};

class ISceneStateEventHandlerProvider
{
	GENERATED_BODY()

public:
	/** Find the Event Handler for the given Event Schema */
	virtual bool FindEventHandlerId(const FSceneStateEventSchemaHandle& InEventSchemaHandle, FGuid& OutHandlerId) const = 0;
};
