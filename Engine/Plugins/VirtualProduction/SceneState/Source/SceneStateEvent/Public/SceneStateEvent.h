// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/InstancedStruct.h"
#include "StructUtils/StructView.h"
#include "SceneStateEvent.generated.h"

/** Holds the name and payload data of an Event handled by Scene State */
USTRUCT()
struct FSceneStateEvent
{
    friend class USceneStateEventSchemaObject;

    GENERATED_BODY()

    FSceneStateEvent() = default;

    const FGuid& GetId() const
    {
        return Id;
    }

    FConstStructView GetDataView() const
    {
        return Data;
    }

    FStructView GetDataViewMutable()
    {
        return Data;
    }

private:
    /** Id of the Schema that was used to create this Event */
    UPROPERTY(meta=(IgnoreForMemberInitializationTest))
    FGuid Id;

    /** Payload Data of the Event */
    UPROPERTY()
    FInstancedStruct Data;
};
