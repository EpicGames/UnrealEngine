// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"
#include "SceneStateEventSchemaHandle.h"
#include "SceneStateEventHandler.generated.h"

USTRUCT(BlueprintType)
struct FSceneStateEventHandler
{
	GENERATED_BODY()

	FSceneStateEventHandler() = default;

	/**
	 * Force init where Id is initialized to a new guid.
	 * This is used in TCppStructOps::Construct
	 */
	explicit FSceneStateEventHandler(EForceInit)
		: HandlerId(FGuid::NewGuid())
	{
	}

	bool operator==(const FSceneStateEventHandler& InOther) const
	{
		return HandlerId == InOther.HandlerId;
	}

	friend uint32 GetTypeHash(const FSceneStateEventHandler& InEventHandler)
	{
		return GetTypeHash(InEventHandler.HandlerId);
	}

	const FGuid& GetHandlerId() const
	{
		return HandlerId;
	}

	const FSceneStateEventSchemaHandle& GetEventSchemaHandle() const
	{
		return SchemaHandle;
	}

	static FName GetSchemaHandlePropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(FSceneStateEventHandler, SchemaHandle);
	}

private:
	/** Unique Identifier for this Handler */
	UPROPERTY(meta=(IgnoreForMemberInitializationTest))
	FGuid HandlerId;

	/** Handle to the Schema of the Event */
	UPROPERTY(EditAnywhere, Category="Event")
	FSceneStateEventSchemaHandle SchemaHandle;
};

template<>
struct TStructOpsTypeTraits<FSceneStateEventHandler> : TStructOpsTypeTraitsBase2<FSceneStateEventHandler>
{
	enum
	{
		WithNoInitConstructor = true,
	};
};
