// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "StructUtils/InstancedStruct.h"
#include "SceneStateEventLibrary.generated.h"

class USceneStateEventStream;
struct FSceneStateEventSchemaHandle;

UCLASS(MinimalAPI)
class USceneStateEventLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/**
	 * Pushes an event to the given event stream
	 * @param InEventStream the stream to push the event to
	 * @param InEventSchemaHandle handle to the event schema used to create the event
	 * @param InEventData the event payload. Must match the event schema struct
	 * @return true if the event was pushed successfully, false otherwise
	 */
	UFUNCTION(BlueprintCallable, Category="Scene State|Event", meta=(BlueprintInternalUseOnly="true"))
	static bool PushEvent(USceneStateEventStream* InEventStream, FSceneStateEventSchemaHandle InEventSchemaHandle, FInstancedStruct InEventData = FInstancedStruct());

	/**
	 * Broadcasts an event to the every registered event stream within a world.
	 * @param WorldContextObject context object to get the world
	 * @param InEventSchemaHandle handle to the event schema used to create the event
	 * @param InEventData the event payload. Must match the event schema struct
	 * @return true if the event was broadcasted successfully, false otherwise
	 */
	UFUNCTION(BlueprintCallable, Category="Scene State|Event", meta=(BlueprintInternalUseOnly="true", WorldContext="WorldContextObject"))
	static bool BroadcastEvent(UObject* WorldContextObject, FSceneStateEventSchemaHandle InEventSchemaHandle, FInstancedStruct InEventData = FInstancedStruct());

	/**
	 * Converts the given event data instanced struct to the wild card if it matches struct types
	 * @param InEventData the event data instanced struct to convert
	 * @param OutStructValue the returned value (wild card) if it matches in struct with the event data
	 * @return true if the conversion succeeeded
	 */
	UFUNCTION(BlueprintPure, CustomThunk, Category="Scene State|Event", meta=(CustomStructureParam="OutStructValue", BlueprintInternalUseOnly="true"))
	static bool EventDataToStruct(UPARAM(Ref) const FInstancedStruct& InEventData, int32& OutStructValue);

	/** Finds a captured event within an Event Stream with option to look into any pushed event in that stream (not only captured) */
	UFUNCTION(BlueprintPure, Category="Scene State|Event", meta=(BlueprintInternalUseOnly="true"))
	static bool FindEvent(UObject* InContextObject, USceneStateEventStream* InEventStream, FSceneStateEventSchemaHandle InEventSchemaHandle, const FString& InEventHandlerId, bool bInCapturedEventsOnly, FInstancedStruct& OutEventData);

	/** Returns whether an event is present as a captured event (or optionally just pushed) within an Event Stream */
	UFUNCTION(BlueprintPure, Category="Scene State|Event", meta=(BlueprintInternalUseOnly="true"))
	static bool HasEvent(UObject* InContextObject, USceneStateEventStream* InEventStream, FSceneStateEventSchemaHandle InEventSchemaHandle, const FString& InEventHandlerId, bool bInCapturedEventsOnly);

private:
	DECLARE_FUNCTION(execEventDataToStruct);

	static bool TryGetEventHandlerId(UObject* InContextObject, const FSceneStateEventSchemaHandle& InEventSchemaHandle, const FString& InEventHandlerId, FGuid& OutEventHandlerId);
};
