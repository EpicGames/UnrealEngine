// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "SceneStateEventSchema.generated.h"

class UUserDefinedStruct;
struct FInstancedStruct;
struct FSharedStruct;

UCLASS(MinimalAPI, EditInlineNew)
class USceneStateEventSchemaObject : public UObject
{
	GENERATED_BODY()

public:
	USceneStateEventSchemaObject() = default;

	/** Creates an event based on this event schema object, ensuring that the event data also matches the event schema struct */
	FSharedStruct CreateEvent(FInstancedStruct&& InEventData) const;

	//~ Begin UObject
	virtual void PostInitProperties() override;
	//~ End UObject

	UPROPERTY(DuplicateTransient, TextExportTransient, meta=(IgnoreForMemberInitializationTest))
	FGuid Id;

	UPROPERTY()
	FName Name;

	UPROPERTY(Instanced)
	TObjectPtr<UUserDefinedStruct> Struct;
};
