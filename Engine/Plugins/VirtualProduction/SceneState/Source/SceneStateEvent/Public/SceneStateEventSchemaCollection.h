// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "SceneStateEventSchemaCollection.generated.h"

class USceneStateEventSchemaObject;

/** Holds a collection of Event Schemas to group common event types in one place */
UCLASS(MinimalAPI, DisplayName="Motion Design Scene State Event Schema Collection")
class USceneStateEventSchemaCollection : public UObject
{
	GENERATED_BODY()

public:
	/** Gets the event schema matching a given id */
	USceneStateEventSchemaObject* GetEventSchema(const FGuid& InEventSchemaId) const;

	TConstArrayView<USceneStateEventSchemaObject*> GetEventSchemas() const
	{
		return EventSchemas;
	}

	static FName GetEventSchemasName()
	{
		return GET_MEMBER_NAME_CHECKED(USceneStateEventSchemaCollection, EventSchemas);
	}

private:
	/** Event schemas owned by this collection */
	UPROPERTY(EditAnywhere, Instanced, Category="Event Schema")
	TArray<TObjectPtr<USceneStateEventSchemaObject>> EventSchemas;
};
