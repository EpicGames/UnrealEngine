// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/SoftObjectPtr.h"
#include "SceneStateEventSchemaHandle.generated.h"

#define UE_API SCENESTATEEVENT_API

class USceneStateEventSchemaObject;
class UUserDefinedStruct;
struct FStructView;

/** Handle to an Event Schema */
USTRUCT(BlueprintType)
struct FSceneStateEventSchemaHandle
{
	GENERATED_BODY()

	/** Get the struct of the Event Schema. Can be null if it's an event with no parameters */
	UE_API UUserDefinedStruct* GetEventStruct() const;

	/** Resolves the event schema soft object pointer */
	UE_API USceneStateEventSchemaObject* GetEventSchema() const;

	/** Get the default data view of the Event Schema struct. Can be invalid if it's an event with no parameters */
	UE_API FStructView GetDefaultDataView() const;

	bool operator==(const FSceneStateEventSchemaHandle& InOther) const
	{
		return EventSchema == InOther.EventSchema;
	}

	static FName GetEventSchemaPropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(FSceneStateEventSchemaHandle, EventSchema);
	}

	static FName GetEventStructPropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(FSceneStateEventSchemaHandle, EventStruct);
	}

private:
	/** Soft reference to the event schema */
	UPROPERTY(EditAnywhere, Category="Event")
	TSoftObjectPtr<USceneStateEventSchemaObject> EventSchema;

	/** Hold a soft reference to the struct within the event schema, if the event schema has a valid struct */
	UPROPERTY(EditAnywhere, Category="Event")
	TSoftObjectPtr<UUserDefinedStruct> EventStruct;
};

#undef UE_API
