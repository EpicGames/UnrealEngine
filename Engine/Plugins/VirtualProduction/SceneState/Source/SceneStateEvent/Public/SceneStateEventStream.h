// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "SceneStateEvent.h"
#include "UObject/Object.h"
#include "SceneStateEventStream.generated.h"

#define UE_API SCENESTATEEVENT_API

struct FSceneStateEventHandler;
struct FSceneStateEventSchemaHandle;

/** Holds and keeps track of Events added at Runtime */
UCLASS(MinimalAPI)
class USceneStateEventStream : public UObject
{
	GENERATED_BODY()

public:
	/** Registers the Event Stream to the Event Subsystem to listen to Broadcast Events */
	UE_API bool Register();

	/** Registers the Event Stream from the Event Subsystem */
	UE_API void Unregister();

	/** Pushes a new event into the stream */
	UE_API void PushEvent(FSharedStruct&& InEvent);

	/** Pushes a new event into the stream */
	UE_API void PushEvent(const FSharedStruct& InEvent);

	/** Consumes the first (oldest) Event that was pushed that matches the given Schema */
	UE_API bool ConsumeEventBySchema(const FSceneStateEventSchemaHandle& InEventSchemaHandle);

	/** Finds the first (oldest) Event that was pushed that matches the given Schema */
	UE_API const FSceneStateEvent* FindEventBySchema(const FSceneStateEventSchemaHandle& InEventSchemaHandle) const;

	/** Finds the event that is captured by the Handler with the given Id */
	UE_API FSceneStateEvent* FindCapturedEvent(const FGuid& InHandlerId);

	/** Removes the Events (first/oldest) that match the given Event Handlers and moves them to the Captured Event map */
	UE_API void CaptureEvents(TConstArrayView<FSceneStateEventHandler> InEventHandlers);

	/** Cleanup the Captured Events that match the given Event Handlers */
	UE_API void ResetCapturedEvents(TConstArrayView<FSceneStateEventHandler> InEventHandlers);

private:
	/** Returns the index of the first event matching the event schema */
	int32 GetEventIndexBySchema(const FSceneStateEventSchemaHandle& InEventSchemaHandle) const;

	/** Active Events kept in push order */
	UPROPERTY(Transient, DuplicateTransient, TextExportTransient)
	TArray<FSharedStruct> Events;

	/** Map of the Handler Id to the Event it has captured */
	UPROPERTY(Transient, DuplicateTransient, TextExportTransient)
	TMap<FGuid, FSharedStruct> CapturedEvents;
};

#undef UE_API
