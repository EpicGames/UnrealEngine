// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneStateEventSchemaHandle.h"
#include "StructUtils/InstancedStruct.h"
#include "SceneStateEventTemplate.generated.h"

#define UE_API SCENESTATEEVENT_API

/** Defines a handle to an event schema and the payload event data to copy to events to push */
USTRUCT(BlueprintType)
struct FSceneStateEventTemplate
{
	GENERATED_BODY()

	const FSceneStateEventSchemaHandle& GetEventSchemaHandle() const
	{
		return EventSchemaHandle;
	}

	const FInstancedStruct& GetEventData() const
	{
		return EventData;
	}

#if WITH_EDITOR
	/** Syncs the event data to match the event schema struct */
	UE_API void SyncEventData();
#endif

	static FName GetEventSchemaHandlePropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(FSceneStateEventTemplate, EventSchemaHandle);
	}

	static FName GetEventDataPropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(FSceneStateEventTemplate, EventData);
	}

private:
	/** Handle to the event schema to use */
	UPROPERTY(EditAnywhere, Category="Event")
	FSceneStateEventSchemaHandle EventSchemaHandle;

	/** Data initialized to the event schema, if it has a valid event struct */
	UPROPERTY(EditAnywhere, Category="Event", meta=(StructTypeConst))
	FInstancedStruct EventData;
};

#undef UE_API
