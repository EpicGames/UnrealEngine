// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

class UObject;
class USceneStateEventStream;
class UWorld;
struct FInstancedStruct;
struct FSceneStateEventSchemaHandle;
struct FSceneStateEventTemplate;

#define UE_API SCENESTATEEVENT_API

namespace UE::SceneState
{
	/**
	 * Pushes an event to the given event stream
	 * @param InEventStream the stream to push the event to
	 * @param InEventSchemaHandle handle to the event schema used to create the event
	 * @param InEventData the event payload. Must match the event schema struct
	 * @return true if the event was pushed successfully, false otherwise
	 */
	UE_API bool PushEvent(USceneStateEventStream* InEventStream, const FSceneStateEventSchemaHandle& InEventSchemaHandle, FInstancedStruct&& InEventData);

	/**
	 * Pushes an event to the given event stream
	 * @param InEventStream the stream to push the event to
	 * @param InEventTemplate the template used to create the event
	 * @return true if the event was pushed successfully, false otherwise
	 */
	UE_API bool PushEvent(USceneStateEventStream* InEventStream, const FSceneStateEventTemplate& InEventTemplate);

	/**
	 * Retrieves a context world for a given object.
	 * @param InObject the context object to look a context world for
	 * @return the context world or GWorld as fallback if no explicit world was found.
	 */
	UE_API const UWorld* GetContextWorld(const UObject* InObject);

	/**
	 * Broadcasts an event to the every registered event stream within a given scope.
	 * @param InEventContext required scope to which the event will be broadcasted to. Only Event Streams within this scope will receive the broadcast event.
	 * @param InEventSchemaHandle handle to the event schema used to create the event
	 * @param InEventData the event payload. Must match the event schema struct
	 * @return true if the event was broadcasted successfully, false otherwise
	 */
	UE_API bool BroadcastEvent(const UObject* InEventContext, const FSceneStateEventSchemaHandle& InEventSchemaHandle, FInstancedStruct&& InEventData);

	/**
	 * Broadcasts an event to the every registered event stream within a given scope.
	 * @param InEventContext required scope to which the event will be broadcasted to. Only Event Streams within this scope will receive the broadcast event.
	 * @param InEventTemplate the template used to create the event
	 * @return true if the event was broadcasted successfully, false otherwise
	 */
	UE_API bool BroadcastEvent(const UObject* InEventContext, const FSceneStateEventTemplate& InEventTemplate);

} // UE::SceneState

#undef UE_API
