// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SceneStateEvent : ModuleRules
{
    public SceneStateEvent(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "Core", 
                "CoreUObject",
            }
        );

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "Engine",
                "Slate",
                "SlateCore"
            }
        );
    }
}