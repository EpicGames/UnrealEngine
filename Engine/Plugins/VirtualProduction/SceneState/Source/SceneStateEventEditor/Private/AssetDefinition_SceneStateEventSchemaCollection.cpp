// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_SceneStateEventSchemaCollection.h"
#include "BlueprintEditorModule.h"
#include "Kismet2/StructureEditorUtils.h"
#include "SceneStateEventSchemaCollection.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_SceneStateEventSchemaCollection"

FText UAssetDefinition_SceneStateEventSchemaCollection::GetAssetDisplayName() const
{
	return LOCTEXT("AssetDisplayName", "Scene State Event Schema Collection");
}

FLinearColor UAssetDefinition_SceneStateEventSchemaCollection::GetAssetColor() const
{
	return FLinearColor(FColor(152, 251, 152));
}

TSoftClassPtr<UObject> UAssetDefinition_SceneStateEventSchemaCollection::GetAssetClass() const
{
	return USceneStateEventSchemaCollection::StaticClass();
}

#undef LOCTEXT_NAMESPACE
   