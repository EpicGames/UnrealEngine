// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinition.h"
#include "AssetDefinitionDefault.h"
#include "AssetDefinition_SceneStateEventSchemaCollection.generated.h"

UCLASS()
class UAssetDefinition_SceneStateEventSchemaCollection : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	//~ Begin UAssetDefinition
	virtual FText GetAssetDisplayName() const override;
	virtual FLinearColor GetAssetColor() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	//~ End UAssetDefinition
};
