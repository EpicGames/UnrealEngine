// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateEventHandlerCustomization.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "SceneStateEventHandler.h"

namespace UE::SceneState::Editor
{

void FEventHandlerCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	TSharedPtr<IPropertyHandle> EventSchemaHandle = InPropertyHandle->GetChildHandle(FSceneStateEventHandler::GetSchemaHandlePropertyName());
	check(EventSchemaHandle);

	InHeaderRow
		.NameContent()
		[
			InPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MinDesiredWidth(200.f)
		[
			EventSchemaHandle->CreatePropertyValueWidgetWithCustomization(nullptr)
		];
}

void FEventHandlerCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
}

} // UE::SceneState::Editor
