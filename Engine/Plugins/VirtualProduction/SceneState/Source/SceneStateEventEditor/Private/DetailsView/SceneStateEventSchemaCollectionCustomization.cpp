// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateEventSchemaCollectionCustomization.h"
#include "DetailLayoutBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "SceneStateEventSchema.h"
#include "SceneStateEventSchemaCollection.h"
#include "SceneStateEventSchemaNodeBuilder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

namespace UE::SceneState::Editor
{

void FEventSchemaCollectionCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	TSharedRef<IPropertyHandle> EventSchemasHandle = InDetailBuilder.GetProperty(USceneStateEventSchemaCollection::GetEventSchemasName());
	EventSchemasHandle->MarkHiddenByCustomization();

	TSharedRef<FDetailArrayBuilder> EventSchemasBuilder = MakeShared<FDetailArrayBuilder>(EventSchemasHandle, /*bGenerateHeader*/false);

	EventSchemasBuilder->OnGenerateArrayElementWidget(FOnGenerateArrayElementWidget::CreateLambda(
		[](TSharedRef<IPropertyHandle> InEventSchemaHandle, int32 InChildIndex, IDetailChildrenBuilder& InChildBuilder)
		{
			InChildBuilder.AddCustomBuilder(MakeShared<FEventSchemaNodeBuilder>(InEventSchemaHandle));
		}));

	IDetailCategoryBuilder& EventSchemaCategory = InDetailBuilder.EditCategory(TEXT("Event Schemas"));

	TSharedRef<SWidget> EventSchemaHeaderContent = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(4, 0, 0, 0)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(EventSchemaCategory.GetDisplayName())
			.Font(IDetailLayoutBuilder::GetDetailFontBold())
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(4, 0, 0, 0)
		[
			EventSchemasHandle->CreateDefaultPropertyButtonWidgets()
		];

	EventSchemaCategory.HeaderContent(EventSchemaHeaderContent, /*bWholeRow*/true);
	EventSchemaCategory.AddCustomBuilder(EventSchemasBuilder);
}

} // UE::SceneState::Editor
