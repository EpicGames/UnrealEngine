// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateEventSchemaFieldNodeBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Framework/Notifications/NotificationManager.h"
#include "IPropertyUtilities.h"
#include "PropertyCustomizationHelpers.h"
#include "SPinTypeSelector.h"
#include "SceneStateEventEditorUtils.h"
#include "SceneStateEventSchema.h"
#include "Styling/AppStyle.h"
#include "UserDefinedStructure/UserDefinedStructEditorData.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SceneStateEventSchemaFieldNodeBuilder"

namespace UE::SceneState::Editor
{

namespace Private
{

FStructVariableDescription* FindStructVarDesc(UUserDefinedStruct* InStruct, const FGuid& InFieldId)
{
	if (!InStruct)
	{
		return nullptr;
	}
	return FStructureEditorUtils::GetVarDesc(InStruct).FindByPredicate(FStructureEditorUtils::FFindByGuidHelper<FStructVariableDescription>(InFieldId));
}

} // Private

FEventSchemaFieldNodeBuilder::FEventSchemaFieldNodeBuilder(const TSharedRef<IPropertyHandle>& InEventSchemaHandle, const FGuid& InFieldGuid)
	: EventSchemaHandleWeak(InEventSchemaHandle)
	, FieldId(InFieldGuid)
{
}

USceneStateEventSchemaObject* FEventSchemaFieldNodeBuilder::GetEventSchema() const
{
	TSharedPtr<IPropertyHandle> EventSchemaHandle = EventSchemaHandleWeak.Pin();
	if (!EventSchemaHandle.IsValid())
	{
		return nullptr;
	}

	UObject* EventSchema = nullptr;
	if (EventSchemaHandle->GetValue(EventSchema) == FPropertyAccess::Success)
	{
		return Cast<USceneStateEventSchemaObject>(EventSchema);
	}
	return nullptr;
}

void FEventSchemaFieldNodeBuilder::OnChildrenChanged()
{
	OnRegenerateChildren.ExecuteIfBound();
}

void FEventSchemaFieldNodeBuilder::OnSiblingsChanged()
{
	OnRegenerateSiblings.ExecuteIfBound();
}

FText FEventSchemaFieldNodeBuilder::GetFieldDisplayName() const
{
	USceneStateEventSchemaObject* EventSchema = GetEventSchema();
	if (EventSchema && EventSchema->Struct)
	{
		return FText::FromString(FStructureEditorUtils::GetVariableFriendlyName(EventSchema->Struct, FieldId));
	}
	return FText::GetEmpty();
}

void FEventSchemaFieldNodeBuilder::OnFieldNameCommitted(const FText& InNewText, ETextCommit::Type InTextCommit)
{
	USceneStateEventSchemaObject* EventSchema = GetEventSchema();
	if (EventSchema && EventSchema->Struct)
	{
		FStructureEditorUtils::RenameVariable(EventSchema->Struct, FieldId, InNewText.ToString());
	}
}

FEdGraphPinType FEventSchemaFieldNodeBuilder::OnGetPinInfo() const
{
	USceneStateEventSchemaObject* EventSchema = GetEventSchema();
	if (EventSchema && EventSchema->Struct)
	{
		if (const FStructVariableDescription* FieldDesc = Private::FindStructVarDesc(EventSchema->Struct, FieldId))
		{
			return FieldDesc->ToPinType();
		}
	}
	return FEdGraphPinType();
}

void FEventSchemaFieldNodeBuilder::PinInfoChanged(const FEdGraphPinType& InPinType)
{
	USceneStateEventSchemaObject* EventSchema = GetEventSchema();
	if (EventSchema && EventSchema->Struct)
	{
		if (FStructureEditorUtils::ChangeVariableType(EventSchema->Struct, FieldId, InPinType))
		{
			OnChildrenChanged();
		}
		else
		{
			FNotificationInfo NotificationInfo(LOCTEXT("VariableTypeChangeError", "Variable type change failed (the selected type may not be compatible with this struct). See log for details."));
			NotificationInfo.ExpireDuration = 5.0f;
			FSlateNotificationManager::Get().AddNotification(NotificationInfo);
		}
	}
}

void FEventSchemaFieldNodeBuilder::RemoveField()
{
	RemoveVariable(GetEventSchema(), FieldId);
	OnChildrenChanged();
	OnSiblingsChanged();
}

EVisibility FEventSchemaFieldNodeBuilder::GetErrorIconVisibility() const
{
	if (USceneStateEventSchemaObject* EventSchema = GetEventSchema())
	{
		FStructVariableDescription* const FieldDesc = Private::FindStructVarDesc(EventSchema->Struct, FieldId);
		if (FieldDesc && FieldDesc->bInvalidMember)
		{
			return EVisibility::Visible;
		}
	}
	return EVisibility::Collapsed;
}

void FEventSchemaFieldNodeBuilder::SetOnRebuildSiblings(FSimpleDelegate InOnRegenerateSiblings)
{
	OnRegenerateSiblings = InOnRegenerateSiblings;
}

void FEventSchemaFieldNodeBuilder::SetOnRebuildChildren(FSimpleDelegate InOnRegenerateChildren)
{
	OnRegenerateChildren = InOnRegenerateChildren;
}

void FEventSchemaFieldNodeBuilder::GenerateHeaderRowContent(FDetailWidgetRow& InNodeRow)
{
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

	// Todo: Drag Drop Handler to move variable
	InNodeRow
		.NameContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.Error"))
				.ToolTipText(LOCTEXT("MemberVariableErrorToolTip", "Member variable is invalid"))
				.Visibility(this, &FEventSchemaFieldNodeBuilder::GetErrorIconVisibility)
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1)
			.VAlign(VAlign_Center)
			[
				SNew(SEditableTextBox)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(this, &FEventSchemaFieldNodeBuilder::GetFieldDisplayName)
				.OnTextCommitted(this, &FEventSchemaFieldNodeBuilder::OnFieldNameCommitted)
			]
		]
		.ValueContent()
		.MaxDesiredWidth(200)
		.MinDesiredWidth(200)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(0, 0, 4, 0)
			[
				SNew(SPinTypeSelector, FGetPinTypeTree::CreateUObject(Schema, &UEdGraphSchema_K2::GetVariableTypeTree))
				.TargetPinType(this, &FEventSchemaFieldNodeBuilder::OnGetPinInfo)
				.OnPinTypeChanged(this, &FEventSchemaFieldNodeBuilder::PinInfoChanged)
				.Schema(GetDefault<UEdGraphSchema_K2>())
				.TypeTreeFilter(ETypeTreeFilter::None)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			[
				PropertyCustomizationHelpers::MakeEmptyButton(FSimpleDelegate::CreateSP(this, &FEventSchemaFieldNodeBuilder::RemoveField)
					, LOCTEXT("RemoveVariable", "Remove member variable"))
			]
		];
}

void FEventSchemaFieldNodeBuilder::GenerateChildContent(IDetailChildrenBuilder& InChildrenBuilder)
{
	// Extra settings could go here to adjust things like UI Slider Range, etc
}

bool FEventSchemaFieldNodeBuilder::RequiresTick() const
{
	return false;
}

FName FEventSchemaFieldNodeBuilder::GetName() const
{
	return FName(*FieldId.ToString());
}

bool FEventSchemaFieldNodeBuilder::InitiallyCollapsed() const
{
	return false;
}

} // UE::SceneState::Editor

#undef LOCTEXT_NAMESPACE
