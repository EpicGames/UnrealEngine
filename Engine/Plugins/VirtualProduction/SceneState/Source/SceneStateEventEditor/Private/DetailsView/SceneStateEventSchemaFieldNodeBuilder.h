// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "IDetailCustomNodeBuilder.h"
#include "Kismet2/StructureEditorUtils.h"
#include "Templates/SharedPointer.h"

class FText;
class IPropertyHandle;
class USceneStateEventSchemaObject;
struct EVisibility;
struct FGuid;

namespace ETextCommit
{
	enum Type : int;
}

namespace UE::SceneState::Editor
{

/** Builder for a single field in the user defined struct of an Event Schema */
class FEventSchemaFieldNodeBuilder : public IDetailCustomNodeBuilder, public TSharedFromThis<FEventSchemaFieldNodeBuilder>
{
public:
	explicit FEventSchemaFieldNodeBuilder(const TSharedRef<IPropertyHandle>& InEventSchemaHandle, const FGuid& InFieldGuid);

	/** Gets the Event Schema for this node builder, or  null if multiple or no event schemas are present*/
	USceneStateEventSchemaObject* GetEventSchema() const;

	/** Called when children layout has changed and needs to be refreshed */
	void OnChildrenChanged();

	/** Called when this changed the parent layout so itself and siblings need to be rebuilt */
	void OnSiblingsChanged();

	/** Retrieves the friendly name of the field */
	FText GetFieldDisplayName() const;

	/** Called when the friendly name for the field needs to be set*/
	void OnFieldNameCommitted(const FText& InNewText, ETextCommit::Type InTextCommit);

	/** Retrieves the current type of the field */
	FEdGraphPinType OnGetPinInfo() const;

	/** Called when the type of the field needs updating */
	void PinInfoChanged(const FEdGraphPinType& InPinType);

	/** Removes this field from the struct */
	void RemoveField();

	/** Returns visible whether this field is current invalid, collapsed otherwise */
	EVisibility GetErrorIconVisibility() const;

	/** Set callback to when siblings require rebuilding */
	void SetOnRebuildSiblings(FSimpleDelegate InOnRegenerateSiblings);

	//~ Begin IDetailCustomNodeBuilder
	virtual void SetOnRebuildChildren(FSimpleDelegate InOnRegenerateChildren) override;
	virtual void GenerateHeaderRowContent(FDetailWidgetRow& InNodeRow) override;
	virtual void GenerateChildContent(IDetailChildrenBuilder& InChildrenBuilder) override;
	virtual bool RequiresTick() const override;
	virtual FName GetName() const override;
	virtual bool InitiallyCollapsed() const override;
	//~ End IDetailCustomNodeBuilder

private:
	/** Handle to the Event schema property */
	TWeakPtr<IPropertyHandle> EventSchemaHandleWeak;

	/** Var Guid of the field */
	FGuid FieldId;

	/** Delegate to execute if parent layout has changed and requires siblings to be rebuilt */
	FSimpleDelegate OnRegenerateSiblings;

	/** Delegate to execute if this layout has changed and requires children to be rebuilt */
	FSimpleDelegate OnRegenerateChildren;
};

} // UE::SceneState::Editor
