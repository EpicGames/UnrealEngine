// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateEventSchemaHandleCustomization.h"
#include "DetailWidgetRow.h"
#include "Widgets/SSceneStateEventSchemaPicker.h"

namespace UE::SceneState::Editor
{

void FEventSchemaHandleCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	InHeaderRow
		.NameContent()
		[
			InStructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SEventSchemaPicker, InStructPropertyHandle)
		];
}

} // UE::SceneState::Editor
