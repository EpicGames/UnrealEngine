// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateEventSchemaNodeBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyUtilities.h"
#include "Kismet2/StructureEditorUtils.h"
#include "Misc/EnumerateRange.h"
#include "SceneStateEventEditorUtils.h"
#include "SceneStateEventSchema.h"
#include "SceneStateEventSchemaFieldNodeBuilder.h"
#include "ScopedTransaction.h"
#include "UserDefinedStructure/UserDefinedStructEditorData.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SceneStateEventSchemaNodeBuilder"

namespace UE::SceneState::Editor
{

FEventSchemaNodeBuilder::FEventSchemaNodeBuilder(const TSharedRef<IPropertyHandle>& InEventSchemaHandle)
	: EventSchemaHandle(InEventSchemaHandle)
{
	// if there's no current event schema, create one
	if (!GetEventSchema())
	{
		CreateEventSchema();
	}
}

TSharedRef<SWidget> FEventSchemaNodeBuilder::CreateEventSchemaNameWidget()
{
	return SNew(SBox)
		.MinDesiredWidth(200)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Fill)
		[
			SNew(SEditableTextBox)
			.Text(this, &FEventSchemaNodeBuilder::GetEventSchemaName)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Justification(ETextJustify::Left)
			.SelectAllTextWhenFocused(true)
			.ClearKeyboardFocusOnCommit(false)
			.MaximumLength(NAME_SIZE - 1)
			.OnTextCommitted(this, &FEventSchemaNodeBuilder::SetEventSchemaName)
			.SelectAllTextOnCommit(true)
		];
}

TSharedRef<SWidget> FEventSchemaNodeBuilder::CreateAddPropertyButton()
{
	return SNew(SBox)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		[
			SNew(SButton)
			.ToolTipText(LOCTEXT("AddPropertyTooltip", "Add new property"))
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.OnClicked(this, &FEventSchemaNodeBuilder::OnAddPropertyClicked)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.PlusCircle"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.Padding(4, 0, 0, 0)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("AddPropertyLabel", "Add Property"))
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
			]
		];
}

void FEventSchemaNodeBuilder::CreateEventSchema()
{
	FProperty* EventSchemaHandleProperty = EventSchemaHandle->GetProperty();
	if (!ensure(EventSchemaHandleProperty))
	{
		return;
	}

	TArray<UObject*> OuterObjects;
	EventSchemaHandle->GetOuterObjects(OuterObjects);

	// Currently only 1 outer object is expected (the Event Schema Collection)
	if (OuterObjects.Num() != 1)
	{
		return;
	}

	UObject* Outer = OuterObjects[0];
	if (!ensure(Outer))
	{
		return;
	}

	EObjectFlags MaskedOuterFlags = Outer->GetMaskedFlags(RF_PropagateToSubObjects);

	// The struct needs to be visible externally
	MaskedOuterFlags |= RF_Public;

	if (Outer->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
	{
		MaskedOuterFlags |= RF_ArchetypeObject;
	}

	// Do not create an additional transaction
	USceneStateEventSchemaObject* EventSchema = NewObject<USceneStateEventSchemaObject>(Outer, NAME_None, MaskedOuterFlags);
	EventSchemaHandle->SetValueFromFormattedString(EventSchema->GetPathName(), EPropertyValueSetFlags::NotTransactable);
}

USceneStateEventSchemaObject* FEventSchemaNodeBuilder::GetEventSchema() const
{
	UObject* EventSchema = nullptr;
	if (EventSchemaHandle->GetValue(EventSchema) == FPropertyAccess::Success)
	{
		return Cast<USceneStateEventSchemaObject>(EventSchema);
	}
	return nullptr;
}

FText FEventSchemaNodeBuilder::GetEventSchemaName() const
{
	if (USceneStateEventSchemaObject* EventSchema = GetEventSchema())
	{
		return FText::FromName(EventSchema->Name);
	}
	return FText::GetEmpty();
}

void FEventSchemaNodeBuilder::SetEventSchemaName(const FText& InText, ETextCommit::Type InCommitType)
{
	USceneStateEventSchemaObject* EventSchema = GetEventSchema();
	if (!EventSchema)
	{
		return;
	}

	FProperty* NameProperty = USceneStateEventSchemaObject::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(USceneStateEventSchemaObject, Name));

	FScopedTransaction Transaction(LOCTEXT("SetEventSchemaName", "Set Event Schema Name"));
	EventSchema->PreEditChange(NameProperty);

	EventSchema->Name = *InText.ToString();

	FPropertyChangedEvent ChangedEvent(NameProperty, EPropertyChangeType::ValueSet, { EventSchema });
	EventSchema->PostEditChangeProperty(ChangedEvent);
}

FReply FEventSchemaNodeBuilder::OnAddPropertyClicked()
{
	USceneStateEventSchemaObject* EventSchema = GetEventSchema();
	if (!EventSchema)
	{
		return FReply::Handled();
	}

	FProperty* StructProperty = USceneStateEventSchemaObject::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(USceneStateEventSchemaObject, Struct));

	FScopedTransaction Transaction(LOCTEXT("AddPropertyTransaction", "Add Property"));
	EventSchema->PreEditChange(StructProperty);

	CreateVariable(EventSchema);

	FPropertyChangedEvent ChangedEvent(StructProperty, EPropertyChangeType::ValueSet, { EventSchema });
	EventSchema->PostEditChangeProperty(ChangedEvent);

	OnChildrenChanged();
	return FReply::Handled();
}

void FEventSchemaNodeBuilder::OnChildrenChanged()
{
	OnRegenerateChildren.ExecuteIfBound();
}

FName FEventSchemaNodeBuilder::GetName() const
{
	return TEXT("FEventSchemaNodeBuilder");
}

void FEventSchemaNodeBuilder::GenerateHeaderRowContent(FDetailWidgetRow& InNodeRow)
{
	InNodeRow
		.NameContent()
		[
			CreateEventSchemaNameWidget()
		]
		.ValueContent()
		[
			CreateAddPropertyButton()
		]
		.ExtensionContent()
		[
			EventSchemaHandle->CreateDefaultPropertyButtonWidgets()
		];
}

void FEventSchemaNodeBuilder::GenerateChildContent(IDetailChildrenBuilder& InChildrenBuilder)
{
	USceneStateEventSchemaObject* EventSchema = GetEventSchema();
	if (!EventSchema || !EventSchema->Struct)
	{
		return;
	}

	for (const FStructVariableDescription& VarDesc : FStructureEditorUtils::GetVarDesc(EventSchema->Struct))
	{
		TSharedRef<FEventSchemaFieldNodeBuilder> FieldBuilder = MakeShared<FEventSchemaFieldNodeBuilder>(EventSchemaHandle, VarDesc.VarGuid);
		FieldBuilder->SetOnRebuildSiblings(FSimpleDelegate::CreateSP(this, &FEventSchemaNodeBuilder::OnChildrenChanged));
		InChildrenBuilder.AddCustomBuilder(FieldBuilder);
	}
}

void FEventSchemaNodeBuilder::SetOnRebuildChildren(FSimpleDelegate InOnRegenerateChildren)
{
	OnRegenerateChildren = InOnRegenerateChildren;
}

TSharedPtr<IPropertyHandle> FEventSchemaNodeBuilder::GetPropertyHandle() const
{
	return EventSchemaHandle;
}

} // UE::SceneState::Editor

#undef LOCTEXT_NAMESPACE
