// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomNodeBuilder.h"
#include "Templates/SharedPointer.h"

class FReply;
class FText;
class IPropertyHandle;
class SWidget;
class USceneStateEventSchemaObject;

namespace ETextCommit
{
	enum Type : int;
}

namespace UE::SceneState::Editor
{

/** Builder for the Event Schema element entry in the Collection Event Schemas array */
class FEventSchemaNodeBuilder : public IDetailCustomNodeBuilder, public TSharedFromThis<FEventSchemaNodeBuilder>
{
public:
	explicit FEventSchemaNodeBuilder(const TSharedRef<IPropertyHandle>& InEventSchemaHandle);

private:
	/** Generates editable text box to edit the Event Schema name */
	TSharedRef<SWidget> CreateEventSchemaNameWidget();

	/** Generates the button to add properties to the Event Schema struct */
	TSharedRef<SWidget> CreateAddPropertyButton();

	/** Creates the event schema for the given event schema handle */
	void CreateEventSchema();

	/** Gets the current event schema value from the handle */
	USceneStateEventSchemaObject* GetEventSchema() const;

	/** Gets the name of the event schema */
	FText GetEventSchemaName() const;

	/** Sets the name for the event schema */
	void SetEventSchemaName(const FText& InText, ETextCommit::Type InCommitType);

	/** Called to add a new property to the struct */
	FReply OnAddPropertyClicked();

	/** Called when the struct layout has changed */
	void OnChildrenChanged();

	//~ Begin IDetailCustomNodeBuilder
	virtual FName GetName() const override;
	virtual void GenerateHeaderRowContent(FDetailWidgetRow& InNodeRow) override;
	virtual void GenerateChildContent(IDetailChildrenBuilder& InChildrenBuilder) override;
	virtual void SetOnRebuildChildren(FSimpleDelegate InOnRegenerateChildren) override;
	virtual TSharedPtr<IPropertyHandle> GetPropertyHandle() const override;
	//~ End IDetailCustomNodeBuilder

	/** Handle to the Event schema property */
	TSharedRef<IPropertyHandle> EventSchemaHandle;

	/** Delegate to execute if the struct layout has changed and requires children to be rebuilt */
	FSimpleDelegate OnRegenerateChildren;
};

} // UE::SceneState::Editor
