// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateEventTemplateCustomization.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "InstancedStructDetails.h"
#include "PropertyHandle.h"
#include "SceneStateEventTemplate.h"
#include "StructUtils/UserDefinedStruct.h"

namespace UE::SceneState::Editor
{

void FEventTemplateCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	EventTemplateHandle = InStructPropertyHandle;

	InStructPropertyHandle->SetOnChildPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FEventTemplateCustomization::OnEventSchemaHandleChanged));

	TSharedRef<IPropertyHandle> EventSchemaHandle = InStructPropertyHandle->GetChildHandle(FSceneStateEventTemplate::GetEventSchemaHandlePropertyName()).ToSharedRef();
	EventSchemaHandle->MarkHiddenByCustomization();

	InHeaderRow
		.NameContent()
		[
			InStructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			EventSchemaHandle->CreatePropertyValueWidgetWithCustomization(nullptr)
		]
		.ShouldAutoExpand(/*bForceExpansion*/true);
}

void FEventTemplateCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	TSharedRef<IPropertyHandle> EventDataHandle = InStructPropertyHandle->GetChildHandle(FSceneStateEventTemplate::GetEventDataPropertyName()).ToSharedRef();
	EventDataHandle->MarkHiddenByCustomization();

	InChildBuilder.AddCustomBuilder(MakeShared<FInstancedStructDataDetails>(EventDataHandle));
}

void FEventTemplateCustomization::OnEventSchemaHandleChanged()
{
	EventTemplateHandle->EnumerateRawData(
		[](void* InRawData, const int32 InDataIndex, const int32 InDataNum)->bool
		{
			if (InRawData)
			{
				FSceneStateEventTemplate& EventTemplate = *static_cast<FSceneStateEventTemplate*>(InRawData);
				EventTemplate.SyncEventData();
			}
			return true; // continue
		});
}

} // UE::SceneState::Editor
