// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSceneStateEventSchemaPicker.h"
#include "DetailLayoutBuilder.h"
#include "GuidStructCustomization.h"
#include "PropertyCustomizationHelpers.h"
#include "SceneStateEventSchema.h"
#include "SceneStateEventSchemaCollection.h"
#include "SceneStateEventSchemaHandle.h"
#include "ScopedTransaction.h"
#include "StructUtils/UserDefinedStruct.h"
#include "Styling/ToolBarStyle.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SSceneStateEventSchemaPicker"

namespace UE::SceneState::Editor
{

namespace Private
{

USceneStateEventSchemaCollection* GetCollection(const USceneStateEventSchemaObject* InObject)
{
	if (InObject)
	{
		return Cast<USceneStateEventSchemaCollection>(InObject->GetOuter());
	}
	return nullptr;
}

} // Private
	
void SEventSchemaPicker::Construct(const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InStructHandle)
{
	EventSchemaHandle = InStructHandle->GetChildHandle(FSceneStateEventSchemaHandle::GetEventSchemaPropertyName());
	EventStructHandle = InStructHandle->GetChildHandle(FSceneStateEventSchemaHandle::GetEventStructPropertyName());

	check(EventSchemaHandle && EventStructHandle);

	EventSchemaHandle->MarkHiddenByCustomization();
	EventStructHandle->MarkHiddenByCustomization();

	InStructHandle->SetOnPropertyResetToDefault(FSimpleDelegate::CreateSP(this, &SEventSchemaPicker::RefreshSchemaOptions, /*bComboBoxOpening*/false));

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SAssignNew(CollectionPicker, SObjectPropertyEntryBox)
			.AllowedClass(USceneStateEventSchemaCollection::StaticClass())
			.ObjectPath(this, &SEventSchemaPicker::GetEventSchemaCollectionPath)
			.OnObjectChanged(this, &SEventSchemaPicker::OnEventSchemaCollectionChanged)
			.DisplayThumbnail(false)
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		[
			SAssignNew(SchemaPicker, SComboBox<TSharedRef<FEventSchemaItem>>)
			.OptionsSource(&SchemaItems)
			.InitiallySelectedItem(SelectedSchemaItem)
			.OnGenerateWidget(this, &SEventSchemaPicker::CreateSchemaItemWidget)
			.OnComboBoxOpening(this, &SEventSchemaPicker::RefreshSchemaOptions, /*bComboBoxOpening*/true)
			.OnSelectionChanged(this, &SEventSchemaPicker::OnSchemaItemSelectionChanged)
			[
				SNew(STextBlock)
				.Text(this, &SEventSchemaPicker::GetCurrentEventSchemaName)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		]
	];

	RefreshSchemaOptions(/*bComboBoxOpening*/false);
}

void SEventSchemaPicker::Tick(const FGeometry& InAllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(InAllottedGeometry, InCurrentTime, InDeltaTime);

	if (bRequestOpenCollectionPicker)
	{
		CollectionPicker->OpenEntryBox();
		bRequestOpenCollectionPicker = false;
	}

	if (bRequestOpenSchemaPicker)
	{
		SchemaPicker->SetIsOpen(/*InIsOpen*/true);
		bRequestOpenSchemaPicker = false;
	}
}

FText SEventSchemaPicker::GetCurrentEventSchemaName() const
{
	return SelectedSchemaItem.IsValid()
		? FText::FromName(SelectedSchemaItem->Name)
		: FText::GetEmpty();
}

USceneStateEventSchemaObject* SEventSchemaPicker::GetEventSchema() const
{
	UObject* EventSchema = nullptr;
	if (EventSchemaHandle->GetValue(EventSchema) == FPropertyAccess::Success && EventSchema)
	{
		return Cast<USceneStateEventSchemaObject>(EventSchema);
	}
	return nullptr;
}

FString SEventSchemaPicker::GetEventSchemaCollectionPath() const
{
	return SelectedSchemaCollectionPath;
}

void SEventSchemaPicker::OnEventSchemaCollectionChanged(const FAssetData& InEventSchemaCollection)
{
	// Ok to load event schema at this point (if not already), as it will be used to build the items of the schemas within the collection
	const USceneStateEventSchemaCollection* EventSchemaCollection = Cast<USceneStateEventSchemaCollection>(InEventSchemaCollection.GetAsset());

	if (SelectedSchemaCollection != EventSchemaCollection)
	{
		RebuildSchemaItems(EventSchemaCollection, FGuid());
	}

	// Open the schema picker. Most likely the next user's action
	bRequestOpenSchemaPicker = true;
}

TSharedRef<SWidget> SEventSchemaPicker::CreateSchemaItemWidget(TSharedRef<FEventSchemaItem> InSchemaItem)
{
	return SNew(STextBlock)
		.Text(FText::FromName(InSchemaItem->Name))
		.Font(IDetailLayoutBuilder::GetDetailFont());
}

void SEventSchemaPicker::RefreshSchemaOptions(bool bIsComboBoxOpening)
{
	const USceneStateEventSchemaObject* const CurrentEventSchema = GetEventSchema();

	SchemaItems.Reset();
	SelectedSchemaItem.Reset();
	SelectedSchemaCollectionPath.Reset();
	SelectedSchemaCollection = FObjectKey();

	if (const USceneStateEventSchemaCollection* EventSchemaCollection = Private::GetCollection(CurrentEventSchema))
	{
		check(CurrentEventSchema);
		RebuildSchemaItems(EventSchemaCollection, CurrentEventSchema->Id);
	}
	// If combo box is opening, and no valid event schema collection is found, open this collection picker first
	else if (bIsComboBoxOpening)
	{
		bRequestOpenCollectionPicker = true;
	}

	SchemaPicker->ClearSelection();
	SchemaPicker->RefreshOptions();
}

void SEventSchemaPicker::RebuildSchemaItems(const USceneStateEventSchemaCollection* InCollection, const FGuid& InCurrentId)
{
	SchemaItems.Reset();
	SelectedSchemaItem.Reset();
	SelectedSchemaCollectionPath.Reset();
	SelectedSchemaCollection = InCollection;

	if (!InCollection)
	{
		return;
	}

	SelectedSchemaCollectionPath = InCollection->GetPathName();

	TConstArrayView<USceneStateEventSchemaObject*> EventSchemas = InCollection->GetEventSchemas();
	SchemaItems.Reserve(EventSchemas.Num());

	for (USceneStateEventSchemaObject* EventSchema : EventSchemas)
	{
		if (!EventSchema)
		{
			continue;
		}

		TSharedRef<FEventSchemaItem>& SchemaItem = SchemaItems.Emplace_GetRef(MakeShared<FEventSchemaItem>());
		SchemaItem->Name = EventSchema->Name;
		SchemaItem->EventSchemaWeak = EventSchema;
		SchemaItem->EventStructWeak = EventSchema->Struct;

		if (InCurrentId == EventSchema->Id)
		{
			SelectedSchemaItem = SchemaItem;
		}
	}
}

void SEventSchemaPicker::OnSchemaItemSelectionChanged(TSharedPtr<FEventSchemaItem> InSchemaItem, ESelectInfo::Type InSelectionType)
{
	if (InSchemaItem == SelectedSchemaItem || InSelectionType == ESelectInfo::Type::Direct)
	{
		return;
	}

	SelectedSchemaItem = InSchemaItem;

	USceneStateEventSchemaObject* EventSchema = nullptr;
	UUserDefinedStruct* EventStruct = nullptr;

	if (SelectedSchemaItem.IsValid())
	{
		EventSchema = SelectedSchemaItem->EventSchemaWeak.Get();
		EventStruct = SelectedSchemaItem->EventStructWeak.Get();
	}

	FScopedTransaction Transaction(LOCTEXT("SetEventSchema", "Set EventSchema"));
	EventSchemaHandle->SetValue(EventSchema);
	EventStructHandle->SetValue(EventStruct);
}

} // UE::SceneState::Editor

#undef LOCTEXT_NAMESPACE
