// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectKey.h"
#include "Widgets/SCompoundWidget.h"

class IPropertyHandle;
class SObjectPropertyEntryBox;
class USceneStateEventSchemaCollection;
class USceneStateEventSchemaObject;
class UUserDefinedStruct;
template<typename OptionType> class SComboBox;

namespace UE::SceneState::Editor
{

/** Schema list item struct */
struct FEventSchemaItem
{
	FName Name;
	TWeakObjectPtr<USceneStateEventSchemaObject> EventSchemaWeak;
	TWeakObjectPtr<UUserDefinedStruct> EventStructWeak;
};

/** Widget to pick both an event schema collection and a schema from the available schemas within the collection */
class SEventSchemaPicker : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SEventSchemaPicker) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InStructHandle);

	//~ Begin SWidget
	virtual void Tick(const FGeometry& InAllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	//~ End SWidget

private:
	/** Gets the selected event schema's name */
	FText GetCurrentEventSchemaName() const;

	/** Gets the current event schema in the handle */
	USceneStateEventSchemaObject* GetEventSchema() const;

	/** Gets the path to the current selected event schema collection */
	FString GetEventSchemaCollectionPath() const;

	/** Called when an event schema collection has been selected */
	void OnEventSchemaCollectionChanged(const FAssetData& InEventSchemaCollection);

	/** Creates the schema option entry row widget */
	TSharedRef<SWidget> CreateSchemaItemWidget(TSharedRef<FEventSchemaItem> InSchemaItem);

	/** Discovers / builds the list of schema items based on the current event schema */
	void RefreshSchemaOptions(bool bIsComboBoxOpening);

	/** Builds the list of schema items based on the given schema collection */
	void RebuildSchemaItems(const USceneStateEventSchemaCollection* InCollection, const FGuid& InCurrentId);

	/** Called when the schema item has been selected */
	void OnSchemaItemSelectionChanged(TSharedPtr<FEventSchemaItem> InSchemaItem, ESelectInfo::Type InSelectionType);

	/** Handle to the Event Schema soft ref property */
	TSharedPtr<IPropertyHandle> EventSchemaHandle;

	/** Handle to the soft ref of the Event Schema struct */
	TSharedPtr<IPropertyHandle> EventStructHandle;

	/** Current event schema collection */
	FObjectKey SelectedSchemaCollection;

	/** Path to the current event schema collection */
	FString SelectedSchemaCollectionPath;

	/** Object picker widget selecting the Event Schema Collection object */
	TSharedPtr<SObjectPropertyEntryBox> CollectionPicker;

	/** Combo box listing the schemas in the current collection, if any  */
	TSharedPtr<SComboBox<TSharedRef<FEventSchemaItem>>> SchemaPicker;

	/** Currently selected event schema */
	TSharedPtr<FEventSchemaItem> SelectedSchemaItem;

	/** The available id options within the collection */
	TArray<TSharedRef<FEventSchemaItem>> SchemaItems;

	/** Flag to request open the collection asset picker in next tick */
	bool bRequestOpenCollectionPicker = false;

	/** Flag to request open the Schema picker in next tick */
	bool bRequestOpenSchemaPicker = false;
};

} // UE::SceneState::Editor
