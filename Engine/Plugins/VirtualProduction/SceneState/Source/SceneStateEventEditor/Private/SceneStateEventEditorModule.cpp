// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateEventEditorModule.h"
#include "DetailsView/SceneStateEventHandlerCustomization.h"
#include "DetailsView/SceneStateEventSchemaCollectionCustomization.h"
#include "DetailsView/SceneStateEventSchemaHandleCustomization.h"
#include "DetailsView/SceneStateEventTemplateCustomization.h"
#include "EdGraphUtilities.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SceneStateEventHandler.h"
#include "SceneStateEventSchema.h"
#include "SceneStateEventSchemaCollection.h"
#include "SceneStateEventTemplate.h"

IMPLEMENT_MODULE(UE::SceneState::Editor::FEventEditorModule, SceneStateEventEditor)

namespace UE::SceneState::Editor
{

void FEventEditorModule::StartupModule()
{
	RegisterDetailCustomizations();
}

void FEventEditorModule::ShutdownModule()
{
	UnregisterDetailCustomizations();
}

void FEventEditorModule::RegisterDetailCustomizations()
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	PropertyEditorModule.RegisterCustomPropertyTypeLayout(CustomizedTypes.Add_GetRef(FSceneStateEventSchemaHandle::StaticStruct()->GetFName())
		, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FEventSchemaHandleCustomization::MakeInstance));

	PropertyEditorModule.RegisterCustomPropertyTypeLayout(CustomizedTypes.Add_GetRef(FSceneStateEventHandler::StaticStruct()->GetFName())
		, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FEventHandlerCustomization::MakeInstance));

	PropertyEditorModule.RegisterCustomPropertyTypeLayout(CustomizedTypes.Add_GetRef(FSceneStateEventTemplate::StaticStruct()->GetFName())
		, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FEventTemplateCustomization::MakeInstance));

	PropertyEditorModule.RegisterCustomClassLayout(CustomizedClasses.Add_GetRef(USceneStateEventSchemaCollection::StaticClass()->GetFName())
		, FOnGetDetailCustomizationInstance::CreateStatic(&FEventSchemaCollectionCustomization::MakeInstance));
}

void FEventEditorModule::UnregisterDetailCustomizations()
{
	if (FPropertyEditorModule* PropertyEditorModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
	{
		for (FName CustomizedType : CustomizedTypes)
		{
			PropertyEditorModule->UnregisterCustomPropertyTypeLayout(CustomizedType);
		}
		CustomizedTypes.Reset();

		for (FName CustomizedClass : CustomizedClasses)
		{
			PropertyEditorModule->UnregisterCustomClassLayout(CustomizedClass);
		}
		CustomizedClasses.Reset();
	}
}

} // UE::SceneState::Editor
