// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

struct FGraphPanelPinFactory;

namespace UE::SceneState::Editor
{

class FEventEditorModule : public IModuleInterface
{
	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface

	void RegisterDetailCustomizations();
	void UnregisterDetailCustomizations();

	TArray<FName> CustomizedTypes;
	TArray<FName> CustomizedClasses;
};

} // UE::SceneState::Editor