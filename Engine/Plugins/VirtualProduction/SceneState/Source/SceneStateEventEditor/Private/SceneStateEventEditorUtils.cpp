// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateEventEditorUtils.h"
#include "EdGraphSchema_K2.h"
#include "Kismet2/StructureEditorUtils.h"
#include "SceneStateEventSchema.h"
#include "ScopedTransaction.h"
#include "UObject/Package.h"
#include "UserDefinedStructure/UserDefinedStructEditorData.h"

#define LOCTEXT_NAMESPACE "SceneStateEventEditorUtils"

namespace UE::SceneState::Editor
{

namespace Private
{

UUserDefinedStruct* CreateUserDefinedStruct(UObject* InOuter)
{
	UUserDefinedStruct* Struct = NewObject<UUserDefinedStruct>(InOuter, NAME_None, RF_Transactional | RF_Public);

	UUserDefinedStructEditorData* EditorData = NewObject<UUserDefinedStructEditorData>(Struct, NAME_None, RF_Transactional);
	Struct->EditorData = EditorData;
	EditorData->MetaData.Add(FBlueprintTags::BlueprintType, TEXT("true"));

	Struct->Guid = FGuid::NewGuid();
	Struct->Bind();
	Struct->StaticLink(true);

	Struct->Status = UDSS_Error;
	return Struct;
}

void DiscardObject(UObject* InObject)
{
	if (InObject)
	{
		UObject* NewOuter = GetTransientPackage();
		FName UniqueName = MakeUniqueObjectName(NewOuter, InObject->GetClass(), *(TEXT("TRASH_") + InObject->GetName()));
		InObject->Rename(*UniqueName.ToString(), NewOuter, REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
		InObject->MarkAsGarbage();
	}
}

} // Private

bool CreateVariable(USceneStateEventSchemaObject* InEventSchema)
{
	if (!InEventSchema)
	{
		return false;
	}

	if (!InEventSchema->Struct)
	{
		InEventSchema->Struct = Private::CreateUserDefinedStruct(InEventSchema);
	}
	check(InEventSchema->Struct);

	const FEdGraphPinType VariableType(UEdGraphSchema_K2::PC_Boolean, NAME_None, nullptr, EPinContainerType::None, false, FEdGraphTerminalType());
	return FStructureEditorUtils::AddVariable(InEventSchema->Struct, VariableType);
}

void RemoveVariable(USceneStateEventSchemaObject* InEventSchema, const FGuid& InFieldId)
{
	if (!InEventSchema || !InEventSchema->Struct)
	{
		return;
	}

	const TArray<FStructVariableDescription>& VariableDescriptions = FStructureEditorUtils::GetVarDesc(InEventSchema->Struct);
	if (VariableDescriptions.Num() == 1)
	{
		if (VariableDescriptions[0].VarGuid == InFieldId)
		{
			const FScopedTransaction Transaction(LOCTEXT("RemoveVariable", "Remove Variable"));
			FStructureEditorUtils::ModifyStructData(InEventSchema->Struct);

			Private::DiscardObject(InEventSchema->Struct->EditorData);
			Private::DiscardObject(InEventSchema->Struct);

			InEventSchema->Struct = nullptr;
		}
	}
	else
	{
		FStructureEditorUtils::RemoveVariable(InEventSchema->Struct, InFieldId);
	}
}

} // UE::SceneState::Editor

#undef LOCTEXT_NAMESPACE
