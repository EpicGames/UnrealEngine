// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

class USceneStateEventSchemaObject;
struct FGuid;

namespace UE::SceneState::Editor
{
	/**
	 * Creates a new boolean variable within the Event schema's struct
	 * If the current event schema struct is null, it creates a new struct and then creates the variable
	 */
	bool CreateVariable(USceneStateEventSchemaObject* InEventSchema);

	/**
	 * Removes the variable matching the given field id in the event schema's struct
	 * If this is the last variable to remove, the struct will be removed (user defined structs cannot be empty)
	 */
	void RemoveVariable(USceneStateEventSchemaObject* InEventSchema, const FGuid& InFieldId);

} // UE::SceneState::Editor