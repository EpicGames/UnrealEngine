// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateEventSchemaCollectionFactory.h"
#include "AssetToolsModule.h"
#include "IAssetTools.h"
#include "SceneStateEventSchemaCollection.h"

USceneStateEventSchemaCollectionFactory::USceneStateEventSchemaCollectionFactory()
{
	SupportedClass = USceneStateEventSchemaCollection::StaticClass();
	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

FText USceneStateEventSchemaCollectionFactory::GetDisplayName() const
{
	return SupportedClass ? SupportedClass->GetDisplayNameText() : Super::GetDisplayName();
}

FString USceneStateEventSchemaCollectionFactory::GetDefaultNewAssetName() const
{
	// Short name removing "Motion Design" and "Scene State" prefix for new assets
	return TEXT("NewEventSchemaCollection");
}

uint32 USceneStateEventSchemaCollectionFactory::GetMenuCategories() const
{
	IAssetTools& AssetTools = FAssetToolsModule::GetModule().Get();
	return AssetTools.FindAdvancedAssetCategory("MotionDesignCategory");
}

UObject* USceneStateEventSchemaCollectionFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, FFeedbackContext* InWarn)
{
	if (!ensure(SupportedClass == InClass))
	{
		return nullptr;
	}

	return NewObject<USceneStateEventSchemaCollection>(InParent, InName, InFlags);
}
