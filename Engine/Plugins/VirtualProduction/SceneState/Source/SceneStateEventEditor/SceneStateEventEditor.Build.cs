// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SceneStateEventEditor : ModuleRules
{
    public SceneStateEventEditor(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "AssetDefinition",
                "AssetTools",
                "BlueprintGraph",
                "Core",
                "CoreUObject",
                "DetailCustomizations",
                "Engine",
                "GraphEditor",
                "InputCore",
                "KismetWidgets",
                "PropertyEditor",
                "SceneStateEvent",
                "Slate",
                "SlateCore",
                "StructUtilsEditor",
                "UnrealEd",
            }
        );
    }
}