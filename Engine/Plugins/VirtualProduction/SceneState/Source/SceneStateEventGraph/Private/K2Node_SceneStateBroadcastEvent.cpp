// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_SceneStateBroadcastEvent.h"
#include "K2Node_CallFunction.h"
#include "K2Node_MakeStruct.h"
#include "Kismet/BlueprintInstancedStructLibrary.h"
#include "KismetCompiler.h"
#include "SceneStateEventLibrary.h"
#include "SceneStateEventSchema.h"
#include "StructUtils/UserDefinedStruct.h"

#define LOCTEXT_NAMESPACE "K2Node_SceneStateBroadcastEvent"

UK2Node_SceneStateBroadcastEvent::UK2Node_SceneStateBroadcastEvent()
{
	EventDataPinDirection = EGPD_Input;
}

void UK2Node_SceneStateBroadcastEvent::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();

	// Add Execution Pins
	CreatePin(EGPD_Input
		, UEdGraphSchema_K2::PC_Exec
		, UEdGraphSchema_K2::PN_Execute);

	CreatePin(EGPD_Output
		, UEdGraphSchema_K2::PC_Exec
		, UEdGraphSchema_K2::PN_Then);

	// Add World Context object pin
	CreatePin(EGPD_Input
		, UEdGraphSchema_K2::PC_Object
		, UObject::StaticClass()
		, UK2Node_SceneStateEventBase::PN_WorldContextObject);
}

FText UK2Node_SceneStateBroadcastEvent::GetNodeTitle(ENodeTitleType::Type InTitleType) const
{
	if (InTitleType == ENodeTitleType::MenuTitle)
	{
		return LOCTEXT("NodeMenuTitle", "Broadcast Event");
	}
	return FText::Format(LOCTEXT("NodeTitle", "Broadcast Event: {0}"), GetSchemaDisplayNameText());
}

void UK2Node_SceneStateBroadcastEvent::ExpandNode(FKismetCompilerContext& InCompilerContext, UEdGraph* InSourceGraph)
{
	Super::ExpandNode(InCompilerContext, InSourceGraph);

	USceneStateEventSchemaObject* const EventSchema = EventSchemaHandle.GetEventSchema();
	if (!EventSchema)
	{
		InCompilerContext.MessageLog.Error(*LOCTEXT("EventSchemaError", "ICE: No Event Schema specified @@").ToString(), this);
		BreakAllNodeLinks();
		return;
	}

	const UEdGraphSchema_K2* Schema = InCompilerContext.GetSchema();
	check(Schema);

	// Create 'Broadcast Event' function call node
	UK2Node_CallFunction* const BroadcastEventNode = InCompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, InSourceGraph);
	check(BroadcastEventNode);
	BroadcastEventNode->FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(USceneStateEventLibrary, BroadcastEvent), USceneStateEventLibrary::StaticClass());
	BroadcastEventNode->AllocateDefaultPins();

	// Set the Event Schema Handle
	if (UEdGraphPin* EventSchemaHandlePin = BroadcastEventNode->FindPin(TEXT("InEventSchemaHandle")))
	{
		EventSchemaHandlePin->DefaultValue = GetSchemaHandleStringValue();
	}

	// Move the World context object pin
	if (!ConnectPinsToIntermediate(InCompilerContext, BroadcastEventNode, UK2Node_SceneStateEventBase::PN_WorldContextObject, TEXT("WorldContextObject")))
	{
		InCompilerContext.MessageLog.Error(*LOCTEXT("EventScopeConnectError", "ICE: Error connecting World Context Object Pin @@").ToString()
			, this);
	}
	
	FNodeExpansionContext Context = {
		.CompilerContext = InCompilerContext,
		.SourceGraph = InSourceGraph,
		.EventDataPin = BroadcastEventNode->FindPin(TEXT("InEventData")) };

	SpawnEventDataNodes(Context);
	ChainNode(Context, BroadcastEventNode);
	FinishChain(Context);

	BroadcastEventNode->ReconstructNode();
	BreakAllNodeLinks();
}

#undef LOCTEXT_NAMESPACE
