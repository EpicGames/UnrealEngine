// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "K2Node_SceneStateEventBase.h"
#include "K2Node_SceneStateBroadcastEvent.generated.h"

UCLASS()
class UK2Node_SceneStateBroadcastEvent : public UK2Node_SceneStateEventBase
{
	GENERATED_BODY()

public:
	UK2Node_SceneStateBroadcastEvent();

protected:
	//~ Begin UEdGraphNode
	virtual void AllocateDefaultPins() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type InTitleType) const override;
	//~ End UEdGraphNode

	//~ Begin UK2Node
	virtual void ExpandNode(FKismetCompilerContext& InCompilerContext, UEdGraph* InSourceGraph) override;
	//~ End UK2Node
};
