// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_SceneStateEventBase.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "K2Node_CallFunction.h"
#include "K2Node_MakeStruct.h"
#include "Kismet/BlueprintInstancedStructLibrary.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "KismetCompiler.h"
#include "SceneStateEventSchema.h"
#include "StructUtils/StructView.h"
#include "StructUtils/UserDefinedStruct.h"

#define LOCTEXT_NAMESPACE "K2Node_SceneStateEventBase"

const FLazyName UK2Node_SceneStateEventBase::PN_EventStream = TEXT("EventStream");
const FLazyName UK2Node_SceneStateEventBase::PN_WorldContextObject = TEXT("WorldContextObject");

FText UK2Node_SceneStateEventBase::GetSchemaDisplayNameText() const
{
	if (const USceneStateEventSchemaObject* EventSchema = EventSchemaHandle.GetEventSchema())
	{
		return FText::FromName(EventSchema->Name);
	}
	return LOCTEXT("InvalidSchemaText", "Invalid");
}

FString UK2Node_SceneStateEventBase::GetSchemaHandleStringValue() const
{
	FProperty* EventSchemaHandleProperty = FindFProperty<FProperty>(GetClass(), GET_MEMBER_NAME_CHECKED(UK2Node_SceneStateEventBase, EventSchemaHandle));
	check(EventSchemaHandleProperty);

	FString StringValue;
	EventSchemaHandleProperty->ExportText_Direct(StringValue, &EventSchemaHandle, &EventSchemaHandle, nullptr, PPF_None);
	return StringValue;
}

void UK2Node_SceneStateEventBase::OnEventSchemaChanged()
{
	if (!bHasEventData)
	{
		return;
	}

	// Remove all pins related to event data
	TArray<UEdGraphPin*> OldPins = Pins;
	TArray<UEdGraphPin*> OldClassPins;

	for (UEdGraphPin* OldPin : OldPins)
	{
		if (IsEventDataPin(OldPin))
		{
			Pins.Remove(OldPin);
			OldClassPins.Add(OldPin);
		}
	}

	CreateEventDataPins(Pins);

	RestoreSplitPins(OldPins);

	// Rewire the old pins to the new pins so connections are maintained if possible
	RewireOldPinsToNewPins(OldClassPins, Pins, nullptr);

	// Refresh the UI for the graph so the pin changes show up
	GetGraph()->NotifyGraphChanged();
	FBlueprintEditorUtils::MarkBlueprintAsModified(GetBlueprint());
}

bool UK2Node_SceneStateEventBase::IsEventDataPin(const UEdGraphPin* InPin) const
{
	if (!InPin)
	{
		return false;
	}

	if (InPin->ParentPin)
	{
		return IsEventDataPin(InPin->ParentPin);
	}

	return InPin->PinName != UEdGraphSchema_K2::PN_Execute &&
		InPin->PinName != UEdGraphSchema_K2::PN_Then &&
		InPin->PinName != UEdGraphSchema_K2::PN_ReturnValue &&
		InPin->PinName != UK2Node_SceneStateEventBase::PN_EventStream &&
		InPin->PinName != UK2Node_SceneStateEventBase::PN_WorldContextObject;
}

void UK2Node_SceneStateEventBase::CreateEventDataPins(TConstArrayView<UEdGraphPin*> InPinsToSearch)
{
	if (!bHasEventData)
	{
		return;
	}

	const UEdGraphSchema_K2* GraphSchema = GetDefault<UEdGraphSchema_K2>();

	const FStructView EventSchemaDefaultData = EventSchemaHandle.GetDefaultDataView();
	if (!EventSchemaDefaultData.IsValid())
	{
		return;
	}

	for (FProperty* Property : TFieldRange<FProperty>(EventSchemaDefaultData.GetScriptStruct(), EFieldIteratorFlags::IncludeSuper))
	{
		// No need to add if Pin already is there
		if (FindPin(Property->GetFName(), Pins))
		{
			continue;
		}

		if (UEdGraphPin* Pin = CreatePin(EventDataPinDirection, NAME_None, Property->GetFName()))
		{
			Pin->PinFriendlyName = Property->GetDisplayNameText();

			GraphSchema->ConvertPropertyToPinType(Property, /*out*/Pin->PinType);

			if (GraphSchema->PinDefaultValueIsEditable(*Pin))
			{
				FString DefaultValueStr;
				if (FBlueprintEditorUtils::PropertyValueToString(Property, EventSchemaDefaultData.GetMemory(), DefaultValueStr, this))
				{
					GraphSchema->SetPinAutogeneratedDefaultValue(Pin, DefaultValueStr);
				}
			}

			// Copy tooltip from the property.
			GraphSchema->ConstructBasicPinTooltip(*Pin, Property->GetToolTipText(), Pin->PinToolTip);
		}
	}
}

void UK2Node_SceneStateEventBase::MoveEventDataPins(FKismetCompilerContext& InCompilerContext, UEdGraphNode* InTargetIntermediateNode)
{
	if (!ensure(InTargetIntermediateNode) || !bHasEventData)
	{
		return;
	}

	for (UEdGraphPin* Pin : Pins)
	{
		if (IsEventDataPin(Pin))
		{
			UEdGraphPin* const TargetPin = InTargetIntermediateNode->FindPin(Pin->GetFName(), EventDataPinDirection);
			if (!TargetPin || !InCompilerContext.MovePinLinksToIntermediate(*Pin, *TargetPin).CanSafeConnect())
			{
				InCompilerContext.MessageLog.Error(*LOCTEXT("EventDataPinConnectError", "ICE: Error connecting Event Data Pin @@. @@").ToString()
					, Pin, this);
			}
		}
	}
}

bool UK2Node_SceneStateEventBase::ConnectPinsToIntermediate(FKismetCompilerContext& InCompilerContext, UK2Node* InTargetIntermediateNode, FName InSourcePin, FName InTargetPin)
{
	check(InTargetIntermediateNode);

	UEdGraphPin* const SourcePin = FindPin(InSourcePin, Pins);
	UEdGraphPin* const TargetPin = InTargetIntermediateNode->FindPin(InTargetPin);

	return SourcePin && TargetPin
		&& ensure(SourcePin->Direction == TargetPin->Direction)
		&& InCompilerContext.MovePinLinksToIntermediate(*SourcePin, *TargetPin).CanSafeConnect();
}

bool UK2Node_SceneStateEventBase::ChainNode(FNodeExpansionContext& Context, const UK2Node* InNode)
{
	check(InNode);

	const UEdGraphSchema_K2* Schema = Context.CompilerContext.GetSchema();
	check(Schema);

	// if a chaining node exists, connect the current node to it
	if (Context.ChainingNode)
	{
		UEdGraphPin* const ExecPin = InNode->GetExecPin();
		UEdGraphPin* const ThenPin = Context.ChainingNode->GetThenPin();

		if (!ExecPin || !ThenPin || !Schema->TryCreateConnection(ThenPin, ExecPin))
		{
			Context.CompilerContext.MessageLog.Error(*LOCTEXT("SchemaConnectionError", "ICE: Error connecting '{0}' to '{1}' node. @@").ToString()
				, *Context.ChainingNode->GetName()
				, *InNode->GetName()
				, this);
			return false;
		}
	}
	// if a chaining node does not exist, move this node's exec pin to the given node's exec pin
	else
	{
		UEdGraphPin* const SourceExecPin = GetExecPin();
		UEdGraphPin* const TargetExecPin = InNode->GetExecPin();

		if (!SourceExecPin || !TargetExecPin || !Context.CompilerContext.MovePinLinksToIntermediate(*SourceExecPin, *TargetExecPin).CanSafeConnect())
		{
			Context.CompilerContext.MessageLog.Error(
				*LOCTEXT("MoveExecPinLinksError", "ICE: Error moving exec pins to '{0}' node. @@").ToString()
				, *InNode->GetName()
				, this);
			return false;
		}
	}

	Context.ChainingNode = InNode;
	return true;
}

bool UK2Node_SceneStateEventBase::FinishChain(const FNodeExpansionContext& Context)
{
	check(Context.ChainingNode);

	UEdGraphPin* const SourceThenPin = GetThenPin();
	UEdGraphPin* const TargetThenPin = Context.ChainingNode->GetThenPin();

	if (!SourceThenPin || !TargetThenPin || !Context.CompilerContext.MovePinLinksToIntermediate(*SourceThenPin, *TargetThenPin).CanSafeConnect())
	{
		Context.CompilerContext.MessageLog.Error(*LOCTEXT("MoveThenPinLinksError", "ICE: Error moving then pin to '{0}' node. @@").ToString()
			, *Context.ChainingNode->GetName()
			, this);
		return false;
	}

	return true;
}

void UK2Node_SceneStateEventBase::SpawnEventDataNodes(FNodeExpansionContext& Context)
{
	UUserDefinedStruct* EventStruct = EventSchemaHandle.GetEventStruct();
	if (!EventStruct)
	{
		// OK for Event struct to be null. Could be an event with no parameters
		return;
	}

	const UEdGraphSchema_K2* Schema = Context.CompilerContext.GetSchema();
	check(Schema);

	// Create 'Make Instanced Struct' node
	UK2Node_CallFunction* MakeInstancedStructNode = Context.CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, Context.SourceGraph);
	check(MakeInstancedStructNode);
	MakeInstancedStructNode->FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(UBlueprintInstancedStructLibrary, MakeInstancedStruct), UBlueprintInstancedStructLibrary::StaticClass());
	MakeInstancedStructNode->AllocateDefaultPins();

	// Connect the Make Instanced Struct Node
	ChainNode(Context, MakeInstancedStructNode);

	// Connect Event Data Pins to a Make Struct Node
	UK2Node_MakeStruct* const MakeStruct = Context.CompilerContext.SpawnIntermediateNode<UK2Node_MakeStruct>(this, Context.SourceGraph);
	check(MakeStruct);
	MakeStruct->StructType = EventStruct;
	MakeStruct->PostPlacedNewNode();
	MakeStruct->AllocateDefaultPins();

	// Move Event Data Input Pins to Make Struct Input
	MoveEventDataPins(Context.CompilerContext, MakeStruct);

	// Connect the output struct pin of 'MakeStruct' to the 'MakeInstancedStruct' struct input
	{
		UEdGraphPin* InputStructPin = MakeInstancedStructNode->FindPin(TEXT("Value"));
		UEdGraphPin* OutputStructPin = MakeStruct->FindPin(EventStruct->GetFName(), EGPD_Output);

		if (!InputStructPin || !OutputStructPin || !Schema->TryCreateConnection(OutputStructPin, InputStructPin))
		{
			Context.CompilerContext.MessageLog.Error(
				*LOCTEXT("MakeStructConnectError", "ICE: Error connecting Make Struct result to Make Instanced Struct. @@").ToString()
				, this);
			return;
		}

		MakeInstancedStructNode->ReconstructNode();
	} 

	// Connect the Output of Instanced Struct to the Input Event Data Pin
	{
		UEdGraphPin* const InputEventDataPin = Context.EventDataPin;
		UEdGraphPin* const OutputInstancedStructPin = MakeInstancedStructNode->FindPin(UEdGraphSchema_K2::PN_ReturnValue);

		if (!InputEventDataPin || !OutputInstancedStructPin || !Schema->TryCreateConnection(OutputInstancedStructPin, InputEventDataPin))
		{
			Context.CompilerContext.MessageLog.Error(
				*LOCTEXT("EventDataConnectError", "ICE: Error connecting Make Instance Struct result to Event Data. @@").ToString()
				, this);
		}
	}
}

UEdGraphPin* UK2Node_SceneStateEventBase::FindPin(FName InPinName, TConstArrayView<UEdGraphPin*> InPinsToSearch) const
{
	for (UEdGraphPin* Pin : InPinsToSearch)
	{
		if (Pin && Pin->PinName == InPinName)
		{
			return Pin;
		}
	}
	return nullptr;
}

void UK2Node_SceneStateEventBase::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	if (InPropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UK2Node_SceneStateEventBase, EventSchemaHandle))
	{
		OnEventSchemaChanged();
	}
}

bool UK2Node_SceneStateEventBase::HasExternalDependencies(TArray<UStruct*>* OutOptionalOutput) const
{
	if (UUserDefinedStruct* EventStruct = EventSchemaHandle.GetEventStruct())
	{
		if (OutOptionalOutput)
		{
			OutOptionalOutput->AddUnique(EventStruct);
		}
		return true;
	}
	return false;
}

bool UK2Node_SceneStateEventBase::IsCompatibleWithGraph(const UEdGraph* InTargetGraph) const
{
	const UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(InTargetGraph);

	return Super::IsCompatibleWithGraph(InTargetGraph)
		&& (!Blueprint || FBlueprintEditorUtils::FindUserConstructionScript(Blueprint) != InTargetGraph);
}

void UK2Node_SceneStateEventBase::PostPlacedNewNode()
{
	Super::PostPlacedNewNode();
	CreateEventDataPins(Pins);
}

bool UK2Node_SceneStateEventBase::ShouldShowNodeProperties() const
{
	return true;
}

bool UK2Node_SceneStateEventBase::IsNodeSafeToIgnore() const
{
	return true;
}

void UK2Node_SceneStateEventBase::ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& InOldPins)
{
	AllocateDefaultPins();
	CreateEventDataPins(InOldPins);
	RestoreSplitPins(InOldPins);
}

void UK2Node_SceneStateEventBase::GetNodeAttributes(TArray<TKeyValuePair<FString, FString>>& OutNodeAttributes) const
{
	Super::GetNodeAttributes(OutNodeAttributes);

	const USceneStateEventSchemaObject* EventSchema = EventSchemaHandle.GetEventSchema();

	const FString EventSchemaName = EventSchema
		? EventSchema->Name.ToString()
		: TEXT("InvalidEventSchema");

	OutNodeAttributes.Add(TKeyValuePair<FString, FString>(TEXT("Type"), TEXT("AddSceneStateEvent")));
	OutNodeAttributes.Add(TKeyValuePair<FString, FString>(TEXT("Class"), GetNameSafe(GetClass())));
	OutNodeAttributes.Add(TKeyValuePair<FString, FString>(TEXT("Name"), GetName()));
	OutNodeAttributes.Add(TKeyValuePair<FString, FString>(TEXT("EventSchema"), EventSchemaName));
}

void UK2Node_SceneStateEventBase::GetMenuActions(FBlueprintActionDatabaseRegistrar& InActionRegistrar) const
{
	UClass* ActionKey = GetClass();
	if (InActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(ActionKey);
		check(NodeSpawner);
		InActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

FText UK2Node_SceneStateEventBase::GetMenuCategory() const
{
	return LOCTEXT("MenuCategory", "Scene State Event");
}

#undef LOCTEXT_NAMESPACE
