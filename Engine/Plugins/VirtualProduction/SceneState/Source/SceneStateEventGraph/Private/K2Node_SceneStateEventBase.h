// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "K2Node.h"
#include "SceneStateEventSchemaHandle.h"
#include "K2Node_SceneStateEventBase.generated.h"

UCLASS(Abstract)
class UK2Node_SceneStateEventBase : public UK2Node
{
	GENERATED_BODY()

protected:
	/** Common Pin Names */
	static const FLazyName PN_EventStream;
	static const FLazyName PN_WorldContextObject;

	FText GetSchemaDisplayNameText() const;

	FString GetSchemaHandleStringValue() const;

	void OnEventSchemaChanged();

	bool IsEventDataPin(const UEdGraphPin* InPin) const;

	void CreateEventDataPins(TConstArrayView<UEdGraphPin*> InPinsToSearch);

	void MoveEventDataPins(FKismetCompilerContext& InCompilerContext, UEdGraphNode* InTargetIntermediateNode);

	bool ConnectPinsToIntermediate(FKismetCompilerContext& InCompilerContext, UK2Node* InTargetIntermediateNode, FName InSourcePin, FName InTargetPin);

	struct FNodeExpansionContext
	{
		/** Compiler context provided by the expand node func */
		FKismetCompilerContext& CompilerContext;
		/** Source graph provided by the expand node func */
		UEdGraph* SourceGraph;
		/** Input intermediate pin to connect the event data output to */
		UEdGraphPin* EventDataPin;
		/** Last node that was added to the chain. Starts as null */
		const UK2Node* ChainingNode = nullptr;
	};

	/**
	 * Adds the provided intermediate node sequentially after the last chained node,
	 * or if it's the first node moves the exec pins of this node to the intermediate node's exec pins
	 */
	bool ChainNode(FNodeExpansionContext& Context, const UK2Node* InNode);

	/** Finishes chain by moving the then pins of this node to the last chained node's then pin */
	bool FinishChain(const FNodeExpansionContext& Context);

	/** Spawns the intermediate nodes relating to event data. Only does it if the event has parameters (i.e. a valid event struct) */
	void SpawnEventDataNodes(FNodeExpansionContext& Context);

	UEdGraphPin* FindPin(FName InPinName, TConstArrayView<UEdGraphPin*> InPinsToSearch) const;

	//~ Begin UObject
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
	//~ End UObject

	//~ Begin UEdGraphNode
	virtual bool HasExternalDependencies(TArray<UStruct*>* OutOptionalOutput) const override;
    virtual bool IsCompatibleWithGraph(const UEdGraph* InTargetGraph) const override;
    virtual void PostPlacedNewNode() override;
    //~ End UEdGraphNode

    //~ Begin UK2Node
	virtual bool ShouldShowNodeProperties() const override;
    virtual bool IsNodeSafeToIgnore() const override;
    virtual void ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& InOldPins) override;
    virtual void GetNodeAttributes(TArray<TKeyValuePair<FString, FString>>& OutNodeAttributes) const override;
    virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& InActionRegistrar) const override;
    virtual FText GetMenuCategory() const override;
    //~ End UK2Node

	UPROPERTY(EditAnywhere, Category="Scene State Event")
	FSceneStateEventSchemaHandle EventSchemaHandle;

	EEdGraphPinDirection EventDataPinDirection = EGPD_Input;

	bool bHasEventData = true;
};
