// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_SceneStateFindEvent.h"
#include "K2Node_BreakStruct.h"
#include "K2Node_CallFunction.h"
#include "KismetCompiler.h"
#include "SceneStateEventLibrary.h"
#include "SceneStateEventSchema.h"
#include "StructUtils/UserDefinedStruct.h"

#define LOCTEXT_NAMESPACE "K2Node_SceneStateFindEvent"

UK2Node_SceneStateFindEvent::UK2Node_SceneStateFindEvent()
{
	EventDataPinDirection = EGPD_Output;
}

FText UK2Node_SceneStateFindEvent::GetNodeTitle(ENodeTitleType::Type InTitleType) const
{
	if (InTitleType == ENodeTitleType::MenuTitle)
	{
		return LOCTEXT("NodeMenuTitle", "Find Event");
	}
	return FText::Format(LOCTEXT("NodeTitle", "Find Event: {0}"), GetSchemaDisplayNameText());
}

void UK2Node_SceneStateFindEvent::ExpandNode(FKismetCompilerContext& InCompilerContext, UEdGraph* InSourceGraph)
{
	Super::ExpandNode(InCompilerContext, InSourceGraph);

	USceneStateEventSchemaObject* const EventSchema = EventSchemaHandle.GetEventSchema();
	if (!EventSchema)
	{
		InCompilerContext.MessageLog.Error(*LOCTEXT("EventSchemaError", "ICE: No Event Schema specified @@").ToString(), this);
		BreakAllNodeLinks();
		return;
	}

	UK2Node_CallFunction* const FindEventNode = SpawnFindEventNode(GET_FUNCTION_NAME_CHECKED(USceneStateEventLibrary, FindEvent), InCompilerContext, InSourceGraph);
	check(FindEventNode);

	UUserDefinedStruct* const EventStruct = EventSchemaHandle.GetEventStruct();
	if (!EventStruct)
	{
		// OK for Event struct to be null. Could be an event with no parameters
		BreakAllNodeLinks();
		return;
	}

	const UEdGraphSchema_K2* Schema = InCompilerContext.GetSchema();
	check(Schema);

	// Create Get Instanced Struct and wire it with the Output Event Data
	UK2Node_CallFunction* const EventDataToStructNode = InCompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, InSourceGraph);
	check(EventDataToStructNode);
	EventDataToStructNode->FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(USceneStateEventLibrary, EventDataToStruct), USceneStateEventLibrary::StaticClass());
	EventDataToStructNode->AllocateDefaultPins();

	// Connect the 'FindEvent' result Event Data to the 'GetInstancedStruct' input struct
	{
		// Connect the Output of Find Event to the Input Call Get Instanced Struct
		UEdGraphPin* const OutEventDataPin = FindEventNode->FindPin(TEXT("OutEventData"));
		UEdGraphPin* const InEventDataPin = EventDataToStructNode->FindPin(TEXT("InEventData"));

		if (!InEventDataPin || !OutEventDataPin || !Schema->TryCreateConnection(OutEventDataPin, InEventDataPin))
		{
			InCompilerContext.MessageLog.Error(
				*LOCTEXT("EventDataConnectError", "ICE: Error connecting Event Data result to Get Instance Struct. @@").ToString()
				, this);
		}
	}

	// Create Break Struct and wire it to both the Get Instanced Struct and the Out Event Data Pins
	UK2Node_BreakStruct* const BreakStruct = InCompilerContext.SpawnIntermediateNode<UK2Node_BreakStruct>(this, InSourceGraph);
	check(BreakStruct);
	BreakStruct->PostPlacedNewNode();
	BreakStruct->StructType = EventStruct;
	BreakStruct->AllocateDefaultPins();

	// Connect Break Struct input struct pin to the Output of Get Instanced Struct
	// and reconstruct GetInstancedStruct
	{
		UEdGraphPin* InputBreakStructPin = BreakStruct->FindPin(EventStruct->GetFName(), EGPD_Input);
		UEdGraphPin* OutputStructPin = EventDataToStructNode->FindPin(TEXT("OutStructValue"), EGPD_Output);

		if (!InputBreakStructPin || !OutputStructPin || !Schema->TryCreateConnection(OutputStructPin, InputBreakStructPin))
		{
			InCompilerContext.MessageLog.Error(
				*LOCTEXT("MakeStructConnectError", "ICE: Error connecting 'Event Data To Struct' result to 'Break Struct'. @@").ToString()
				, this);
		}

		EventDataToStructNode->ReconstructNode();
	}

	// Move Event Data outputs Pins to the Break Struct output 
	MoveEventDataPins(InCompilerContext, BreakStruct);

	FindEventNode->ReconstructNode();

	BreakAllNodeLinks();
}

#undef LOCTEXT_NAMESPACE
