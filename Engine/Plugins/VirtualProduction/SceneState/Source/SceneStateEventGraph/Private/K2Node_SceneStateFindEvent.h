// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "K2Node_SceneStateFindEventBase.h"
#include "K2Node_SceneStateFindEvent.generated.h"

UCLASS()
class UK2Node_SceneStateFindEvent : public UK2Node_SceneStateFindEventBase
{
	GENERATED_BODY()

public:
	UK2Node_SceneStateFindEvent();

protected:
	//~ Begin UEdGraphNode
	virtual FText GetNodeTitle(ENodeTitleType::Type InTitleType) const override;
	//~ End UEdGraphNode

	//~ Begin UK2Node
	virtual void ExpandNode(FKismetCompilerContext& InCompilerContext, UEdGraph* InSourceGraph) override;
	//~ End UK2Node
};
