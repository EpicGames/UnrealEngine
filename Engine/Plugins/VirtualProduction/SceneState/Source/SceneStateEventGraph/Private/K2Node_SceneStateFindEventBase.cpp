// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_SceneStateFindEventBase.h"
#include "ISceneStateEventHandlerProvider.h"
#include "K2Node_CallFunction.h"
#include "K2Node_Self.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "KismetCompiler.h"
#include "SceneStateEventHandler.h"
#include "SceneStateEventLibrary.h"
#include "SceneStateEventSchema.h"
#include "SceneStateEventStream.h"

#define LOCTEXT_NAMESPACE "K2Node_SceneStateFindEventBase"

UK2Node_SceneStateFindEventBase::UK2Node_SceneStateFindEventBase()
{
	bCanHandleCapturedEventsOnly = CanHandleCapturedEventsOnly();
}

bool UK2Node_SceneStateFindEventBase::CanHandleCapturedEventsOnly(FKismetCompilerContext* InCompilerContext) const
{
	const UK2Node_SceneStateFindEventBase* SourceNode = this;
	if (InCompilerContext)
	{
		SourceNode = CastChecked<UK2Node_SceneStateFindEventBase>(InCompilerContext->MessageLog.FindSourceObject(this));
	}

	if (SourceNode->GetImplementingOuter<ISceneStateEventHandlerProvider>() != nullptr)
	{
		return true;
	}

	if (UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNode(this))
	{
		constexpr bool bIncludeInherited = true;
		return FBlueprintEditorUtils::ImplementsInterface(Blueprint, bIncludeInherited, USceneStateEventHandlerProvider::StaticClass());
	}

	return false;
}

bool UK2Node_SceneStateFindEventBase::FindEventHandlerId(FKismetCompilerContext& InCompilerContext, FGuid& OutHandlerId) const
{
	// Find Source Object of this Node. If this node hasn't been duplicated, FindSourceObject will return this node so it's expected the result to be valid
	// The reason this is required is that On Expand Node this node is most likely already in a cloned graph outered to the Ubergraph
	const UK2Node_SceneStateFindEventBase* SourceNode = CastChecked<UK2Node_SceneStateFindEventBase>(InCompilerContext.MessageLog.FindSourceObject(this));

	// Iterate the outer chain to find an Event Handler Provider that has a Handler for the Event Schema
	UObject* NodeOuter = SourceNode->GetOuter();
	while (NodeOuter)
	{
		ISceneStateEventHandlerProvider* EventHandlerProvider = Cast<ISceneStateEventHandlerProvider>(NodeOuter);
		if (EventHandlerProvider && EventHandlerProvider->FindEventHandlerId(EventSchemaHandle, OutHandlerId))
		{
			return true;
		}
		NodeOuter = NodeOuter->GetOuter();
	}
	return false;
}

UK2Node_CallFunction* UK2Node_SceneStateFindEventBase::SpawnFindEventNode(FName InFindEventFunctionName, FKismetCompilerContext& InCompilerContext, UEdGraph* InSourceGraph)
{
	UK2Node_CallFunction* FindEventNode = InCompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, InSourceGraph);
	check(FindEventNode);
	FindEventNode->FunctionReference.SetExternalMember(InFindEventFunctionName, USceneStateEventLibrary::StaticClass());
	FindEventNode->AllocateDefaultPins();

	// Set Context Object
	bCanHandleCapturedEventsOnly = CanHandleCapturedEventsOnly();
	if (bCanHandleCapturedEventsOnly)
	{
		UK2Node_Self* SelfNode = InCompilerContext.SpawnIntermediateNode<UK2Node_Self>(this, InSourceGraph);
		check(SelfNode);
		SelfNode->AllocateDefaultPins();

		UEdGraphPin* SelfPin = SelfNode->FindPinChecked(UEdGraphSchema_K2::PSC_Self);
		UEdGraphPin* ContextObjectPin = FindEventNode->FindPinChecked(TEXT("InContextObject"));

		const UEdGraphSchema_K2* Schema = InCompilerContext.GetSchema();
		check(Schema);

		if (!Schema->TryCreateConnection(SelfPin, ContextObjectPin))
		{
			InCompilerContext.MessageLog.Error(*LOCTEXT("ContextObjectConnectError", "ICE: Error connecting Context Object Pin @@").ToString(), this);
		}
	}

	// Move Event Stream to the input of the Find Event call function node
	if (!ConnectPinsToIntermediate(InCompilerContext, FindEventNode, UK2Node_SceneStateFindEventBase::PN_EventStream, TEXT("InEventStream")))
	{
		InCompilerContext.MessageLog.Error(*LOCTEXT("EventStreamConnectError", "ICE: Error connecting Event Stream Pin @@").ToString(), this);
	}

	// Set Event Schema
	{
		UEdGraphPin* EventSchemaPin = FindEventNode->FindPinChecked(TEXT("InEventSchema"));
		EventSchemaPin->DefaultValue = GetSchemaHandleStringValue();
	}

	// Set Event Handler Id
	{
		UEdGraphPin* EventHandlerIdPin = FindEventNode->FindPinChecked(TEXT("InEventHandlerId"));
		FGuid HandlerId;
		if (FindEventHandlerId(InCompilerContext, HandlerId))
		{
			EventHandlerIdPin->DefaultValue = LexToString(HandlerId);
		}
	}

	// Set Captured Events Only
	{
		UEdGraphPin* CapturedEventsOnlyPin = FindEventNode->FindPinChecked(TEXT("bInCapturedEventsOnly"));
		CapturedEventsOnlyPin->DefaultValue = (bCanHandleCapturedEventsOnly && bCapturedEventsOnly) ? TEXT("true") : TEXT("false");
	}

	// Move Return Value pin to the output of the Find Event call function node
	if (!ConnectPinsToIntermediate(InCompilerContext, FindEventNode, UEdGraphSchema_K2::PN_ReturnValue, UEdGraphSchema_K2::PN_ReturnValue))
	{
		InCompilerContext.MessageLog.Error(*LOCTEXT("ResultConnectError", "ICE: Error connecting Result Pin @@").ToString(), this);
	}

	return FindEventNode;
}

void UK2Node_SceneStateFindEventBase::PostRename(UObject* InOldOuter, const FName InOldName)
{
	Super::PostRename(InOldOuter, InOldName);
	if (GetOuter() != InOldOuter)
	{
		bCapturedEventsOnly = CanHandleCapturedEventsOnly();
	}
}

void UK2Node_SceneStateFindEventBase::PostEditUndo()
{
	Super::PostEditUndo();
	bCapturedEventsOnly = CanHandleCapturedEventsOnly();
}

void UK2Node_SceneStateFindEventBase::AllocateDefaultPins()
{
	// Creates the Default Event Schema Pins
	Super::AllocateDefaultPins();

	// Event Stream Pin
	CreatePin(EGPD_Input
		, UEdGraphSchema_K2::PC_Object
		, USceneStateEventStream::StaticClass()
		, UK2Node_SceneStateEventBase::PN_EventStream);

	CreatePin(EGPD_Output
		, UEdGraphSchema_K2::PC_Boolean
		, UEdGraphSchema_K2::PN_ReturnValue);
}

bool UK2Node_SceneStateFindEventBase::IsNodePure() const
{
	return true;
}

#undef LOCTEXT_NAMESPACE
