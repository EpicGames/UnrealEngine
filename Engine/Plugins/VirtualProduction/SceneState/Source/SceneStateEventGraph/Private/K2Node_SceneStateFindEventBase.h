// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "K2Node_SceneStateEventBase.h"
#include "K2Node_SceneStateFindEventBase.generated.h"

class UK2Node_CallFunction;

/** Base Implementation of Find Event for FindEvent and HasEvent nodes */
UCLASS(Abstract)
class UK2Node_SceneStateFindEventBase : public UK2Node_SceneStateEventBase
{
	GENERATED_BODY()

public:
	UK2Node_SceneStateFindEventBase();

protected:
	/** Whether a Context Object input pin is required to retrieve the Event Handler Id */
	bool CanHandleCapturedEventsOnly(FKismetCompilerContext* InCompilerContext = nullptr) const;

	/** Attempts to Find a Handler Id only if the node is outered to the Event Handler Provider */
	bool FindEventHandlerId(FKismetCompilerContext& InCompilerContext, FGuid& OutHandlerId) const;

	/** Spawns the Find Event Node and sets up the default pins (Event Stream, Handler Id, etc) */
	UK2Node_CallFunction* SpawnFindEventNode(FName InFindEventFunctionName, FKismetCompilerContext& InCompilerContext, UEdGraph* InSourceGraph);

	//~ Begin UObject
	virtual void PostRename(UObject* InOldOuter, const FName InOldName) override;
	virtual void PostEditUndo() override;
	//~ End UObject

	//~ Begin UEdGraphNode
	virtual void AllocateDefaultPins() override;
	//~ End UEdGraphNode

	//~ Begin UK2Node
	virtual bool IsNodePure() const override;
	//~ End UK2Node

	/** Whether to only consider Events captured by a containing object that handles events */
	UPROPERTY(EditAnywhere, Category="Scene State Event", meta=(DisplayAfter="EventSchema", EditCondition="bCanHandleCapturedEventsOnly", EditConditionHides, HideEditConditionToggle))
    bool bCapturedEventsOnly = true;

	UPROPERTY(Transient, DuplicateTransient, TextExportTransient)
	bool bCanHandleCapturedEventsOnly = false;
};
