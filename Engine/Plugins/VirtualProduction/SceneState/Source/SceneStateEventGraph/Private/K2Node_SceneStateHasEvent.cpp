// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_SceneStateHasEvent.h"
#include "KismetCompiler.h"
#include "SceneStateEventLibrary.h"

#define LOCTEXT_NAMESPACE "K2Node_SceneStateHasEvent"

UK2Node_SceneStateHasEvent::UK2Node_SceneStateHasEvent()
{
	// This only returns a boolean, no Event Data pins provided
	bHasEventData = false;
}

FText UK2Node_SceneStateHasEvent::GetNodeTitle(ENodeTitleType::Type InTitleType) const
{
	if (InTitleType == ENodeTitleType::MenuTitle)
	{
		return LOCTEXT("NodeMenuTitle", "Has Event");
	}
	return FText::Format(LOCTEXT("NodeTitle", "Has Event: {0}"), GetSchemaDisplayNameText());
}

void UK2Node_SceneStateHasEvent::ExpandNode(FKismetCompilerContext& InCompilerContext, UEdGraph* InSourceGraph)
{
	Super::ExpandNode(InCompilerContext, InSourceGraph);

	if (!EventSchemaHandle.GetEventSchema())
	{
		InCompilerContext.MessageLog.Error(*LOCTEXT("EventSchemaError", "ICE: No Event Schema specified @@").ToString(), this);
		BreakAllNodeLinks();
		return;
	}

	SpawnFindEventNode(GET_FUNCTION_NAME_CHECKED(USceneStateEventLibrary, HasEvent), InCompilerContext, InSourceGraph);
	BreakAllNodeLinks();
}

#undef LOCTEXT_NAMESPACE
