// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_SceneStatePushEvent.h"
#include "K2Node_CallFunction.h"
#include "K2Node_MakeStruct.h"
#include "Kismet/BlueprintInstancedStructLibrary.h"
#include "KismetCompiler.h"
#include "SceneStateEventLibrary.h"
#include "SceneStateEventSchema.h"
#include "SceneStateEventStream.h"
#include "StructUtils/UserDefinedStruct.h"

#define LOCTEXT_NAMESPACE "K2Node_SceneStatePushEvent"

UK2Node_SceneStatePushEvent::UK2Node_SceneStatePushEvent()
{
	EventDataPinDirection = EGPD_Input;
}

void UK2Node_SceneStatePushEvent::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();

	// Add Execution Pins
	CreatePin(EGPD_Input
		, UEdGraphSchema_K2::PC_Exec
		, UEdGraphSchema_K2::PN_Execute);

	CreatePin(EGPD_Output
		, UEdGraphSchema_K2::PC_Exec
		, UEdGraphSchema_K2::PN_Then);

	// Event Stream Pin
	CreatePin(EGPD_Input
		, UEdGraphSchema_K2::PC_Object
		, USceneStateEventStream::StaticClass()
		, UK2Node_SceneStateEventBase::PN_EventStream);
}

FText UK2Node_SceneStatePushEvent::GetNodeTitle(ENodeTitleType::Type InTitleType) const
{
	if (InTitleType == ENodeTitleType::MenuTitle)
	{
		return LOCTEXT("NodeMenuTitle", "Push Event");
	}
	return FText::Format(LOCTEXT("NodeTitle", "Push Event: {0}"), GetSchemaDisplayNameText());
}

void UK2Node_SceneStatePushEvent::ExpandNode(FKismetCompilerContext& InCompilerContext, UEdGraph* InSourceGraph)
{
	Super::ExpandNode(InCompilerContext, InSourceGraph);

	USceneStateEventSchemaObject* const EventSchema = EventSchemaHandle.GetEventSchema();
	if (!EventSchema)
	{
		InCompilerContext.MessageLog.Error(*LOCTEXT("EventSchemaError", "ICE: No Event Schema specified @@").ToString(), this);
		BreakAllNodeLinks();
		return;
	}

	const UEdGraphSchema_K2* Schema = InCompilerContext.GetSchema();
	check(Schema);

	// Create 'Push Event' function call node
	UK2Node_CallFunction* const PushEventNode = InCompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, InSourceGraph);
	check(PushEventNode);
	PushEventNode->FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(USceneStateEventLibrary, PushEvent), USceneStateEventLibrary::StaticClass());
	PushEventNode->AllocateDefaultPins();

	// Set the Event Schema
	if (UEdGraphPin* EventSchemaPin = PushEventNode->FindPin(TEXT("InEventSchema")))
	{
		EventSchemaPin->DefaultValue = GetSchemaHandleStringValue();
	}

	// Move the Event Stream Pin to 'PushEvent' Event Stream Input Pin
	if (!ConnectPinsToIntermediate(InCompilerContext, PushEventNode, UK2Node_SceneStateEventBase::PN_EventStream, TEXT("InEventStream")))
	{
		InCompilerContext.MessageLog.Error(*LOCTEXT("EventStreamConnectError", "ICE: Error connecting Event Stream Pin @@").ToString()
			, this);
	}

	FNodeExpansionContext Context = {
		.CompilerContext = InCompilerContext,
		.SourceGraph = InSourceGraph,
		.EventDataPin = PushEventNode->FindPin(TEXT("InEventData")) };

	SpawnEventDataNodes(Context);
	ChainNode(Context, PushEventNode);
	FinishChain(Context);

	PushEventNode->ReconstructNode();
	BreakAllNodeLinks();
}

#undef LOCTEXT_NAMESPACE
