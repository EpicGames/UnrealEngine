// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SceneStateEventGraph : ModuleRules
{
    public SceneStateEventGraph(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "BlueprintGraph",
                "Core",
                "CoreUObject",
                "Engine",
                "SceneStateEvent",
            }
        );

        if (Target.Type == TargetRules.TargetType.Editor)
        {
            PrivateDependencyModuleNames.AddRange(
                new string[]
                {
                    "KismetCompiler",
                    "UnrealEd",
                });
        }
    }
}