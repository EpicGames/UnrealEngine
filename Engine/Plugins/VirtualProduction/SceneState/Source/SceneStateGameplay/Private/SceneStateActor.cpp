// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateActor.h"
#include "SceneStateComponent.h"

const FLazyName ASceneStateActor::SceneStateComponentName(TEXT("SceneStateComponent"));

ASceneStateActor::ASceneStateActor(const FObjectInitializer& InObjectInitializer)
	: Super(InObjectInitializer)
{
	PrimaryActorTick.bCanEverTick = true;
	PrimaryActorTick.bStartWithTickEnabled = true;

	SceneStateComponent = CreateDefaultSubobject<USceneStateComponent>(ASceneStateActor::SceneStateComponentName);

#if WITH_EDITORONLY_DATA
	bIsSpatiallyLoaded = false;
#endif
}

void ASceneStateActor::SetSceneStateClass(TSubclassOf<USceneStateObject> InSceneStateClass)
{
	if (SceneStateComponent)
	{
		SceneStateComponent->SetSceneStateClass(InSceneStateClass);
	}
}

TSubclassOf<USceneStateObject> ASceneStateActor::GetSceneStateClass() const
{
	if (SceneStateComponent)
	{
		SceneStateComponent->GetSceneStateClass();
	}
	return nullptr;
}

USceneStateObject* ASceneStateActor::GetSceneState() const
{
	if (SceneStateComponent)
	{
		return SceneStateComponent->GetSceneState();
	}
	return nullptr;
}
