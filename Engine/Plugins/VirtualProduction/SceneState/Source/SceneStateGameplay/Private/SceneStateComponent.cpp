// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateComponent.h"
#include "GameFramework/Actor.h"
#include "SceneStateComponentInstanceData.h"
#include "SceneStateComponentPlayer.h"
#include "SceneStateUtils.h"

const FLazyName USceneStateComponent::SceneStatePlayerName(TEXT("SceneStatePlayer"));

USceneStateComponent::USceneStateComponent(const FObjectInitializer& InObjectInitializer)
	: Super(InObjectInitializer)
{
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.bStartWithTickEnabled = true;

	SceneStatePlayer = CreateDefaultSubobject<USceneStateComponentPlayer>(USceneStateComponent::SceneStatePlayerName);
}

TSubclassOf<USceneStateObject> USceneStateComponent::GetSceneStateClass() const
{
	if (SceneStatePlayer)
	{
		return SceneStatePlayer->GetSceneStateClass();
	}
	return nullptr;
}

void USceneStateComponent::SetSceneStateClass(TSubclassOf<USceneStateObject> InSceneStateClass)
{
	if (SceneStatePlayer)
	{
		SceneStatePlayer->SetSceneStateClass(InSceneStateClass);
	}
}

USceneStateObject* USceneStateComponent::GetSceneState() const
{
	if (SceneStatePlayer)
	{
		return SceneStatePlayer->GetSceneState();
	}
	return nullptr;
}

void USceneStateComponent::ApplyComponentInstanceData(FSceneStateComponentInstanceData* InComponentInstanceData)
{
	check(InComponentInstanceData);

	SceneStatePlayer = InComponentInstanceData->GetSceneStatePlayer();

	// The instance data's player is outered to the old component. Rename it to be outered to this new one
	// avoid having to do this if the scene state player is already outered to this component
	if (SceneStatePlayer && SceneStatePlayer->GetOuter() != this)
	{
		// discard any existing object that has the name of the Scene State Player, so that there's no collision
		UE::SceneState::DiscardObject(this, *SceneStatePlayer->GetName());

		SceneStatePlayer->Rename(nullptr, this, REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
	}
}

void USceneStateComponent::InitializeComponent()
{
	Super::InitializeComponent();

	if (SceneStatePlayer)
	{
		SceneStatePlayer->Setup();
	}
}

void USceneStateComponent::OnRegister()
{
	Super::OnRegister();

	if (SceneStatePlayer)
	{
		SceneStatePlayer->Setup();
	}
}

void USceneStateComponent::BeginPlay()
{
	Super::BeginPlay();

	if (SceneStatePlayer)
	{
		SceneStatePlayer->Begin();
	}
}

void USceneStateComponent::TickComponent(float InDeltaTime, ELevelTick InTickType, FActorComponentTickFunction* InThisTickFunction)
{
	Super::TickComponent(InDeltaTime, InTickType, InThisTickFunction);

	if (SceneStatePlayer)
	{
		SceneStatePlayer->Tick(InDeltaTime);
	}
}

void USceneStateComponent::EndPlay(const EEndPlayReason::Type InEndPlayReason)
{
	if (SceneStatePlayer)
	{
		SceneStatePlayer->End();
	}

	Super::EndPlay(InEndPlayReason);
}

TStructOnScope<FActorComponentInstanceData> USceneStateComponent::GetComponentInstanceData() const
{
	return MakeStructOnScope<FActorComponentInstanceData, FSceneStateComponentInstanceData>(this);
}
