// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateComponentInstanceData.h"
#include "SceneStateComponent.h"

FSceneStateComponentInstanceData::FSceneStateComponentInstanceData(const USceneStateComponent* InSourceComponent)
	: FActorComponentInstanceData(InSourceComponent)
	, SceneStatePlayer(InSourceComponent->GetSceneStatePlayer())
{
}

bool FSceneStateComponentInstanceData::ContainsData() const
{
	return true;
}

void FSceneStateComponentInstanceData::ApplyToComponent(UActorComponent* InComponent, const ECacheApplyPhase InCacheApplyPhase)
{
	Super::ApplyToComponent(InComponent, InCacheApplyPhase);
	CastChecked<USceneStateComponent>(InComponent)->ApplyComponentInstanceData(this);
}

void FSceneStateComponentInstanceData::AddReferencedObjects(FReferenceCollector& InCollector)
{
	Super::AddReferencedObjects(InCollector);
	InCollector.AddReferencedObject(SceneStatePlayer);
}
