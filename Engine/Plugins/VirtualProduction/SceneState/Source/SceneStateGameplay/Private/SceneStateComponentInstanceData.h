// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ComponentInstanceDataCache.h"
#include "SceneStateComponentInstanceData.generated.h"

class USceneStateComponent;
class USceneStateComponentPlayer;

USTRUCT()
struct FSceneStateComponentInstanceData : public FActorComponentInstanceData
{
	GENERATED_BODY()

	FSceneStateComponentInstanceData()
		: SceneStatePlayer(nullptr)
	{
	}

	explicit FSceneStateComponentInstanceData(const USceneStateComponent* InSourceComponent);

	USceneStateComponentPlayer* GetSceneStatePlayer() const
	{
		return SceneStatePlayer;
	}

	//~ Begin FActorComponentInstanceData
	virtual bool ContainsData() const override;
	virtual void ApplyToComponent(UActorComponent* InComponent, const ECacheApplyPhase InCacheApplyPhase) override;
	virtual void AddReferencedObjects(FReferenceCollector& InCollector) override;
	//~ End FActorComponentInstanceData

private:
	TObjectPtr<USceneStateComponentPlayer> SceneStatePlayer;
};
