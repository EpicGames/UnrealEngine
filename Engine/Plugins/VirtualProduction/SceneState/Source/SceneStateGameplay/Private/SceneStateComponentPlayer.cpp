// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateComponentPlayer.h"
#include "GameFramework/Actor.h"
#include "SceneStateComponent.h"

AActor* USceneStateComponentPlayer::GetActor() const
{
	if (UActorComponent* Component = Cast<UActorComponent>(GetOuter()))
	{
		return Component->GetOwner();
	}
	return nullptr;
}

bool USceneStateComponentPlayer::OnGetContextName(FString& OutContextName) const
{
	if (AActor* Actor = GetActor())
	{
#if WITH_EDITOR
		OutContextName = Actor->GetActorLabel();
#else
		OutContextName = Actor->GetName();
#endif
		return true;
	}
	return false;
}

bool USceneStateComponentPlayer::OnGetContextObject(UObject*& OutContextObject) const
{
	OutContextObject = GetActor();
	return true;
}
