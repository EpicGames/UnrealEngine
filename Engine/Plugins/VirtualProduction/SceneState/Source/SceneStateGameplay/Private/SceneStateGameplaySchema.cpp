// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateGameplaySchema.h"
#include "GameFramework/Actor.h"

#if WITH_EDITOR
#include "SceneStateUtils.h"
#include "Tasks/SceneStateTask.h"
#include "Tasks/SceneStateTaskMetadata.h"
#endif

#if WITH_EDITOR
bool USceneStateGameplaySchema::OnIsTaskStructAllowed(TSubScriptStructOf<FSceneStateTask> InTaskStruct) const
{
	// Only include core/utility and gameplay tasks.
	constexpr bool bIncludeSuperStructs = false;
	return UE::SceneState::Editor::StructHasAnyMetaData(InTaskStruct,
		{
			UE::SceneState::Editor::MD_CoreTask,
			UE::SceneState::Editor::MD_UtilityTask,
			UE::SceneState::Editor::MD_GameplayTask,
		}, bIncludeSuperStructs);
}
#endif

TSubclassOf<UObject> USceneStateGameplaySchema::OnGetContextObjectClass() const
{
	return AActor::StaticClass();
}
