// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "SceneStateActor.generated.h"

#define UE_API SCENESTATEGAMEPLAY_API

class USceneStateComponent;
class USceneStateObject;

UCLASS(MinimalAPI)
class ASceneStateActor : public AActor
{
	GENERATED_BODY()

public:
	UE_API static const FLazyName SceneStateComponentName;

	UE_API ASceneStateActor(const FObjectInitializer& InObjectInitializer);

	USceneStateComponent* GetSceneStateComponent() const
	{
		return SceneStateComponent;
	}

	UE_API void SetSceneStateClass(TSubclassOf<USceneStateObject> InSceneStateClass);

	UE_API TSubclassOf<USceneStateObject> GetSceneStateClass() const;

	UFUNCTION(BlueprintCallable, Category = "Scene State")
	UE_API USceneStateObject* GetSceneState() const;

private:
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category="Scene State", meta=(AllowPrivateAccess="true"))
	TObjectPtr<USceneStateComponent> SceneStateComponent;
};

#undef UE_API
