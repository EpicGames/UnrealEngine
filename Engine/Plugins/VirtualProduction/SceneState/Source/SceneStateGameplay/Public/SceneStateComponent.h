// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"
#include "SceneStateComponent.generated.h"

#define UE_API SCENESTATEGAMEPLAY_API

class USceneStateObject;
class USceneStateComponentPlayer;
struct FSceneStateComponentInstanceData;

UCLASS(MinimalAPI, BlueprintType, meta=(BlueprintSpawnableComponent))
class USceneStateComponent : public UActorComponent
{
	GENERATED_BODY()

public:
	UE_API static const FLazyName SceneStatePlayerName;

	UE_API USceneStateComponent(const FObjectInitializer& InObjectInitializer);

	USceneStateComponentPlayer* GetSceneStatePlayer() const
	{
		return SceneStatePlayer;
	}

	UE_API TSubclassOf<USceneStateObject> GetSceneStateClass() const;

	UE_API void SetSceneStateClass(TSubclassOf<USceneStateObject> InSceneStateClass);

	UFUNCTION(BlueprintCallable, Category = "Scene State")
	UE_API USceneStateObject* GetSceneState() const;

	void ApplyComponentInstanceData(FSceneStateComponentInstanceData* InComponentInstanceData);

	//~ Begin UActorComponent
	UE_API virtual void InitializeComponent() override;
	UE_API virtual void OnRegister() override;
	UE_API virtual void BeginPlay() override;
	UE_API virtual void TickComponent(float InDeltaTime, ELevelTick InTickType, FActorComponentTickFunction* InThisTickFunction) override;
	UE_API virtual void EndPlay(const EEndPlayReason::Type InEndPlayReason) override;
	UE_API virtual TStructOnScope<FActorComponentInstanceData> GetComponentInstanceData() const override;
	//~ End UActorComponent

private:
	UPROPERTY(VisibleAnywhere, Instanced, Category="Scene State", meta=(ShowInnerProperties))
	TObjectPtr<USceneStateComponentPlayer> SceneStatePlayer;
};

#undef UE_API
