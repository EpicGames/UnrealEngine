// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneStatePlayer.h"
#include "SceneStateComponentPlayer.generated.h"

#define UE_API SCENESTATEGAMEPLAY_API

class AActor;
class USceneStateComponent;

/** Scene State Player for Scene State Components */
UCLASS(MinimalAPI)
class USceneStateComponentPlayer : public USceneStatePlayer
{
	GENERATED_BODY()

protected:
	UE_API AActor* GetActor() const;

	//~ Begin USceneStatePlayer
	UE_API virtual bool OnGetContextName(FString& OutContextName) const override;
	UE_API virtual bool OnGetContextObject(UObject*& OutContextObject) const override;
	//~ End USceneStatePlayer
};

#undef UE_API
