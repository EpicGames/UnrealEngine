// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneStateSchema.h"
#include "Templates/SubclassOf.h"
#include "SceneStateGameplaySchema.generated.h"

#define UE_API SCENESTATEGAMEPLAY_API

#if WITH_EDITOR
namespace UE::SceneState::Editor
{
	/** 
	 * Metadata to identify a task as a gameplay task 
	 * Derived tasks will NOT be recognized as 'GameplayTask' unless they also specify this metadata.
	 */
	const FLazyName MD_GameplayTask(TEXT("GameplayTask"));
}
#endif

/** Schema for scene states played via the Scene State Component / Actor context */
UCLASS(MinimalAPI)
class USceneStateGameplaySchema : public USceneStateSchema
{
	GENERATED_BODY()

protected:
	//~ Begin USceneStateSchema
#if WITH_EDITOR
	UE_API virtual bool OnIsTaskStructAllowed(TSubScriptStructOf<FSceneStateTask> InTaskStruct) const override;
#endif
	UE_API virtual TSubclassOf<UObject> OnGetContextObjectClass() const override;
	//~ End USceneStateSchema
};

#undef UE_API
