// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateActorFactory.h"
#include "SceneStateActor.h"
#include "SceneStateBlueprint.h"
#include "SceneStateGameplaySchema.h"
#include "SceneStateGeneratedClass.h"
#include "SceneStateObject.h"

#define LOCTEXT_NAMESPACE "SceneStateActorFactory"

USceneStateActorFactory::USceneStateActorFactory()
{
	DisplayName = LOCTEXT("SceneStateDisplayName", "Scene State");
	NewActorClass = ASceneStateActor::StaticClass();
}

bool USceneStateActorFactory::CanCreateActorFrom(const FAssetData& InAssetData, FText& OutErrorMessage)
{
	if (!InAssetData.IsValid() || !InAssetData.IsInstanceOf<USceneStateBlueprint>())
	{
		OutErrorMessage = LOCTEXT("InvalidSceneStateAsset", "A valid Scene State asset must be specified.");
		return false;
	}

	const FAssetTagValueRef SchemaClassTagValue = InAssetData.TagsAndValues.FindTag(TEXT("SceneStateSchemaClass"));
	if (!SchemaClassTagValue.IsSet())
	{
		// Allow creating from older scene states with no schema tag.
		return true;
	}

	const UClass* const SchemaClass = FindObject<UClass>(nullptr, SchemaClassTagValue.AsString());
	if (!SchemaClass)
	{
		OutErrorMessage = FText::Format(LOCTEXT("NullSchemaClass", "Schema path '{0}' is not found.")
			, FText::FromString(SchemaClassTagValue.AsString()));
		return false;
	}

	if (!SchemaClass->IsChildOf<USceneStateGameplaySchema>())
	{
		OutErrorMessage = FText::Format(LOCTEXT("InvalidSchemaClass", "Schema class '{0}' is not child of Scene State Gameplay Schema.")
			, FText::FromString(SchemaClassTagValue.AsString()));
		return false;
	}

	return true;
}

void USceneStateActorFactory::PostSpawnActor(UObject* InAsset, AActor* InNewActor)
{
	Super::PostSpawnActor(InAsset, InNewActor);

	ASceneStateActor* SceneStateActor = CastChecked<ASceneStateActor>(InNewActor);
	SceneStateActor->SetSceneStateClass(GetSceneStateClass(InAsset));
}

TSubclassOf<USceneStateObject> USceneStateActorFactory::GetSceneStateClass(UObject* InAsset) const
{
	if (USceneStateBlueprint* Blueprint = Cast<USceneStateBlueprint>(InAsset))
	{
		return Blueprint->GeneratedClass.Get();
	}

	return Cast<USceneStateGeneratedClass>(InAsset);
}

#undef LOCTEXT_NAMESPACE
