// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorFactories/ActorFactory.h"
#include "SceneStateActorFactory.generated.h"

class USceneStateObject;

UCLASS()
class USceneStateActorFactory : public UActorFactory
{
	GENERATED_BODY()

public:
	USceneStateActorFactory();

	//~ Begin UActorFactory
	virtual bool CanCreateActorFrom(const FAssetData& InAssetData, FText& OutErrorMessage) override;
	virtual void PostSpawnActor(UObject* InAsset, AActor* InNewActor) override;
	//~ End UActorFactory

private:
	TSubclassOf<USceneStateObject> GetSceneStateClass(UObject* InAsset) const;
};
