// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateGameplayContextEditor.h"
#include "Engine/Engine.h"
#include "Engine/GameViewportClient.h"
#include "Framework/Application/SlateApplication.h"
#include "Slate/SceneViewport.h"
#include "Styling/SlateStyleMacros.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SViewport.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SceneStateGameplayContextEditor"

namespace UE::SceneState::Editor
{

namespace Private
{

class SGameViewport : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SGameViewport){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArguments, UGameViewportClient* InViewportClient)
	{
		TSharedRef<SViewport> ViewportWidget = SNew(SViewport)
			.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute())
			.RenderDirectlyToWindow(false)
			.EnableGammaCorrection(false) // Scene rendering handles gamma correction
			.EnableBlending(true);

		SceneViewport = FSceneViewport::Create(InViewportClient, ViewportWidget);
		ViewportClient.Reset(InViewportClient);

		ChildSlot
		[
			ViewportWidget
		];
	}

	//~ Begin SWidget
	virtual int32 OnPaint(const FPaintArgs& InArgs, const FGeometry& InAllottedGeometry, const FSlateRect& InMyCullingRect, FSlateWindowElementList& OutDrawElements, int32 InLayerId, const FWidgetStyle& InWidgetStyle, bool bInParentEnabled) const override
	{
		SceneViewport->Draw();
		return SCompoundWidget::OnPaint(InArgs, InAllottedGeometry, InMyCullingRect, OutDrawElements, InLayerId, InWidgetStyle, bInParentEnabled);
	}
	//~ End SWidget

	TStrongObjectPtr<UGameViewportClient> ViewportClient;
	TSharedPtr<FSceneViewport> SceneViewport;
};

} // Private

void FGameplayContextEditor::GetContextClasses(TArray<TSubclassOf<UObject>>& OutContextClasses) const
{
	OutContextClasses.Append({ AActor::StaticClass(), UActorComponent::StaticClass() });
}

TSharedPtr<SWidget> FGameplayContextEditor::CreateViewWidget(const FContextParams& InContextParams) const
{
	FWorldContext* const WorldContext = GEngine ? GEngine->GetWorldContextFromWorld(InContextParams.ContextObject->GetWorld()) : nullptr;
	if (WorldContext && WorldContext->World())
	{
		if (UGameViewportClient* ViewportClient = CreateViewportClient(*WorldContext))
		{
			return SNew(Private::SGameViewport, ViewportClient);
		}

		// When the game viewport client could not be created (e.g. in editor view)
		return SNew(SBox)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("EditorContextObjectTitle", "Play in editor to view"))
				.Font(DEFAULT_FONT("Italic", 10))
				.ColorAndOpacity(FLinearColor::White)
			];
	}
	return nullptr;
}

UGameViewportClient* FGameplayContextEditor::CreateViewportClient(FWorldContext& InWorldContext) const
{
	if (UGameInstance* const GameInstance = InWorldContext.World()->GetGameInstance())
	{
		UGameViewportClient* ViewportClient = NewObject<UGameViewportClient>(GEngine, GEngine->GameViewportClientClass);
		ViewportClient->Init(InWorldContext, GameInstance, /*bCreateNewAudioDevice*/false);
		ViewportClient->SetIgnoreInput(true);
		ViewportClient->SetHideCursorDuringCapture(false);
		ViewportClient->SetIsSimulateInEditorViewport(true);
		return ViewportClient;
	}
	return nullptr;
}

} // UE::SceneState::Editor

#undef LOCTEXT_NAMESPACE
