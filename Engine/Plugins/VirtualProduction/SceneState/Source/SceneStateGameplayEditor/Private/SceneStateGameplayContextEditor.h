// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISceneStateContextEditor.h"

class UGameViewportClient;
struct FWorldContext;

namespace UE::SceneState::Editor
{

/** Context editor for gameplay elements like Actors and Actor Components */
class FGameplayContextEditor : public IContextEditor
{
public:
	//~ Begin IContextEditor
	virtual void GetContextClasses(TArray<TSubclassOf<UObject>>& OutContextClasses) const override;
	virtual TSharedPtr<SWidget> CreateViewWidget(const FContextParams& InContextParams) const override;
	//~ End IContextEditor

	/** Creates the game viewport client for the given world context, or null if there's no valid game instance in that world */
	UGameViewportClient* CreateViewportClient(FWorldContext& InWorldContext) const;
};

} // UE::SceneState::Editor
