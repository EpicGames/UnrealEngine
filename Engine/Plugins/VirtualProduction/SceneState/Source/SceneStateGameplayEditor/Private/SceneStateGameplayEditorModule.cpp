// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateGameplayEditorModule.h"
#include "ISceneStateBlueprintEditorModule.h"
#include "ISequencerModule.h"
#include "Modules/ModuleManager.h"
#include "SceneStateGameplayContextEditor.h"
#include "SceneStateSequencerSchema.h"

IMPLEMENT_MODULE(UE::SceneState::Editor::FGameplayEditorModule, SceneStateGameplayEditor)

namespace UE::SceneState::Editor
{

void FGameplayEditorModule::StartupModule()
{
	GameplayContextEditor = MakeShared<FGameplayContextEditor>();
	IBlueprintEditorModule::Get().RegisterContextEditor(GameplayContextEditor);

	SequencerSchema = MakeShared<FSequencerSchema>();
	ISequencerModule& SequencerModule = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer");
	SequencerModule.RegisterObjectSchema(SequencerSchema);
}

void FGameplayEditorModule::ShutdownModule()
{
	if (IBlueprintEditorModule* BlueprintEditorModule = IBlueprintEditorModule::GetPtr())
	{
		BlueprintEditorModule->UnregisterContextEditor(GameplayContextEditor);
		GameplayContextEditor.Reset();
	}

	if (ISequencerModule* SequencerModule = FModuleManager::Get().GetModulePtr<ISequencerModule>("Sequencer"))
	{
		SequencerModule->UnregisterObjectSchema(SequencerSchema);
		SequencerSchema.Reset();
	}
}

} // UE::SceneState::Editor
