// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

namespace UE::SceneState::Editor
{
	class FGameplayContextEditor;
	class FSequencerSchema;
}

namespace UE::SceneState::Editor
{

class FGameplayEditorModule : public IModuleInterface
{
	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface

	TSharedPtr<FGameplayContextEditor> GameplayContextEditor;
	TSharedPtr<FSequencerSchema> SequencerSchema;
};

} // UE::SceneState::Editor
