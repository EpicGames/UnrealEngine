// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateSequencerSchema.h"
#include "ISequencerModule.h"
#include "MVVM/ObjectBindingModelStorageExtension.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/Selection/SequencerSelectionEventSuppressor.h"
#include "MVVM/ViewModels/ObjectBindingModel.h"
#include "SceneStateComponent.h"
#include "SceneStateObject.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "SceneStateSequencerSchema"

namespace UE::SceneState::Editor
{

UObject* FSequencerSchema::GetParentObject(UObject* InObject) const
{
	if (const USceneStateObject* SceneState = Cast<USceneStateObject>(InObject))
	{
		return SceneState->GetTypedOuter<USceneStateComponent>();
	}

	if (const USceneStateComponent* Component = Cast<USceneStateComponent>(InObject))
	{
		return Component->GetOwner();
	}

	return nullptr;
}

UE::Sequencer::FObjectSchemaRelevancy FSequencerSchema::GetRelevancy(const UObject* InObject) const
{
	if (InObject->IsA<USceneStateObject>())
	{
		return USceneStateObject::StaticClass();
	}

	if (InObject->IsA<USceneStateComponent>())
	{
		return USceneStateComponent::StaticClass();
	}

	return UE::Sequencer::FObjectSchemaRelevancy();
}

TSharedPtr<FExtender> FSequencerSchema::ExtendObjectBindingMenu(TSharedRef<FUICommandList> InCommandList
	, TWeakPtr<ISequencer> InSequencerWeak
	, TConstArrayView<UObject*> InContextSensitiveObjects) const
{
	TArray<TWeakObjectPtr<USceneStateComponent>> SceneStateComponents;
	for (UObject* Object : InContextSensitiveObjects)
	{
		if (USceneStateComponent* SceneStateComponent = Cast<USceneStateComponent>(Object))
		{
			SceneStateComponents.Add(SceneStateComponent);
		}
	}

	// No Scene State Components nothing to add
	if (SceneStateComponents.IsEmpty())
	{
		return nullptr;
	}

	TSharedRef<FExtender> Extender = MakeShared<FExtender>();

	Extender->AddMenuExtension(SequencerMenuExtensionPoints::AddTrackMenu_PropertiesSection
		, EExtensionHook::Before
		, InCommandList
		, FMenuExtensionDelegate::CreateStatic(&FSequencerSchema::OnAddTrackMenuExtension
			, InSequencerWeak
			, SceneStateComponents));

	return Extender;
}

void FSequencerSchema::OnAddTrackMenuExtension(FMenuBuilder& InMenuBuilder
	, TWeakPtr<ISequencer> InSequencerWeak
	, TArray<TWeakObjectPtr<USceneStateComponent>> InComponents)
{
	InMenuBuilder.BeginSection(TEXT("Scene State"), LOCTEXT("SceneStateSection", "Scene State"));
	{
		InMenuBuilder.AddMenuEntry(LOCTEXT("AddSceneStateLabel", "Scene State")
			, LOCTEXT("AddSceneStateTooltip", "Add Scene State Object(s)")
			, FSlateIcon()
			, FExecuteAction::CreateStatic(&FSequencerSchema::AddSceneStateComponents
				, InSequencerWeak
				, InComponents));
	}
	InMenuBuilder.EndSection();
}

void FSequencerSchema::AddSceneStateComponents(TWeakPtr<ISequencer> InSequencerWeak
	, TArray<TWeakObjectPtr<USceneStateComponent>> InComponents)
{
	using namespace UE::Sequencer;

	TSharedPtr<ISequencer> Sequencer = InSequencerWeak.Pin();
	if (!Sequencer)
	{
		return;
	}

	const TSharedPtr<FSequencerEditorViewModel> SequencerViewModel = Sequencer->GetViewModel();
	if (!ensure(SequencerViewModel.IsValid()))
	{
		return;
	}

	FObjectBindingModelStorageExtension* const ObjectStorage = SequencerViewModel->GetRootModel()->CastDynamic<FObjectBindingModelStorageExtension>();
	if (!ensure(ObjectStorage))
	{
		return;
	}

	const TSharedPtr<FSequencerSelection> Selection = SequencerViewModel->GetSelection();
	if (!ensure(Selection.IsValid()))
	{
		return;
	}

	TArray<USceneStateObject*> SceneStateObjects;
	SceneStateObjects.Reserve(InComponents.Num());

	for (const TWeakObjectPtr<USceneStateComponent>& ComponentWeak : InComponents)
	{
		USceneStateComponent* ResolvedComponent = ComponentWeak.Get();
		if (!ResolvedComponent)
		{
			continue;
		}

		if (USceneStateObject* SceneStateObject = ResolvedComponent->GetSceneState())
		{
			SceneStateObjects.Add(SceneStateObject);
		}
	}

	if (SceneStateObjects.IsEmpty())
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("AddSceneState", "Add Scene State"));

	FSelectionEventSuppressor SuppressEvents = Selection->SuppressEvents();
	Selection->Outliner.Empty();

	if (SceneStateObjects.Num() > 0)
	{
		// Use only first element.
		// This is to be consistent with the behavior with how Components are added in FActorSchema
		// see: FActorSchema::HandleAddComponentActionExecute
		USceneStateObject* SceneStateObject = SceneStateObjects[0];

		const FGuid ObjectId = Sequencer->GetHandleToObject(SceneStateObject);

		if (TSharedPtr<FObjectBindingModel> Model = ObjectStorage->FindModelForObjectBinding(ObjectId))
		{
			Selection->Outliner.Select(Model);
		}
	}
}

} // UE::SceneState::Editor

#undef LOCTEXT_NAMESPACE
