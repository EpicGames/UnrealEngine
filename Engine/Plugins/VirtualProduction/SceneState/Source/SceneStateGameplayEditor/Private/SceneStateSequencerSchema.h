// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "ISequencerObjectSchema.h"
#include "UObject/WeakObjectPtrTemplatesFwd.h"

class FMenuBuilder;
class USceneStateComponent;

namespace UE::SceneState::Editor
{

class FSequencerSchema : public UE::Sequencer::IObjectSchema
{
	//~ Begin IObjectSchema
	virtual UObject* GetParentObject(UObject* InObject) const override;
	virtual UE::Sequencer::FObjectSchemaRelevancy GetRelevancy(const UObject* InObject) const override;
	virtual TSharedPtr<FExtender> ExtendObjectBindingMenu(TSharedRef<FUICommandList> InCommandList, TWeakPtr<ISequencer> InSequencerWeak, TConstArrayView<UObject*> InContextSensitiveObjects) const override;
	//~ End IObjectSchema

	static void OnAddTrackMenuExtension(FMenuBuilder& InMenuBuilder, TWeakPtr<ISequencer> InSequencerWeak, TArray<TWeakObjectPtr<USceneStateComponent>> InComponents);
	static void AddSceneStateComponents(TWeakPtr<ISequencer> InSequencerWeak, TArray<TWeakObjectPtr<USceneStateComponent>> InComponents);
};

} // UE::SceneState::Editor
