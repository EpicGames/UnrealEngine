// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SceneStateGameplayEditor : ModuleRules
{
    public SceneStateGameplayEditor(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "Core",
                "CoreUObject",
                "Engine",
                "LevelEditor",
                "SceneState",
                "SceneStateBlueprint",
                "SceneStateBlueprintEditor",
                "SceneStateGameplay",
                "Sequencer",
                "SequencerCore",
                "Slate",
                "SlateCore",
                "UnrealEd",
            }
        );
    }
}
