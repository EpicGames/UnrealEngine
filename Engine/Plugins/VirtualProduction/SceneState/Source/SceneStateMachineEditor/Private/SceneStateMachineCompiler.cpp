// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateMachineCompiler.h"
#include "Algo/Count.h"
#include "EdGraphUtilities.h"
#include "ISceneStateMachineCompilerContext.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Misc/EnumerateRange.h"
#include "Nodes/SceneStateMachineConduitNode.h"
#include "Nodes/SceneStateMachineEntryNode.h"
#include "Nodes/SceneStateMachineExitNode.h"
#include "Nodes/SceneStateMachineStateNode.h"
#include "Nodes/SceneStateMachineTransitionNode.h"
#include "Nodes/SceneStateTransitionResultNode.h"
#include "SceneState.h"
#include "SceneStateMachine.h"
#include "SceneStateMachineConduitCompiler.h"
#include "SceneStateMachineGraph.h"
#include "SceneStateMachineTaskCompiler.h"
#include "SceneStateMachineTransitionCompiler.h"
#include "SceneStateTemplateData.h"
#include "SceneStateTransitionGraph.h"
#include "SceneStateUtils.h"
#include "Tasks/SceneStateBlueprintableTask.h"
#include "Tasks/SceneStateTask.h"
#include "UObject/UObjectThreadContext.h"

namespace UE::SceneState::Editor
{

namespace Private
{

// Converts Transition Parameters to map of absolute index to the parameter property bag, only if they're valid.
// A map is used because most likely there will be much more transitions without parameters than transitions with parameters.
TMap<uint16, FInstancedPropertyBag> ConvertParameterMap(FSceneStateRange InTransitionRange, TArrayView<FInstancedPropertyBag> InParametersList)
{
	TMap<uint16, FInstancedPropertyBag> TransitionParameterMap;
	TransitionParameterMap.Reserve(Algo::CountIf(InParametersList,
		[](const FInstancedPropertyBag& InParameters)
		{
			return InParameters.IsValid();
		}));

	for (TEnumerateRef<FInstancedPropertyBag> Parameters : EnumerateRange(InParametersList))
	{
		if (Parameters->IsValid())
		{
			// Convert the index to absolute
			const uint16 AbsoluteIndex = InTransitionRange.Index + Parameters.GetIndex();
			TransitionParameterMap.Add(AbsoluteIndex, MoveTemp(*Parameters));
		}
	}
	return TransitionParameterMap;
}

} // Private

FStateMachineCompiler::FStateMachineCompiler(USceneStateMachineGraph* InGraph, IStateMachineCompilerContext& InCompilerContext)
	: TemplateData(InCompilerContext.GetTemplateData())
	, StateMachineGraph(InGraph)
	, Blueprint(FBlueprintEditorUtils::FindBlueprintForGraph(InGraph))
	, Context(InCompilerContext)
{
	check(TemplateData && StateMachineGraph);
}

FSceneStateMachine FStateMachineCompiler::Compile()
{
	FSceneStateMachine StateMachine;

	USceneStateMachineEntryNode* EntryNode = StateMachineGraph->GetEntryNode();
	if (!EntryNode)
	{
		return StateMachine;
	}

	USceneStateMachineStateNode* EntryState = EntryNode->GetStateNode();
	if (!EntryState)
	{
		return StateMachine;
	}

	StateNodesToProcess.Reset();
	StateNodesToProcess.Emplace(EntryState);

	StateMachine.EntryIndex = 0;
	StateMachine.Parameters = StateMachineGraph->Parameters;

	// Step #1: Compile States
	while (!StateNodesToProcess.IsEmpty())
	{
		if (const USceneStateMachineStateNode* StateNode = StateNodesToProcess.Pop())
		{
			CompileState(StateNode);	
		}
	}

	// Step #2: Compile Conduits
	for (const USceneStateMachineConduitNode* ConduitNode : ConduitNodesToCompile)
	{
		check(ConduitNodeIndexMap.Contains(ConduitNode));
		CompileConduit(ConduitNode);
	}

	// Step #3: Compile State Transitions
	// State array is filled with all the states, so they can be identified by indices now
	for (const TPair<FObjectKey, uint16>& Pair : StateNodeIndexMap)
	{
		const USceneStateMachineStateNode* StateNode = CastChecked<USceneStateMachineStateNode>(Pair.Key.ResolveObjectPtr());

		FSceneState& SceneState = States[Pair.Value];
		CompileStateTransitions(SceneState, StateNode);
	}

	StateMachine.StateRange.Index = TemplateData->States.Num();
	StateMachine.StateRange.Count = States.Num();

	TemplateData->States.Append(MoveTemp(States));
	TemplateData->StateMetadata.Append(MoveTemp(StateMetadata));

	States.Reset();
	StateMetadata.Reset();

	// Upgrade the map to absolute indices before baking it to the generated class' map
	ToAbsoluteIndexMap(StateNodeIndexMap, StateMachine.StateRange.Index);
	TemplateData->StateNodeToIndex.Append(MoveTemp(StateNodeIndexMap));
	StateNodeIndexMap.Reset();

	// Upgrade the map to absolute indices before baking it to the generated class' map
	ToAbsoluteIndexMap(StateParameterIdIndexMap, StateMachine.StateRange.Index);
	TemplateData->StateParameterIdToIndex.Append(MoveTemp(StateParameterIdIndexMap));
	StateParameterIdIndexMap.Reset();

	StateMachine.ConduitRange.Index = TemplateData->Conduits.Num();
	StateMachine.ConduitRange.Count = Conduits.Num();

	TemplateData->Conduits.Append(MoveTemp(Conduits));
	TemplateData->ConduitLinks.Append(MoveTemp(ConduitLinks));

	Conduits.Reset();
	ConduitLinks.Reset();

	return StateMachine;
}

void FStateMachineCompiler::CompileState(const USceneStateMachineStateNode* InStateNode)
{
	if (!InStateNode->IsNodeEnabled())
	{
		return;
	}

	if (StateNodeIndexMap.Contains(InStateNode))
	{
		return;
	}

	const int32 StateIndex = States.AddDefaulted();
	const int32 StateMetadataIndex = StateMetadata.AddDefaulted();

	check(StateIndex == StateMetadataIndex);

	FSceneState& NewState = States[StateIndex];
	NewState.Parameters = InStateNode->Parameters;

	FSceneStateMetadata& NewStateMetadata = StateMetadata[StateIndex];
	NewStateMetadata.StateName = InStateNode->GetNodeName().ToString();

	StateNodeIndexMap.Add(InStateNode, StateIndex);
	StateParameterIdIndexMap.Add(InStateNode->ParametersId, StateIndex);

	CompileSubStateMachines(NewState, InStateNode);
	CompileTasks(NewState, InStateNode);
	CompileEventHandlers(NewState, InStateNode);

	// Gather more states to process by following the linked transitions
	const TArray<USceneStateMachineTransitionNode*> TransitionNodes = InStateNode->GatherTransitions(/*bInSortList*/true);
	FollowTransitions(TransitionNodes);
}

void FStateMachineCompiler::FollowTransitions(TConstArrayView<USceneStateMachineTransitionNode*> InExitTransitions)
{
	StateNodesToProcess.Reserve(StateNodesToProcess.Num() + InExitTransitions.Num());

	for (USceneStateMachineTransitionNode* const TransitionNode : InExitTransitions)
	{
		check(TransitionNode);

		const USceneStateMachineNode* TargetNode = TransitionNode->GetTargetNode();

		if (const USceneStateMachineStateNode* TargetStateNode = Cast<USceneStateMachineStateNode>(TargetNode))
		{
			StateNodesToProcess.Add(TargetStateNode);
		}
		else if (const USceneStateMachineConduitNode* TargetConduitNode = Cast<USceneStateMachineConduitNode>(TargetNode))
		{
			AddConduitToCompile(TargetConduitNode);
		}
	}
}

void FStateMachineCompiler::CompileSubStateMachines(FSceneState& InNewState, const USceneStateMachineStateNode* InStateNode)
{
	// Compile Sub State Machine Graphs into Runtime State Machines owned by this new State
	const TArray<UEdGraph*> SubGraphs = InStateNode->GetSubGraphs();

	TArray<FSceneStateMachine> SubStateMachines;
	SubStateMachines.Reserve(SubGraphs.Num());

	TMap<FObjectKey, uint16> StateMachineGraphToIndex;
	StateMachineGraphToIndex.Reserve(SubGraphs.Num());

	TMap<FGuid, uint16> StateMachineIdToIndex;
	StateMachineIdToIndex.Reserve(SubGraphs.Num());

	for (UEdGraph* SubGraph : SubGraphs)
	{
		USceneStateMachineGraph* NewStateMachineGraph = Cast<USceneStateMachineGraph>(SubGraph);
		if (!NewStateMachineGraph)
		{
			continue;	
		}

		FStateMachineCompiler StateMachineCompiler(NewStateMachineGraph, Context);
		FSceneStateMachine NewStateMachine = StateMachineCompiler.Compile();
		if (NewStateMachine.IsValid())
		{
			const uint16 StateMachineIndex = SubStateMachines.Num();
			SubStateMachines.Emplace(MoveTemp(NewStateMachine));
			StateMachineGraphToIndex.Emplace(NewStateMachineGraph, StateMachineIndex);
			StateMachineIdToIndex.Emplace(NewStateMachineGraph->ParametersId, StateMachineIndex);
		}
	}

	InNewState.StateMachineRange.Index = TemplateData->StateMachines.Num();
	InNewState.StateMachineRange.Count = SubStateMachines.Num();
	TemplateData->StateMachines.Append(MoveTemp(SubStateMachines));

	// Upgrade the map to absolute indices before baking it to the generated class' map
	ToAbsoluteIndexMap(StateMachineGraphToIndex, InNewState.StateMachineRange.Index);
	TemplateData->StateMachineGraphToIndex.Append(MoveTemp(StateMachineGraphToIndex));

	// Upgrade the map to absolute indices before baking it to the generated class' map
	ToAbsoluteIndexMap(StateMachineIdToIndex, InNewState.StateMachineRange.Index);
	TemplateData->StateMachineIdToIndex.Append(MoveTemp(StateMachineIdToIndex));
}

void FStateMachineCompiler::AddConduitToCompile(const USceneStateMachineConduitNode* InConduitNode)
{
	if (!InConduitNode->IsNodeEnabled())
	{
		return;
	}

	// Conduit already added to nodes to process list
	if (ConduitNodeIndexMap.Contains(InConduitNode))
	{
		return;
	}

	// No exit transitions, skip compile
	TArray<USceneStateMachineTransitionNode*> ConduitTransitions = InConduitNode->GatherTransitions(/*bInSortList*/true);
	if (ConduitTransitions.IsEmpty())
	{
		return;
	}

	// Add defaulted conduit for now for indexing/discovery.
	const int32 ConduitIndex = ConduitNodesToCompile.Add(InConduitNode);
	ConduitNodeIndexMap.Add(InConduitNode, ConduitIndex);

	// Gather more nodes to process by following the exit transitions of the conduit
	FollowTransitions(ConduitTransitions);
}

void FStateMachineCompiler::CompileTasks(FSceneState& InNewState, const USceneStateMachineStateNode* InStateNode)
{
	FStateMachineTaskCompiler::FCompileResult TaskCompileResult;
	{
		FStateMachineTaskCompiler TaskCompiler(InStateNode->GetTaskPin(), /*Outer*/TemplateData);
		TaskCompiler.Compile(TaskCompileResult);
	}

	InNewState.TaskRange.Index = TemplateData->Tasks.Num();
	InNewState.TaskRange.Count = TaskCompileResult.Tasks.Num();

	// Convert the prerequisite range from relative to absolute
	const uint16 TaskPrerequisiteIndex = TemplateData->TaskPrerequisites.Num();
	for (FInstancedStruct& Task : TaskCompileResult.Tasks)
	{
		Task.GetMutable<FSceneStateTask>().PrerequisiteRange.Index += TaskPrerequisiteIndex;
	}

	TemplateData->Tasks.Append(MoveTemp(TaskCompileResult.Tasks));
	TemplateData->TaskInstances.Append(TaskCompileResult.TaskInstances);
	TemplateData->TaskMetadata.Append(MoveTemp(TaskCompileResult.TaskMetadata));
	TemplateData->TaskPrerequisites.Append(MoveTemp(TaskCompileResult.TaskPrerequisites));

	InNewState.InstanceTaskObjects(TemplateData
		, GetStructViews(TemplateData->TaskInstances, InNewState.TaskRange)
		, TaskCompileResult.TaskInstances
		, [this](FObjectDuplicationParameters& InParams)->UObject*
		{
			return this->DuplicateObject(InParams);
		});

	ToAbsoluteIndexMap(TaskCompileResult.TaskToIndexMap, InNewState.TaskRange.Index);
	TemplateData->TaskNodeToIndex.Append(MoveTemp(TaskCompileResult.TaskToIndexMap));
}

void FStateMachineCompiler::CompileEventHandlers(FSceneState& InNewState, const USceneStateMachineStateNode* InStateNode)
{
	InNewState.EventHandlerRange.Index = TemplateData->EventHandlers.Num();
	InNewState.EventHandlerRange.Count = InStateNode->EventHandlers.Num();

	TemplateData->EventHandlers.Append(InStateNode->EventHandlers);
}

void FStateMachineCompiler::FinishTransitionCompilation(FSceneStateRange& OutTransitionRange, FStateMachineTransitionCompileResult&& InCompileResult)
{
	OutTransitionRange.Index = TemplateData->Transitions.Num();
	OutTransitionRange.Count = InCompileResult.Transitions.Num();

	TemplateData->Transitions.Append(MoveTemp(InCompileResult.Transitions));
	TemplateData->TransitionLinks.Append(MoveTemp(InCompileResult.Links));
	TemplateData->TransitionMetadata.Append(MoveTemp(InCompileResult.Metadata));
	TemplateData->TransitionParameters.Append(Private::ConvertParameterMap(OutTransitionRange, InCompileResult.Parameters));
}

void FStateMachineCompiler::CompileStateTransitions(FSceneState& InNewState, const USceneStateMachineStateNode* InStateNode)
{
	FStateMachineTransitionCompileResult TransitionCompileResult;

	const FStateMachineTransitionCompiler::FCompileParams CompileParams
		{
			.Context = Context,
			.Node = InStateNode,
			.StateNodeIndexMap = StateNodeIndexMap,
			.ConduitNodeIndexMap = ConduitNodeIndexMap,
		};

	FStateMachineTransitionCompiler TransitionCompiler(CompileParams);
	TransitionCompiler.Compile(TransitionCompileResult);

	FinishTransitionCompilation(InNewState.TransitionRange, MoveTemp(TransitionCompileResult));
}

void FStateMachineCompiler::CompileConduit(const USceneStateMachineConduitNode* InConduitNode)
{
	FStateMachineConduitCompileResult ConduitCompileResult;

	const FStateMachineConduitCompiler::FCompileParams CompileParams
		{
			.Context = Context,
			.ConduitNode = InConduitNode,
			.StateNodeIndexMap = StateNodeIndexMap,
			.ConduitNodeIndexMap = ConduitNodeIndexMap,
		};

	FStateMachineConduitCompiler ConduitCompiler(CompileParams);
	if (!ConduitCompiler.Compile(ConduitCompileResult))
	{
		return;
	}

	const uint16 ConduitIndex = ConduitNodeIndexMap[InConduitNode];

	Conduits.Insert(MoveTemp(ConduitCompileResult.Conduit), ConduitIndex);
	ConduitLinks.Insert(MoveTemp(ConduitCompileResult.ConduitLink), ConduitIndex);

	FinishTransitionCompilation(Conduits[ConduitIndex].TransitionRange, MoveTemp(ConduitCompileResult.TransitionCompileResult));
}

UObject* FStateMachineCompiler::DuplicateObject(FObjectDuplicationParameters& InDuplicationParams)
{
	TMap<UObject*, UObject*> DuplicationMap;

	// if recompiling the BP on load, skip post load and defer it to the loading process
	FUObjectSerializeContext* LinkerLoadingContext = nullptr;

	if (Blueprint && Blueprint->bIsRegeneratingOnLoad)
	{
		if (FLinkerLoad* Linker = Blueprint->GetLinker())
		{
			LinkerLoadingContext = FUObjectThreadContext::Get().GetSerializeContext();
		}
		InDuplicationParams.bSkipPostLoad = true;
		InDuplicationParams.CreatedObjects = &DuplicationMap;
	}

	UObject* DuplicateObject = StaticDuplicateObjectEx(InDuplicationParams);

	// if we have anything in here after duplicate, then hook them in the loading process so they get post loaded
	if (LinkerLoadingContext)
	{
		TArray<UObject*> DupObjects;
		DuplicationMap.GenerateValueArray(DupObjects);
		LinkerLoadingContext->AddUniqueLoadedObjects(DupObjects);
	}

	return DuplicateObject;
}

} // UE::SceneState::Editor
