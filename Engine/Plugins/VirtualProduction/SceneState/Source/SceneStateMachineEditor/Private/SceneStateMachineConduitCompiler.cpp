// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateMachineConduitCompiler.h"
#include "ISceneStateMachineCompilerContext.h"
#include "Nodes/SceneStateMachineConduitNode.h"
#include "SceneStateConduitGraph.h"

namespace UE::SceneState::Editor
{

FStateMachineConduitCompiler::FStateMachineConduitCompiler(const FCompileParams& InParams)
	: Params(InParams)
{
	check(Params.ConduitNode);
}

bool FStateMachineConduitCompiler::Compile(FStateMachineConduitCompileResult& OutResult)
{
	if (!CompileConduitGraph())
	{
		return false;
	}

	CompileExitTransitions();
	OutResult = MoveTemp(Result);
	return true;
}

bool FStateMachineConduitCompiler::CompileConduitGraph()
{
	// Defaults to SkippedAlwaysTrue if there's no valid graph to compile
	FTransitionGraphCompileResult GraphCompileResult;
	GraphCompileResult.ReturnCode = ETransitionGraphCompileReturnCode::SkippedAlwaysTrue;

	// Compile conduit graph if present
	if (USceneStateConduitGraph* ConduitGraph = Cast<USceneStateConduitGraph>(Params.ConduitNode->GetBoundGraph()))
	{
		GraphCompileResult = Params.Context.CompileTransitionGraph(ConduitGraph);
	}

	// Do not create any transition if the compilation failed
	// or if the transition will always lead to a false evaluation
	if (GraphCompileResult.ReturnCode == ETransitionGraphCompileReturnCode::Failed || GraphCompileResult.ReturnCode == ETransitionGraphCompileReturnCode::SkippedAlwaysFalse)
	{
		return false;
	}

	// Transition compilation was skipped because it's going to always return true
	// mark evaluation as always true
	if (GraphCompileResult.ReturnCode == ETransitionGraphCompileReturnCode::SkippedAlwaysTrue)
	{
		Result.Conduit.EvaluationFlags |= ESceneStateTransitionEvaluationFlags::EvaluationEventAlwaysTrue;
	}

	if (Params.ConduitNode->ShouldWaitForTasksToFinish())
	{
		Result.Conduit.EvaluationFlags |= ESceneStateTransitionEvaluationFlags::WaitForTasksToFinish;
	}

	Result.ConduitLink.EventName = GraphCompileResult.EventName;
	Result.ConduitLink.ResultPropertyName = GraphCompileResult.ResultPropertyName;

	return true;
}

void FStateMachineConduitCompiler::CompileExitTransitions()
{
	const FStateMachineTransitionCompiler::FCompileParams CompileParams
		{
			.Context = Params.Context,
			.Node = Params.ConduitNode,
			.StateNodeIndexMap = Params.StateNodeIndexMap,
			.ConduitNodeIndexMap = Params.ConduitNodeIndexMap,
		};

	FStateMachineTransitionCompiler TransitionCompiler(CompileParams);
	TransitionCompiler.Compile(Result.TransitionCompileResult);
}

} // UE::SceneState::Editor
