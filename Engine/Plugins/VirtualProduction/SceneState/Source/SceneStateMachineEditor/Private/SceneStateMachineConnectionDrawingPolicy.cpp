// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateMachineConnectionDrawingPolicy.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "Nodes/SceneStateMachineEntryNode.h"
#include "Nodes/SceneStateMachineExitNode.h"
#include "Nodes/SceneStateMachineTaskNode.h"
#include "Nodes/SceneStateMachineTransitionNode.h"
#include "SceneStateMachineEditorStyle.h"

namespace UE::SceneState::Editor
{

namespace Private
{

bool IsTaskConnection(UEdGraphPin* InOutputPin, UEdGraphPin* InInputPin)
{
	UEdGraphNode* SourceNode = InInputPin ? InInputPin->GetOwningNodeUnchecked() : nullptr;
	UEdGraphNode* TargetNode = InOutputPin ? InOutputPin->GetOwningNodeUnchecked() : nullptr;

	return Cast<USceneStateMachineTaskNode>(SourceNode) || Cast<USceneStateMachineTaskNode>(TargetNode);
}

} // Private
	
FStateMachineConnectionDrawingPolicy::FStateMachineConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements)
	: FConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, InZoomFactor, InClippingRect, InDrawElements)
{
}

void FStateMachineConnectionDrawingPolicy::DetermineWiringStyle(UEdGraphPin* InOutputPin, UEdGraphPin* InInputPin, FConnectionParams& InOutParams)
{
	InOutParams.bUserFlag2 = true;
	InOutParams.AssociatedPin1 = InOutputPin;
	InOutParams.AssociatedPin2 = InInputPin;

	if (HoveredPins.Num() > 0)
	{
		ApplyHoverDeemphasis(InOutputPin, InInputPin, InOutParams.WireThickness, InOutParams.WireColor);
	}

	// Task Wiring Style
	if (Private::IsTaskConnection(InOutputPin, InInputPin))
	{
		InOutParams.WireThickness = 1.5f;
		InOutParams.WireColor = FStateMachineEditorStyle::Get().GetColor("WireColor.Task");
	}
	// Transition Wiring Style
	else
	{
		InOutParams.WireThickness = 2.f;
		InOutParams.WireColor = FStateMachineEditorStyle::Get().GetColor("WireColor.Transition");
	}

	// Make the transition that is currently relinked, semi-transparent.
	for (const FRelinkConnection& Connection : RelinkConnections)
	{
		if (!InInputPin || !InOutputPin)
		{
			continue;
		}

		const FGraphPinHandle SourcePinHandle(Connection.SourcePin);
		const FGraphPinHandle TargetPinHandle(Connection.TargetPin);

		// Skip all transitions that don't start at the node our dragged and relink transition starts from
		if (InOutputPin->GetOwningNode()->NodeGuid == SourcePinHandle.NodeGuid)
		{
			// Safety check to verify if the node is a transition node
			if (USceneStateMachineTransitionNode* TransitionNode = Cast<USceneStateMachineTransitionNode>(InInputPin->GetOwningNode()))
			{
				if (UEdGraphPin* TransitionOutputPin = TransitionNode->GetOutputPin())
				{
					if (TargetPinHandle.NodeGuid == TransitionOutputPin->GetOwningNode()->NodeGuid)
					{
						InOutParams.WireColor.A *= 0.2f;
					}
				}
			}
		}
	}
}

void FStateMachineConnectionDrawingPolicy::Draw(TMap<TSharedRef<SWidget>, FArrangedWidget>& InPinGeometries, FArrangedChildren& InArrangedNodes)
{
	const int32 ArrangeNodeCount = InArrangedNodes.Num();

	// Build an acceleration structure to quickly find geometry for the nodes
	NodeWidgetMap.Empty(ArrangeNodeCount);

	for (int32 NodeIndex = 0; NodeIndex < ArrangeNodeCount; ++NodeIndex)
	{
		TSharedRef<SGraphNode> ChildNode = StaticCastSharedRef<SGraphNode>(InArrangedNodes[NodeIndex].Widget);
		NodeWidgetMap.Add(ChildNode->GetNodeObj(), NodeIndex);
	}

	FConnectionDrawingPolicy::Draw(InPinGeometries, InArrangedNodes);
}

void FStateMachineConnectionDrawingPolicy::DetermineLinkGeometry(FArrangedChildren& InArrangedNodes
	, TSharedRef<SWidget>& InOutputPinWidget
	, UEdGraphPin* InOutputPin
	, UEdGraphPin* InInputPin
	, FArrangedWidget*& OutStartWidgetGeometry
	, FArrangedWidget*& OutEndWidgetGeometry)
{
	if (USceneStateMachineEntryNode* const EntryNode = Cast<USceneStateMachineEntryNode>(InOutputPin->GetOwningNode()))
	{
		OutStartWidgetGeometry = PinGeometries->Find(InOutputPinWidget);

		USceneStateMachineNode* const Node = CastChecked<USceneStateMachineNode>(InInputPin->GetOwningNode());
		const int32 NodeIndex = NodeWidgetMap.FindChecked(Node);

		OutEndWidgetGeometry = &(InArrangedNodes[NodeIndex]);
	}
	else if (USceneStateMachineExitNode* const ExitNode = Cast<USceneStateMachineExitNode>(InInputPin->GetOwningNode()))
	{
		OutStartWidgetGeometry = PinGeometries->Find(InOutputPinWidget);

		USceneStateMachineNode* const Node = CastChecked<USceneStateMachineNode>(InOutputPin->GetOwningNode());
		const int32 NodeIndex = NodeWidgetMap.FindChecked(Node);

		OutEndWidgetGeometry = &(InArrangedNodes[NodeIndex]);
	}
	else if (USceneStateMachineTransitionNode* const TransitionNode = Cast<USceneStateMachineTransitionNode>(InInputPin->GetOwningNode()))
	{
		USceneStateMachineNode* const SourceState = TransitionNode->GetSourceNode();
		USceneStateMachineNode* const TargetState = TransitionNode->GetTargetNode();

		if (SourceState && TargetState)
		{
			const int32* SourceStateIndex = NodeWidgetMap.Find(SourceState);
			const int32* TargetStateIndex = NodeWidgetMap.Find(TargetState);
			if (SourceStateIndex && TargetStateIndex)
			{
				OutStartWidgetGeometry = &(InArrangedNodes[*SourceStateIndex]);
				OutEndWidgetGeometry = &(InArrangedNodes[*TargetStateIndex]);
			}
		}
	}
	else
	{
		USceneStateMachineNode* SourceNode = Cast<USceneStateMachineNode>(InOutputPin->GetOwningNode());
		USceneStateMachineNode* TargetNode = Cast<USceneStateMachineNode>(InInputPin->GetOwningNode());

		const int32* SourceNodeIndex = NodeWidgetMap.Find(SourceNode);
		const int32* TargetNodeIndex = NodeWidgetMap.Find(TargetNode);

		if (SourceNodeIndex && TargetNodeIndex)
		{
			OutStartWidgetGeometry = &(InArrangedNodes[*SourceNodeIndex]);
			OutEndWidgetGeometry = &(InArrangedNodes[*TargetNodeIndex]);
		}
	}
}

void FStateMachineConnectionDrawingPolicy::DrawSplineWithArrow(const FGeometry& InStartGeometry, const FGeometry& InEndGeometry, const FConnectionParams& InParams)
{
	// Get a reasonable seed point (halfway between the boxes)
	const FVector2f StartCenter = FGeometryHelper::CenterOf(InStartGeometry);
	const FVector2f EndCenter = FGeometryHelper::CenterOf(InEndGeometry);

	const FVector2f SeedPoint = (StartCenter + EndCenter) * 0.5f;

	// Find the (approximate) closest points between the two boxes
	const FVector2f StartAnchorPoint = FGeometryHelper::FindClosestPointOnGeom(InStartGeometry, SeedPoint);
	const FVector2f EndAnchorPoint = FGeometryHelper::FindClosestPointOnGeom(InEndGeometry, SeedPoint);

	const float LineSeparationAmount = 4.f;

	const FVector2f DeltaPosition = EndAnchorPoint - StartAnchorPoint;
	const FVector2f DeltaNormal = FVector2f(DeltaPosition.Y, -DeltaPosition.X).GetSafeNormal();

	// Come up with the final start/end points
	const FVector2f DirectionBias = DeltaNormal * LineSeparationAmount;

	DrawSplineWithArrow(StartAnchorPoint + DirectionBias, EndAnchorPoint + DirectionBias, InParams);
}

void FStateMachineConnectionDrawingPolicy::DrawSplineWithArrow(const FVector2f& InStartPoint, const FVector2f& InEndPoint, const FConnectionParams& InParams)
{
	DrawArrowLine(InStartPoint, InEndPoint, InParams);
}

void FStateMachineConnectionDrawingPolicy::DrawPreviewConnector(const FGeometry& InPinGeometry, const FVector2f& InStartPoint, const FVector2f& InEndPoint, UEdGraphPin* InPin)
{
	FConnectionParams Params;
	DetermineWiringStyle(InPin, nullptr, Params);

	// bUserFlag2 is used to indicate whether the drawn arrow is a preview transition (the temporary transition when creating or relinking).
	Params.bUserFlag2 = false;
	DrawSplineWithArrow(FGeometryHelper::FindClosestPointOnGeom(InPinGeometry, InEndPoint), InEndPoint, Params);
}

FVector2f FStateMachineConnectionDrawingPolicy::ComputeSplineTangent(const FVector2f& InStart, const FVector2f& InEnd) const
{
	return (InEnd - InStart).GetSafeNormal();
}

void FStateMachineConnectionDrawingPolicy::DrawArrowLine(const FVector2f& InStartPoint, const FVector2f& InEndPoint, const FConnectionParams& InParams)
{
	const bool bIsTaskConnection = Private::IsTaskConnection(InParams.AssociatedPin1, InParams.AssociatedPin2);

	const float ArrowScale = bIsTaskConnection ? 0.75f : 1.0f;
	const FVector2f ScaledArrowRadius = ArrowRadius * ArrowScale;
	const FVector2f DeltaPosition = InEndPoint - InStartPoint;
	const FVector2f UnitDelta = DeltaPosition.GetSafeNormal();
	const FVector2f EndPoint = InEndPoint - (ScaledArrowRadius.X * UnitDelta);
	const FVector2f ArrowPosition = EndPoint - ScaledArrowRadius;
	FLinearColor ArrowHeadColor = InParams.WireColor;

	// Draw a line/spline
	DrawConnection(WireLayerID, InStartPoint, EndPoint, InParams);

	// Draw the transition grab handles in case the mouse is hovering the transition
	const FVector2f ClosestPoint = FMath::ClosestPointOnSegment2D(AbsoluteMousePosition, InStartPoint, EndPoint);

	constexpr double RelinkHandleHoverRadius = 20.0;

	if ((ClosestPoint - AbsoluteMousePosition).Length() < RelinkHandleHoverRadius)
	{
		const double RelinkHoverRadiusSq = FMath::Square(RelinkHandleHoverRadius);
		const double StartMouseSizeSq = FVector2f(InStartPoint - AbsoluteMousePosition).SizeSquared();
		const double EndMouseSizeSq = FVector2f(EndPoint - AbsoluteMousePosition).SizeSquared();

		if (EndMouseSizeSq < RelinkHoverRadiusSq)
		{
			// Set the hovered pin results. This will be used by the SGraphPanel again.
			const double SquaredDistToPin1 = InParams.AssociatedPin1 ? StartMouseSizeSq : TNumericLimits<double>::Max();
			const double SquaredDistToPin2 = InParams.AssociatedPin2 ? EndMouseSizeSq : TNumericLimits<double>::Max();

			SplineOverlapResult = FGraphSplineOverlapResult(InParams.AssociatedPin1, InParams.AssociatedPin2, SquaredDistToPin2, SquaredDistToPin1, SquaredDistToPin2, true);
		}

		// Draw grab handles only in case no relinking operation is performed
		if (RelinkConnections.IsEmpty() && InParams.bUserFlag2)
		{
			if (EndMouseSizeSq < RelinkHoverRadiusSq)
			{
				// Draw solid orange circle behind the arrow head in case the arrow head is hovered (area that enables a relink).
				const FSlateRoundedBoxBrush RoundedBoxBrush = FSlateRoundedBoxBrush(FLinearColor(0.0f, 0.0f, 0.0f, 0.0f), 9.0f, FStyleColors::AccentOrange, 100.0f);

				FSlateDrawElement::MakeBox(DrawElementsList
					, ArrowLayerID - 1 // Draw behind the arrow
					, FPaintGeometry(ArrowPosition, BubbleImage->ImageSize * ZoomFactor * ArrowScale, ZoomFactor)
					, &RoundedBoxBrush);

				ArrowHeadColor = FLinearColor::Black;
			}
			else
			{
				// Draw circle around the arrow in case the transition is hovered (mouse close or over transition line or arrow head).
				const FVector2f CircleCenter = EndPoint - UnitDelta * 2.0;
				DrawCircle(CircleCenter, /*Radius*/10.f  * ArrowScale, InParams.WireColor, /*Segments*/16);
			}
		}
	}

	const double ArrowAngle = FMath::Atan2(DeltaPosition.Y, DeltaPosition.X);

	// Draw the transition arrow triangle
	FSlateDrawElement::MakeRotatedBox(DrawElementsList
		, ArrowLayerID
		, FPaintGeometry(ArrowPosition, ScaledArrowRadius * 2.0, ZoomFactor)
		, ArrowImage
		, ESlateDrawEffect::None
		, ArrowAngle
		, TOptional<FVector2f>()
		, FSlateDrawElement::RelativeToElement
		, ArrowHeadColor
	);
}

void FStateMachineConnectionDrawingPolicy::DrawCircle(const FVector2f& InCenter, float InRadius, const FLinearColor& InColor, int32 InSegments)
{
	check(InSegments > 0);
	const float SegmentWeight = 1.f / static_cast<float>(InSegments);

	TArray<FVector2f> Points;
	Points.Reserve(InSegments + 1);

	for (int32 Index = 0; Index <= InSegments; ++Index)
	{
		const float Angle = Index * SegmentWeight * TWO_PI;
		Points.Emplace(FMath::Cos(Angle) * InRadius, FMath::Sin(Angle) * InRadius);
	}

	FSlateDrawElement::MakeLines(DrawElementsList
		, ArrowLayerID + 1
		, FPaintGeometry(InCenter, FVector2f(InRadius, InRadius) * ZoomFactor, ZoomFactor)
		, Points
		, ESlateDrawEffect::None
		, InColor
	);
}

} // UE::SceneState::Editor
