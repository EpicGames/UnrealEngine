// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConnectionDrawingPolicy.h"

namespace UE::SceneState::Editor
{

class FStateMachineConnectionDrawingPolicy : public FConnectionDrawingPolicy
{
public:
	explicit FStateMachineConnectionDrawingPolicy(int32 InBackLayerID
		, int32 InFrontLayerID
		, float InZoomFactor
		, const FSlateRect& InClippingRect
		, FSlateWindowElementList& InDrawElements);

private:
	//~ Begin FConnectionDrawingPolicy
	virtual void DetermineWiringStyle(UEdGraphPin* InOutputPin, UEdGraphPin* InInputPin, FConnectionParams& InOutParams) override;
	virtual void Draw(TMap<TSharedRef<SWidget>, FArrangedWidget>& InPinGeometries, FArrangedChildren& InArrangedNodes) override;
	virtual void DetermineLinkGeometry(FArrangedChildren& InArrangedNodes, TSharedRef<SWidget>& InOutputPinWidget, UEdGraphPin* InOutputPin, UEdGraphPin* InInputPin, FArrangedWidget*& OutStartWidgetGeometry, FArrangedWidget*& OutEndWidgetGeometry) override;
	virtual void DrawSplineWithArrow(const FGeometry& InStartGeometry, const FGeometry& InEndGeometry, const FConnectionParams& InParams) override;
	virtual void DrawSplineWithArrow(const FVector2f& InStartPoint, const FVector2f& InEndPoint, const FConnectionParams& InParams) override;
	virtual void DrawPreviewConnector(const FGeometry& InPinGeometry, const FVector2f& InStartPoint, const FVector2f& InEndPoint, UEdGraphPin* InPin) override;
	virtual FVector2f ComputeSplineTangent(const FVector2f& InStart, const FVector2f& InEnd) const override;
	// End of FConnectionDrawingPolicy interface

	void DrawArrowLine(const FVector2f& InStartPoint, const FVector2f& InEndPoint, const FConnectionParams& InParams);

	void DrawCircle(const FVector2f& InCenter, float InRadius, const FLinearColor& InColor, int32 InSegments);

	TMap<FObjectKey, int32> NodeWidgetMap;
};

} // UE::SceneState::Editor
