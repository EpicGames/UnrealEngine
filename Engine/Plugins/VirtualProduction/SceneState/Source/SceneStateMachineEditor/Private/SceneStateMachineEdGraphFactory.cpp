// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateMachineEdGraphFactory.h"
#include "KismetPins/SGraphPinExec.h"
#include "Nodes/SceneStateMachineStateNode.h"
#include "SceneStateMachineConnectionDrawingPolicy.h"
#include "SceneStateMachineGraphSchema.h"
#include "Widgets/SSceneStateMachineEntryPin.h"
#include "Widgets/SSceneStateMachineOutputPin.h"

namespace UE::SceneState::Editor
{

TSharedPtr<SGraphNode> FStateMachineEdGraphNodeFactory::CreateNode(UEdGraphNode* InNode) const
{
	if (!InNode)
	{
		return nullptr;
	}

	if (const TFunction<TSharedPtr<SGraphNode>(UEdGraphNode*)>* NodeFactory = NodeFactories.Find(InNode->GetClass()))
	{
		return (*NodeFactory)(InNode);
	}

	return nullptr;
}

TSharedPtr<SGraphPin> FStateMachineEdGraphPinFactory::CreatePin(UEdGraphPin* InPin) const
{
	check(InPin);

	const USceneStateMachineNode* const Node = Cast<USceneStateMachineNode>(InPin->GetOwningNodeUnchecked());
	if (!Node)
	{
		return nullptr;
	}

	switch (Node->GetNodeType())
	{
	case Graph::EStateMachineNodeType::Entry:
	case Graph::EStateMachineNodeType::Exit:
		// Special node pin 'rounded' look for Exit/Entry nodes
		return SNew(SStateMachineEntryPin, InPin);
	}

	return SNew(SStateMachineOutputPin, InPin);
}

FConnectionDrawingPolicy* FStateMachineEdGraphPinConnectionFactory::CreateConnectionPolicy(const UEdGraphSchema* InSchema
	, int32 InBackLayerID
	, int32 InFrontLayerID
	, float InZoomFactor
	, const FSlateRect& InClippingRect
	, FSlateWindowElementList& InDrawElements
	, UEdGraph* InGraph) const
{
	if (InSchema && InSchema->IsA<USceneStateMachineGraphSchema>())
	{
		return new FStateMachineConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, InZoomFactor, InClippingRect, InDrawElements);
	}
	return nullptr;
}

} // UE::SceneState::Editor
