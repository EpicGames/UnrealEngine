// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraphUtilities.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

namespace UE::SceneState::Editor
{

struct FStateMachineEdGraphNodeFactory : FGraphPanelNodeFactory
{
	//~ Begin FGraphPanelNodeFactory
	virtual TSharedPtr<SGraphNode> CreateNode(UEdGraphNode* InNode) const override;
	//~ End FGraphPanelNodeFactory

	template<typename InNodeType, typename InGraphNodeType>
	void RegisterDefaultNodeFactory()
	{
		NodeFactories.Add(InNodeType::StaticClass(),
			[](UEdGraphNode* InNode)->TSharedPtr<SGraphNode>
			{
				return SNew(InGraphNodeType, CastChecked<InNodeType>(InNode));
			});
	}

private:
	TMap<UClass*, TFunction<TSharedPtr<SGraphNode>(UEdGraphNode*)>> NodeFactories;
};

struct FStateMachineEdGraphPinFactory : FGraphPanelPinFactory
{
	//~ Begin FGraphPanelPinFactory
	virtual TSharedPtr<SGraphPin> CreatePin(UEdGraphPin* InPin) const override;
	//~ End FGraphPanelPinFactory
};

struct FStateMachineEdGraphPinConnectionFactory : FGraphPanelPinConnectionFactory
{
	//~ Begin FGraphPanelPinConnectionFactory
	virtual FConnectionDrawingPolicy* CreateConnectionPolicy(const UEdGraphSchema* InSchema, int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements, UEdGraph* InGraph) const override;
	//~ End FGraphPanelPinConnectionFactory
};

} // UE::SceneState::Editor
