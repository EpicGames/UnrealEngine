// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateMachineEditorModule.h"
#include "EdGraphUtilities.h"
#include "Modules/ModuleManager.h"
#include "Nodes/SceneStateMachineConduitNode.h"
#include "Nodes/SceneStateMachineEntryNode.h"
#include "Nodes/SceneStateMachineExitNode.h"
#include "Nodes/SceneStateMachineStateNode.h"
#include "Nodes/SceneStateMachineTaskNode.h"
#include "Nodes/SceneStateMachineTransitionNode.h"
#include "PropertyEditorModule.h"
#include "SceneStateMachineEdGraphFactory.h"
#include "SceneStateMachineEditorStyle.h"
#include "Widgets/SSceneStateMachineConduitNode.h"
#include "Widgets/SSceneStateMachineEntryNode.h"
#include "Widgets/SSceneStateMachineExitNode.h"
#include "Widgets/SSceneStateMachineStateNode.h"
#include "Widgets/SSceneStateMachineTaskNode.h"
#include "Widgets/SSceneStateMachineTransitionNode.h"

IMPLEMENT_MODULE(UE::SceneState::Editor::FStateStateMachineEditorModule, SceneStateMachineEditor)

namespace UE::SceneState::Editor
{

void FStateStateMachineEditorModule::StartupModule()
{
	FStateMachineEditorStyle::Get();
	RegisterGraphFactories();
}

void FStateStateMachineEditorModule::ShutdownModule()
{
	UnregisterGraphFactories();
}

void FStateStateMachineEditorModule::RegisterGraphFactories()
{
	NodeFactory = MakeShared<FStateMachineEdGraphNodeFactory>();
	FEdGraphUtilities::RegisterVisualNodeFactory(NodeFactory);

	PinFactory = MakeShared<FStateMachineEdGraphPinFactory>();
	FEdGraphUtilities::RegisterVisualPinFactory(PinFactory);

	PinConnectionFactory = MakeShared<FStateMachineEdGraphPinConnectionFactory>();
	FEdGraphUtilities::RegisterVisualPinConnectionFactory(PinConnectionFactory);

	NodeFactory->RegisterDefaultNodeFactory<USceneStateMachineStateNode, SStateMachineStateNode>();
	NodeFactory->RegisterDefaultNodeFactory<USceneStateMachineEntryNode, SStateMachineEntryNode>();
	NodeFactory->RegisterDefaultNodeFactory<USceneStateMachineExitNode, SStateMachineExitNode>();
	NodeFactory->RegisterDefaultNodeFactory<USceneStateMachineTransitionNode, SStateMachineTransitionNode>();
	NodeFactory->RegisterDefaultNodeFactory<USceneStateMachineTaskNode, SStateMachineTaskNode>();
	NodeFactory->RegisterDefaultNodeFactory<USceneStateMachineConduitNode, SStateMachineConduitNode>();
}

void FStateStateMachineEditorModule::UnregisterGraphFactories()
{
	FEdGraphUtilities::UnregisterVisualNodeFactory(NodeFactory);
	NodeFactory.Reset();

	FEdGraphUtilities::UnregisterVisualPinFactory(PinFactory);
	PinFactory.Reset();

	FEdGraphUtilities::UnregisterVisualPinConnectionFactory(PinConnectionFactory);
	PinConnectionFactory.Reset();
}

} // UE::SceneState::Editor
