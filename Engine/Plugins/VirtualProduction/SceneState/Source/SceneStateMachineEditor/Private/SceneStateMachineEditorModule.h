// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

namespace UE::SceneState::Editor
{
	struct FStateMachineEdGraphNodeFactory;
	struct FStateMachineEdGraphPinConnectionFactory;
	struct FStateMachineEdGraphPinFactory;
}

namespace UE::SceneState::Editor
{

class FStateStateMachineEditorModule : public IModuleInterface
{
	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface

	void RegisterGraphFactories();
	void UnregisterGraphFactories();

	TSharedPtr<FStateMachineEdGraphNodeFactory> NodeFactory;

	TSharedPtr<FStateMachineEdGraphPinFactory> PinFactory;

	TSharedPtr<FStateMachineEdGraphPinConnectionFactory> PinConnectionFactory;
};

} // UE::SceneState::Editor
