// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateMachineEditorStyle.h"
#include "Interfaces/IPluginManager.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"

namespace UE::SceneState::Editor
{

FStateMachineEditorStyle::FStateMachineEditorStyle()
	: FSlateStyleSet(TEXT("SceneStateMachineEditor"))
	, DefaultNumberFormat(FNumberFormattingOptions()
		.SetMinimumFractionalDigits(2)
		.SetMaximumFractionalDigits(2))
{
	ParentStyleName = FAppStyle::GetAppStyleSetName();

	ContentRootDir = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME)->GetContentDir();
	CoreContentRootDir = FPaths::EngineContentDir() / TEXT("Slate");

	RegisterGraphStyles();

	Set("SpillColor.State.Inactive", FLinearColor(0.08f, 0.08f, 0.08f));
	Set("SpillColor.State.Active", FLinearColor(1.f, 0.6f, 0.35f));
	Set("SpillColor.State.Finished", FLinearColor(0.25, 1.f, 0.25f));
	Set("SpillColor.Conduit", FLinearColor(0.38f, 0.45f, 0.21f));

	Set("SpillColor.Task.Inactive", FLinearColor(0.08f, 0.08f, 0.08f));
	Set("SpillColor.Task.Active", FLinearColor(1.f, 0.6f, 0.35f));
	Set("SpillColor.Task.Finished", FLinearColor(0.25, 1.f, 0.25f));

	Set("NodeColor.State", FLinearColor(0.6f, 0.6f, 0.6f));
	Set("NodeColor.Enter", FLinearColor(0.0f, 0.25f, 0.0f));
	Set("NodeColor.Exit", FLinearColor(0.25f, 0.0f, 0.0f));
	Set("NodeColor.Task", FLinearColor(0.08f, 0.08f, 0.3f));

	Set("WireColor.Transition", FLinearColor::White);
	Set("WireColor.Task", FLinearColor(0.3f, 0.3f, 0.3f));

	// Entry/Exit Pin styles
	Set("EntryNode.OuterBorder", new FSlateRoundedBoxBrush(FStyleColors::White, 20.0f));
	Set("EntryNode.InnerBorder", new FSlateRoundedBoxBrush(FStyleColors::White, 10.0f));
	Set("EntryNode.Shadow", new BOX_BRUSH("Graph/EntryNodeShadow", FMargin(26.0f/64.0f)));
	Set("EntryNode.ShadowSelected", new BOX_BRUSH("Graph/EntryNodeShadowSelected", FMargin(26.0f/64.0f)));
	Set("EntryPin.Normal", new FSlateRoundedBoxBrush(FStyleColors::Transparent, 20.0f));
	Set("EntryPin.Hovered", new FSlateRoundedBoxBrush(FStyleColors::White, 20.0f));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FStateMachineEditorStyle::~FStateMachineEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

void FStateMachineEditorStyle::RegisterGraphStyles()
{
	const FTextBlockStyle GraphTaskNodeTitle = FTextBlockStyle(FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("Graph.Node.NodeTitle"))
		.SetFont(DEFAULT_FONT("Regular", FCoreStyle::RegularTextSize))
		.SetColorAndOpacity(FLinearColor(0.9f, 0.9f, 0.9f))
		.SetShadowOffset(FVector2D::ZeroVector)
		.SetShadowColorAndOpacity(FLinearColor(0.f, 0.f, 0.f, 0.f));

	Set("Graph.TaskNode.Title", GraphTaskNodeTitle);

	Set("Graph.Node.NodeTitleExtraLines", FTextBlockStyle(FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("Graph.Node.NodeTitleExtraLines"))
		.SetFontSize(FCoreStyle::SmallTextSize));

	Set("Graph.TaskNode.TitleInlineEditableText", FInlineEditableTextBlockStyle(FAppStyle::Get().GetWidgetStyle<FInlineEditableTextBlockStyle>("Graph.StateNode.NodeTitleInlineEditableText"))
		.SetTextStyle(GraphTaskNodeTitle));
}

} // UE::SceneState::Editor
