// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

namespace UE::SceneState::Editor
{

class FStateMachineEditorStyle : public FSlateStyleSet
{
public:
	static FStateMachineEditorStyle& Get()
	{
		static FStateMachineEditorStyle Instance;
		return Instance;
	}

	const FNumberFormattingOptions& GetDefaultNumberFormat() const
	{
		return DefaultNumberFormat;
	}

private:
	FStateMachineEditorStyle();

	virtual ~FStateMachineEditorStyle() override;

	void RegisterGraphStyles();

	const FNumberFormattingOptions DefaultNumberFormat;
};

} // UE::SceneState::Editor
