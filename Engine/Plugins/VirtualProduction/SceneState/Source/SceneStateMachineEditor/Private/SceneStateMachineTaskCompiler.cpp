// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateMachineTaskCompiler.h"
#include "Algo/Accumulate.h"
#include "EdGraph/EdGraphPin.h"
#include "Nodes/SceneStateMachineTaskNode.h"
#include "Tasks/SceneStateTask.h"

namespace UE::SceneState::Editor
{

FStateMachineTaskCompiler::FStateMachineTaskCompiler(UEdGraphPin* InSourceOutputPin, UObject* InOuter)
	: SourceOutputPin(InSourceOutputPin)
	, Outer(InOuter)
{
	check(SourceOutputPin && Outer);
}

void FStateMachineTaskCompiler::Compile(FCompileResult& OutCompileResult)
{
	TaskInfos.Reset();

	int32 TaskCount = 0;

	GatherTasks(SourceOutputPin);

	// Keep iterating as tasks keep being added
	while (TaskCount != TaskInfos.Num())
	{
		const int32 LastTaskCount = TaskCount;
		TaskCount = TaskInfos.Num();

		for (int32 Index = LastTaskCount; Index < TaskCount; ++Index)
		{
			GatherTasks(TaskInfos[Index].Node->GetOutputPin());
		}
	}

	FCompileResult CompilationResult;
	CompileTasks(CompilationResult);
	OutCompileResult = MoveTemp(CompilationResult);
}

void FStateMachineTaskCompiler::GatherTasks(UEdGraphPin* InOutputPin)
{
	if (!InOutputPin)
	{
		return;
	}

	TArray<FTaskInfo> NewTaskInfos;
	NewTaskInfos.Reserve(InOutputPin->LinkedTo.Num());

	ProcessedNodes.Reserve(ProcessedNodes.Num() + InOutputPin->LinkedTo.Num());
	for (UEdGraphPin* Link : InOutputPin->LinkedTo)
	{
		check(Link);
		USceneStateMachineTaskNode* Node = Cast<USceneStateMachineTaskNode>(Link->GetOwningNode());
		if (!Node || ProcessedNodes.Contains(Node))
		{
			continue;
		}

		FTaskInfo TaskInfo = FStateMachineTaskCompiler::MakeTaskInfo(Node);
		if (!TaskInfo.Task.IsValid())
		{
			continue;
		}

		NewTaskInfos.Emplace(MoveTemp(TaskInfo));
		ProcessedNodes.Add(Node, INDEX_NONE);
	}

	// Sort tasks by node's position from left to right first, then top to bottom
	NewTaskInfos.Sort(
		[](const FTaskInfo& A, const FTaskInfo& B)
		{
			// First sort rule by Pos X
			if (A.Node->NodePosX != B.Node->NodePosX)
			{
				return A.Node->NodePosX < B.Node->NodePosX;
			}

			// Second sort rule by Pos Y
			if (A.Node->NodePosY != B.Node->NodePosY)
			{
				return A.Node->NodePosY < B.Node->NodePosY;
			}

			return false;
		});

	// Fill in the processed node index after sort
	const int32 StartingIndex = TaskInfos.Num();
	int32 CurrentIndex = 0;
	for (const FTaskInfo& TaskInfo : NewTaskInfos)
	{
		ProcessedNodes[TaskInfo.Node] = StartingIndex + CurrentIndex++;
	}

	TaskInfos.Append(MoveTemp(NewTaskInfos));
}

FStateMachineTaskCompiler::FTaskInfo FStateMachineTaskCompiler::MakeTaskInfo(USceneStateMachineTaskNode* InNode)
{
	FTaskInfo TaskInfo;
	TaskInfo.Node = InNode;
	TaskInfo.Task = InNode->GetTask();
	TaskInfo.TaskInstance = InNode->GetTaskInstance();
	TaskInfo.Metadata.TaskId = InNode->GetTaskId();

	UEdGraphPin* InputPin = InNode->GetInputPin();
	check(InputPin);

	TaskInfo.Prerequisites.Reserve(InputPin->LinkedTo.Num());
	for (UEdGraphPin* Link : InputPin->LinkedTo)
	{
		check(Link);
		if (USceneStateMachineTaskNode* Node = Cast<USceneStateMachineTaskNode>(Link->GetOwningNode()))
		{
			TaskInfo.Prerequisites.Add(Node);
		}
	}

	return TaskInfo;
}

void FStateMachineTaskCompiler::CompileTasks(FCompileResult& OutCompilationResult)
{
	OutCompilationResult.Tasks.Reserve(TaskInfos.Num());
	OutCompilationResult.TaskInstances.Reserve(TaskInfos.Num());
	OutCompilationResult.TaskMetadata.Reserve(TaskInfos.Num());

	// Count prerequisites and reserve ahead of time to avoid multiple reallocations
	OutCompilationResult.TaskPrerequisites.Reserve(Algo::Accumulate(TaskInfos, 0,
		[](int32 InValue, const FTaskInfo& InTaskInfo)
		{
			InValue += InTaskInfo.Prerequisites.Num();
			return InValue;
		}));

	for (const FTaskInfo& TaskInfo : TaskInfos)
	{
		FSceneStateTask& Task = OutCompilationResult.Tasks.Emplace_GetRef(TaskInfo.Task).GetMutable<FSceneStateTask>();
		OutCompilationResult.TaskInstances.Add(TaskInfo.TaskInstance);
		OutCompilationResult.TaskMetadata.Add(TaskInfo.Metadata);

		Task.PrerequisiteRange.Index = OutCompilationResult.TaskPrerequisites.Num();
		Task.PrerequisiteRange.Count = 0;

		for (const USceneStateMachineTaskNode* PrerequisiteNode : TaskInfo.Prerequisites)
		{
			if (const uint16* PrerequisiteNodeIndex = ProcessedNodes.Find(PrerequisiteNode))
			{
				OutCompilationResult.TaskPrerequisites.Add(*PrerequisiteNodeIndex);
				++Task.PrerequisiteRange.Count;
			}
		}
	}

	OutCompilationResult.TaskToIndexMap = MoveTemp(ProcessedNodes);
}

} // UE::SceneState::Editor
