// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateMachineTransitionCompiler.h"
#include "ISceneStateMachineCompilerContext.h"
#include "Nodes/SceneStateMachineNode.h"
#include "Nodes/SceneStateMachineTransitionNode.h"
#include "SceneStateTransitionGraph.h"
#include "UObject/ObjectKey.h"

namespace UE::SceneState::Editor
{

FStateMachineTransitionCompiler::FStateMachineTransitionCompiler(const FCompileParams& InParams)
	: Params(InParams)
{
	check(Params.Node);
}

void FStateMachineTransitionCompiler::Compile(FStateMachineTransitionCompileResult& OutResult)
{
	// Gather Transitions, and sort them by priority
	TArray<USceneStateMachineTransitionNode*> TransitionNodes = Params.Node->GatherTransitions(/*bSortList*/true);

	Result.Transitions.Reset(TransitionNodes.Num());
	Result.Links.Reset(TransitionNodes.Num());
	Result.Metadata.Reset(TransitionNodes.Num());
	Result.Parameters.Reset(TransitionNodes.Num());

	for (USceneStateMachineTransitionNode* TransitionNode : TransitionNodes)
	{
		if (IsNodeValid(TransitionNode))
		{
			CompileTransitionNode(TransitionNode);
		}
	}

	OutResult = MoveTemp(Result);
}

bool FStateMachineTransitionCompiler::IsNodeValid(const USceneStateMachineTransitionNode* InTransitionNode) const
{
	// Node is invalid if null or source does not match the compiling state node
	if (!ensure(InTransitionNode && InTransitionNode->GetSourceNode() == Params.Node))
	{
		return false;
	}

	// Node is invalid if it doesn't have a valid target
	const USceneStateMachineNode* TargetNode = InTransitionNode->GetTargetNode();
	if (!ensure(TargetNode))
	{
		return false;
	}

	// Node can only be valid if the target is accessible
	switch (TargetNode->GetNodeType())
	{
	case Graph::EStateMachineNodeType::State:
		return Params.StateNodeIndexMap.Contains(TargetNode);

	case Graph::EStateMachineNodeType::Exit:
		// Exit points are always valid
		return true;

	case Graph::EStateMachineNodeType::Conduit:
		return Params.ConduitNodeIndexMap.Contains(TargetNode);
	}

	return false;
}

void FStateMachineTransitionCompiler::CompileTransitionNode(const USceneStateMachineTransitionNode* InTransitionNode)
{
	// Defaults to SkippedAlwaysTrue if there's no valid graph to compile
	FTransitionGraphCompileResult GraphCompileResult;
	GraphCompileResult.ReturnCode = ETransitionGraphCompileReturnCode::SkippedAlwaysTrue;

	// Compile transition graph if present
	if (USceneStateTransitionGraph* TransitionGraph = Cast<USceneStateTransitionGraph>(InTransitionNode->GetBoundGraph()))
	{
		GraphCompileResult = Params.Context.CompileTransitionGraph(TransitionGraph);
	}

	// Do not create any transition if the compilation failed
	// or if the transition will always lead to a false evaluation
	if (GraphCompileResult.ReturnCode == ETransitionGraphCompileReturnCode::Failed || GraphCompileResult.ReturnCode == ETransitionGraphCompileReturnCode::SkippedAlwaysFalse)
	{
		return;
	}

	FSceneStateTransition& Transition = Result.Transitions.AddDefaulted_GetRef();
	Transition = BuildTransition(InTransitionNode, GraphCompileResult);

	FSceneStateTransitionLink& Link = Result.Links.AddDefaulted_GetRef();
	Link.EventName = GraphCompileResult.EventName;
	Link.ResultPropertyName = GraphCompileResult.ResultPropertyName;

	FSceneStateTransitionMetadata& Metadata = Result.Metadata.AddDefaulted_GetRef();
	Metadata.ParametersId = InTransitionNode->GetParametersId();

	Result.Parameters.Add(InTransitionNode->GetParameters());
}

FSceneStateTransition FStateMachineTransitionCompiler::BuildTransition(const USceneStateMachineTransitionNode* InTransitionNode, const FTransitionGraphCompileResult& InGraphCompileResult)
{
	FSceneStateTransition Transition;

	// Transition compilation was skipped because it's going to always return true
	// mark evaluation as always true
	if (InGraphCompileResult.ReturnCode == ETransitionGraphCompileReturnCode::SkippedAlwaysTrue)
	{
		Transition.EvaluationFlags |= ESceneStateTransitionEvaluationFlags::EvaluationEventAlwaysTrue;
	}

	if (InTransitionNode->ShouldWaitForTasksToFinish())
	{
		Transition.EvaluationFlags |= ESceneStateTransitionEvaluationFlags::WaitForTasksToFinish;
	}

	const USceneStateMachineNode* TargetNode = InTransitionNode->GetTargetNode();
	check(TargetNode);

	switch (TargetNode->GetNodeType())
	{
	case Graph::EStateMachineNodeType::State:
		Transition.Target.Type = ESceneStateTransitionTargetType::State;
		Transition.Target.Index = Params.StateNodeIndexMap[TargetNode];
		break;

	case Graph::EStateMachineNodeType::Exit:
		Transition.Target.Type = ESceneStateTransitionTargetType::Exit;
		break;

	case Graph::EStateMachineNodeType::Conduit:
		Transition.Target.Type = ESceneStateTransitionTargetType::Conduit;
		Transition.Target.Index = Params.ConduitNodeIndexMap[TargetNode];
		break;

	default:
		checkNoEntry();
	}

	return Transition;
}

} // UE::SceneState::Editor
