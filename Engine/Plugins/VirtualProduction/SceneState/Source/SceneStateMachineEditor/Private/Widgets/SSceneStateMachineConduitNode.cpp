// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSceneStateMachineConduitNode.h"
#include "Nodes/SceneStateMachineConduitNode.h"
#include "SGraphPreviewer.h"
#include "SceneStateMachineEditorStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Notifications/SErrorText.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

namespace UE::SceneState::Editor
{

void SStateMachineConduitNode::Construct(const FArguments& InArgs, USceneStateMachineConduitNode* InNode)
{
	GraphNode = InNode;
	SetCursor(EMouseCursor::CardinalCross);
	UpdateGraphNode();
}

TSharedRef<SWidget> SStateMachineConduitNode::MakeNodeInnerWidget()
{
	TSharedRef<SNodeTitle> NodeTitle = SNew(SNodeTitle, GraphNode);

	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(0)
		[
			SAssignNew(ErrorReporting, SErrorText)
			.BackgroundColor(this, &SStateMachineConduitNode::GetErrorColor)
			.ToolTipText(this, &SStateMachineConduitNode::GetErrorMsgToolTip)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(0)
		[
			SNew(SImage)
			.Image(FAppStyle::GetBrush(TEXT("Graph.ConduitNode.Icon")))
		]
		+ SHorizontalBox::Slot()
		.Padding(FMargin(5.0f, 0.0f, 5.0f, 0.0f))
		.VAlign(VAlign_Center)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Center)
			[
				SAssignNew(InlineEditableText, SInlineEditableTextBlock)
				.Style(FAppStyle::Get(), "Graph.StateNode.NodeTitleInlineEditableText")
				.Text(NodeTitle, &SNodeTitle::GetHeadTitle)
				.OnVerifyTextChanged(this, &SStateMachineConduitNode::OnVerifyNameTextChanged)
				.OnTextCommitted(this, &SStateMachineConduitNode::OnNameTextCommited)
				.IsReadOnly(this, &SStateMachineConduitNode::IsNameReadOnly)
				.IsSelected(this, &SStateMachineConduitNode::IsSelectedExclusively)
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Center)
			[
				NodeTitle
			]
		];
}

void SStateMachineConduitNode::UpdateGraphNode()
{
	Super::UpdateGraphNode();

	const FLinearColor& SpillColor = FStateMachineEditorStyle::Get().GetColor("SpillColor.Conduit");

	GetOrAddSlot(ENodeZone::Center)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Graph.StateNode.Body"))
			.BorderBackgroundColor(SpillColor)
			.Padding(0)
			[
				SNew(SOverlay)
				// Pin Area
				+ SOverlay::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SAssignNew(RightNodeBox, SVerticalBox)
				]
				// Conduit Name Area
				+ SOverlay::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Padding(10.0f)
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("Graph.StateNode.ColorSpill"))
					.BorderBackgroundColor(SpillColor)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						MakeNodeInnerWidget()
					]
				]
			]
		];

	ErrorReporting->SetError(ErrorMsg);
	CreatePinWidgets();
}

TSharedPtr<SToolTip> SStateMachineConduitNode::GetComplexTooltip()
{
	USceneStateMachineConduitNode* Node = CastChecked<USceneStateMachineConduitNode>(GraphNode);

	UEdGraph* BoundGraph = Node->GetBoundGraph();
	if (!BoundGraph)
	{
		return nullptr;
	}

	return SNew(SToolTip)
		[
			// Create the tooltip preview, ensure to disable state overlays to stop PIE and read-only borders obscuring the graph
			SNew(SGraphPreviewer, BoundGraph)
			.CornerOverlayText(FText::FromName(Node->GetNodeName()))
			.ShowGraphStateOverlay(false)
		];
}

} // UE::SceneState::Editor
