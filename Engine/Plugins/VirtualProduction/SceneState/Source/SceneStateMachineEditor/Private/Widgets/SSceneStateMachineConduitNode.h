// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SSceneStateMachineNode.h"

class USceneStateMachineConduitNode;

namespace UE::SceneState::Editor
{

class SStateMachineConduitNode : public SStateMachineNode
{
public:
	using Super = SStateMachineNode;

	SLATE_BEGIN_ARGS(SStateMachineConduitNode) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, USceneStateMachineConduitNode* InNode);

private:
	TSharedRef<SWidget> MakeNodeInnerWidget();

	//~ Begin SGraphNode
	virtual void UpdateGraphNode() override;
	virtual TSharedPtr<SToolTip> GetComplexTooltip() override;
	//~ End SGraphNode
};

} // UE::SceneState::Editor
