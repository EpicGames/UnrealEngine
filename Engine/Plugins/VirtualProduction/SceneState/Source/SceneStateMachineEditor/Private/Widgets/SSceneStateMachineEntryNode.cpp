// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSceneStateMachineEntryNode.h"
#include "Nodes/SceneStateMachineEntryNode.h"
#include "SGraphPin.h"
#include "SSceneStateMachineEntryPin.h"
#include "SceneStateMachineEditorStyle.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"

namespace UE::SceneState::Editor
{

void SStateMachineEntryNode::Construct(const FArguments& InArgs, USceneStateMachineEntryNode* InNode)
{
	GraphNode = InNode;
	SetCursor(EMouseCursor::CardinalCross);
	UpdateGraphNode();
}

void SStateMachineEntryNode::UpdateGraphNode()
{
	Super::UpdateGraphNode();

	const FStateMachineEditorStyle& Style = FStateMachineEditorStyle::Get();

	GetOrAddSlot(ENodeZone::Center)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SBorder)
			.BorderImage(Style.GetBrush("EntryNode.OuterBorder"))
			.BorderBackgroundColor(FLinearColor(0.08f, 0.08f, 0.08f))
			.Padding(0)
			[
				SNew(SOverlay)
				// Pin Area
				+ SOverlay::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SAssignNew(RightNodeBox, SVerticalBox)
				]
				+ SOverlay::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Padding(10.0f)
				[
					SNew(SBorder)
					.BorderImage(Style.GetBrush("EntryNode.InnerBorder"))
					.BorderBackgroundColor(Style.GetColor("NodeColor.Enter"))
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Padding(5, 2)
					[
						SNew(STextBlock)
						.Font(FAppStyle::GetWidgetStyle<FTextBlockStyle>("Graph.StateNode.NodeTitle").Font)
						.Text(GraphNode->GetNodeTitle(ENodeTitleType::FullTitle))
					]
				]
			]			
		];

	CreatePinWidgets();
}

const FSlateBrush* SStateMachineEntryNode::GetShadowBrush(bool bInSelected) const
{
	const FStateMachineEditorStyle& Style = FStateMachineEditorStyle::Get();

	return bInSelected
		? Style.GetBrush("EntryNode.ShadowSelected")
		: Style.GetBrush("EntryNode.Shadow");
}

} // UE::SceneState::Editor
