// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SSceneStateMachineNode.h"

class USceneStateMachineEntryNode;

namespace UE::SceneState::Editor
{

class SStateMachineEntryNode : public SStateMachineNode
{
public:
	using Super = SStateMachineNode;

	SLATE_BEGIN_ARGS(SStateMachineEntryNode) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, USceneStateMachineEntryNode* InNode);

private:
	//~ Begin SGraphNode
	virtual void UpdateGraphNode() override;
	virtual const FSlateBrush* GetShadowBrush(bool bInSelected) const override;
	//~ End SGraphNode
};

} // UE::SceneState::Editor
