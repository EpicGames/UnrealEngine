// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSceneStateMachineEntryPin.h"
#include "SceneStateMachineEditorStyle.h"

namespace UE::SceneState::Editor
{

void SStateMachineEntryPin::Construct(const FArguments& InArgs, UEdGraphPin* InPin)
{
	SetCursor(EMouseCursor::Default);

	bShowLabel = true;

	GraphPinObj = InPin;
	check(InPin);

	const UEdGraphSchema* Schema = GraphPinObj->GetSchema();
	check(Schema);

	// Set up a hover for pins that is tinted the color of the pin.
	SBorder::Construct(SBorder::FArguments()
		.BorderImage(this, &SStateMachineEntryPin::GetEntryPinBorder)
		.BorderBackgroundColor(this, &SStateMachineEntryPin::GetPinColor)
		.OnMouseButtonDown(this, &SStateMachineEntryPin::OnPinMouseDown)
		.Cursor(this, &SStateMachineEntryPin::GetPinCursor));
}

const FSlateBrush* SStateMachineEntryPin::GetEntryPinBorder() const
{
	const FStateMachineEditorStyle& Style = FStateMachineEditorStyle::Get();

	return IsHovered()
		? Style.GetBrush(TEXT("EntryPin.Hovered"))
		: Style.GetBrush(TEXT("EntryPin.Normal"));
}

} // UE::SceneState::Editor
