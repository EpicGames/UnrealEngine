// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SGraphPin.h"

namespace UE::SceneState::Editor
{

class SStateMachineEntryPin : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SStateMachineEntryPin) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InPin);

private:
	const FSlateBrush* GetEntryPinBorder() const;
};

} // UE::SceneState::Editor
