// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SSceneStateMachineNode.h"

class USceneStateMachineExitNode;

namespace UE::SceneState::Editor
{

class SStateMachineExitNode : public SStateMachineNode
{
public:
	using Super = SStateMachineNode;

	SLATE_BEGIN_ARGS(SStateMachineExitNode) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, USceneStateMachineExitNode* InNode);

private:
	//~ Begin SGraphNode
	virtual void UpdateGraphNode() override;
	virtual const FSlateBrush* GetShadowBrush(bool bInSelected) const override;
	//~ End SGraphNode
};

} // UE::SceneState::Editor
