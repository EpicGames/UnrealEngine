// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSceneStateMachineNode.h"
#include "IDocumentation.h"
#include "SGraphPin.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SToolTip.h"

namespace UE::SceneState::Editor
{

void SStateMachineNode::AddPinWidgetToSlot(const TSharedRef<SGraphPin>& InPinWidget)
{
	TSharedPtr<SVerticalBox>& NodeBox = InPinWidget->GetDirection() == EGPD_Input
		? LeftNodeBox
		: RightNodeBox;

	NodeBox->AddSlot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.FillHeight(1.0f)
		[
			InPinWidget
		];
}

void SStateMachineNode::UpdateGraphNode()
{
	if (!SWidget::GetToolTip().IsValid())
	{
		TSharedRef<SToolTip> DefaultToolTip = IDocumentation::Get()->CreateToolTip(TAttribute<FText>(this, &SGraphNode::GetNodeTooltip)
			, /*OverrideContent*/nullptr
			, GraphNode->GetDocumentationLink()
			, GraphNode->GetDocumentationExcerptName());

		SetToolTip(DefaultToolTip);
	}

	InputPins.Empty();
	OutputPins.Empty();

	RightNodeBox.Reset();
	LeftNodeBox.Reset();
}

void SStateMachineNode::CreateStandardPinWidget(UEdGraphPin* InPin)
{
	AddPin(CreatePinWidget(InPin).ToSharedRef());
}

void SStateMachineNode::AddPin(const TSharedRef<SGraphPin>& InPinWidget)
{
	InPinWidget->SetOwner(SharedThis(this));

	// Regardless of visibility always add to the pin list to get logic like FGraphSplineOverlapResult to recognize connected pins
	if (InPinWidget->GetDirection() == EGPD_Input)
	{
		InputPins.Add(InPinWidget);
	}
	else
	{
		OutputPins.Add(InPinWidget);
	}

	UEdGraphPin* const Pin = InPinWidget->GetPinObj();
	if (Pin && !Pin->bHidden)
	{
		AddPinWidgetToSlot(InPinWidget);
	}
}

} // UE::SceneState::Editor
