// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SGraphNode.h"

class USceneStateMachineNode;

namespace UE::SceneState::Editor
{

/** Base class for State Machine Graph Nodes */
class SStateMachineNode : public SGraphNode
{
protected:
	virtual void AddPinWidgetToSlot(const TSharedRef<SGraphPin>& InPinWidget);

	//~ Begin SGraphNode
	virtual void UpdateGraphNode() override;
	virtual void CreateStandardPinWidget(UEdGraphPin* InPin) override;
	virtual void AddPin(const TSharedRef<SGraphPin>& InPinWidget) override;
	//~ End SGraphNode
};

} // UE::SceneState::Editor
