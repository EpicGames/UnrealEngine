// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSceneStateMachineOutputPin.h"

namespace UE::SceneState::Editor
{

void SStateMachineOutputPin::Construct(const FArguments& InArgs, UEdGraphPin* InPin)
{
	SetCursor(EMouseCursor::Default);

	bShowLabel = true;

	GraphPinObj = InPin;
	check(InPin);

	const UEdGraphSchema* Schema = GraphPinObj->GetSchema();
	check(Schema);

	// Set up a hover for pins that is tinted the color of the pin.
	SBorder::Construct(SBorder::FArguments()
		.BorderImage(this, &SStateMachineOutputPin::GetOutputPinBorder)
		.BorderBackgroundColor(this, &SStateMachineOutputPin::GetPinColor)
		.OnMouseButtonDown(this, &SStateMachineOutputPin::OnPinMouseDown)
		.Cursor(this, &SStateMachineOutputPin::GetPinCursor));
}

TSharedRef<SWidget> SStateMachineOutputPin::GetDefaultValueWidget()
{
	return SNew(STextBlock);
}

const FSlateBrush* SStateMachineOutputPin::GetOutputPinBorder() const
{
	return IsHovered()
		? FAppStyle::GetBrush(TEXT("Graph.StateNode.Pin.BackgroundHovered"))
		: FAppStyle::GetBrush(TEXT("Graph.StateNode.Pin.Background"));
}

} // UE::SceneState::Editor
