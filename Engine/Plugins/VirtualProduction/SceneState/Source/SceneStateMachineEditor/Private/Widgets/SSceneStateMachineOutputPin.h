// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SGraphPin.h"

namespace UE::SceneState::Editor
{

class SStateMachineOutputPin : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SStateMachineOutputPin) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InPin);

private:
	//~ Begin SGraphPin
	virtual TSharedRef<SWidget> GetDefaultValueWidget() override;
	//~ End SGraphPin

	const FSlateBrush* GetOutputPinBorder() const;
};

} // UE::SceneState::Editor
