// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSceneStateMachineStateNode.h"
#include "Debug/SceneStateDebugStateContext.h"
#include "Debug/SceneStateDebugger.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Nodes/SceneStateMachineStateNode.h"
#include "SGraphPanel.h"
#include "SGraphPin.h"
#include "SGraphPreviewer.h"
#include "SSceneStateMachineOutputPin.h"
#include "SceneStateMachine.h"
#include "SceneStateMachineEditorStyle.h"
#include "SceneStateMachineGraphSchema.h"
#include "SceneStateMachineInstance.h"
#include "SceneStateObject.h"
#include "SceneStateTemplateData.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Notifications/SErrorText.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#define LOCTEXT_NAMESPACE "SSceneStateMachineStateNode"

namespace UE::SceneState::Editor
{

void SStateMachineStateNode::Construct(const FArguments& InArgs, USceneStateMachineStateNode* InNode)
{
	const FStateMachineEditorStyle& Style = FStateMachineEditorStyle::Get();

	ActiveColor = Style.GetColor("SpillColor.State.Active");
	FinishedColor = Style.GetColor("SpillColor.State.Finished");
	InactiveColor = Style.GetColor("SpillColor.State.Inactive");

	GraphNode = InNode;
	SetCursor(EMouseCursor::CardinalCross);
	UpdateGraphNode();
}

TSharedRef<SWidget> SStateMachineStateNode::MakeNodeInnerWidget()
{
	TSharedRef<SNodeTitle> NodeTitle = SNew(SNodeTitle, GraphNode);

	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(0)
		[
			SAssignNew(ErrorReporting, SErrorText)
			.BackgroundColor(this, &SStateMachineStateNode::GetErrorColor)
			.ToolTipText(this, &SStateMachineStateNode::GetErrorMsgToolTip)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(0)
		[
			SNew(SImage)
			.Image(FAppStyle::GetBrush(TEXT("Graph.StateNode.Icon")))
		]
		+ SHorizontalBox::Slot()
		.Padding(FMargin(5.0f, 0.0f, 5.0f, 0.0f))
		.VAlign(VAlign_Center)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Center)
			[
				SAssignNew(InlineEditableText, SInlineEditableTextBlock)
				.Style(FAppStyle::Get(), "Graph.StateNode.NodeTitleInlineEditableText")
				.Text(NodeTitle, &SNodeTitle::GetHeadTitle)
				.OnVerifyTextChanged(this, &SStateMachineStateNode::OnVerifyNameTextChanged)
				.OnTextCommitted(this, &SStateMachineStateNode::OnNameTextCommited)
				.IsReadOnly(this, &SStateMachineStateNode::IsNameReadOnly)
				.IsSelected(this, &SStateMachineStateNode::IsSelectedExclusively)
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Center)
			[
				NodeTitle
			]
		];
}

FSlateColor SStateMachineStateNode::GetStateBackgroundColor() const
{
	if (!StateStatus.IsSet())
	{
		return InactiveColor;
	}

	const FLinearColor* TargetColor = &InactiveColor;

	switch (*StateStatus)
	{
	case EExecutionStatus::Running:
		TargetColor = &ActiveColor;
		break;

	case EExecutionStatus::Finished:
		TargetColor = &FinishedColor;
		break;
	}

	return FMath::Lerp(InactiveColor, *TargetColor, ActivePercentage);
}

void SStateMachineStateNode::UpdateGraphNode()
{
	Super::UpdateGraphNode();

	GetOrAddSlot(ENodeZone::Center)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Graph.StateNode.Body"))
			.BorderBackgroundColor(this, &SStateMachineStateNode::GetStateBackgroundColor)
			.Padding(0)
			[
				SNew(SOverlay)
				// Pin Area
				+ SOverlay::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SAssignNew(RightNodeBox, SVerticalBox)
					+ SVerticalBox::Slot()
					.FillHeight(1.f)
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					[
						SAssignNew(OutputPinOverlay, SOverlay)
					]
				]
				// State Name Area
				+ SOverlay::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Padding(10.0f)
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("Graph.StateNode.ColorSpill"))
					.BorderBackgroundColor(FStateMachineEditorStyle::Get().GetColor("NodeColor.State"))
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						MakeNodeInnerWidget()
					]
				]
			]
		];

	ErrorReporting->SetError(ErrorMsg);
	CreatePinWidgets();
}

void SStateMachineStateNode::AddPinWidgetToSlot(const TSharedRef<SGraphPin>& InPinWidget)
{
	const UEdGraphPin* Pin = InPinWidget->GetPinObj();

	const bool IsOutputPin = Pin && Pin->GetFName() == USceneStateMachineGraphSchema::PN_Out;

	// Ensure the output pin overlays ontop of everything else
	OutputPinOverlay->AddSlot(IsOutputPin ? 1 : 0)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			InPinWidget
		];
}

TSharedPtr<SToolTip> SStateMachineStateNode::GetComplexTooltip()
{
	USceneStateMachineStateNode* Node = CastChecked<USceneStateMachineStateNode>(GraphNode);

	UEdGraph* BoundGraph = Node->GetBoundGraph();
	if (!BoundGraph)
	{
		return nullptr;
	}

	return SNew(SToolTip)
		[
			// Create the tooltip preview, ensure to disable state overlays to stop PIE and read-only borders obscuring the graph
			SNew(SGraphPreviewer, BoundGraph)
			.CornerOverlayText(FText::FromName(Node->GetNodeName()))
			.ShowGraphStateOverlay(false)
		];
}

void SStateMachineStateNode::GetNodeInfoPopups(FNodeInfoContext* InContext, TArray<FGraphInformationPopupInfo>& OutPopups) const
{
	ActivePercentage = 0.f;
	StateStatus.Reset();

	const USceneStateMachineStateNode* const StateNode = Cast<USceneStateMachineStateNode>(GraphNode);
	if (!StateNode || !StateNode->GetCompiledIndex().IsSet())
	{
		return;
	}

	const UBlueprint* const Blueprint = FBlueprintEditorUtils::FindBlueprintForNode(GraphNode);
	if (!Blueprint)
	{
		return;
	}

	const USceneStateObject* const RootState = Cast<USceneStateObject>(Blueprint->GetObjectBeingDebugged());
	if (!RootState)
	{
		return;
	}

	const TSharedPtr<UE::SceneState::FDebugger> Debugger = RootState->GetDebugger();
	if (!Debugger.IsValid())
	{
		return;
	}

	Debugger->ForEachDebugInstanceOfState(*StateNode->GetCompiledIndex(),
		[&OutPopups, This=this](const FDebugStateInstance& InDebugInstance)
		{
			const float CurrentActivePercentage = InDebugInstance.GetActivePercentage();
			This->ActivePercentage = FMath::Max(This->ActivePercentage, CurrentActivePercentage);

			switch (InDebugInstance.GetExecutionStatus())
			{
			case EExecutionStatus::Running:
				{
					This->StateStatus = EExecutionStatus::Running;

					const FText StateText = FText::Format(LOCTEXT("StateStatusFormat", "Active for {0} s")
						, FText::AsNumber(InDebugInstance.GetStatusElapsedTime(), &FStateMachineEditorStyle::Get().GetDefaultNumberFormat()));

					OutPopups.Emplace(/*Icon*/nullptr
						, FMath::Lerp(This->InactiveColor, This->ActiveColor, CurrentActivePercentage)
						, StateText.ToString());
				}
				break;

			case EExecutionStatus::Finished:
				if (!This->StateStatus.IsSet())
				{
					This->StateStatus = EExecutionStatus::Finished;
				}
				break;
			}
		});
}

void SStateMachineStateNode::OnMouseEnter(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent)
{
	// Add pins to the hover set so outgoing transitions arrows remains highlighted while the mouse is over the state node
	const USceneStateMachineStateNode* Node = CastChecked<USceneStateMachineStateNode>(GraphNode);

	TSharedPtr<SGraphPanel> OwnerPanel = GetOwnerPanel();
	check(OwnerPanel.IsValid());

	if (const UEdGraphPin* OutputPin = Node->GetOutputPin())
	{
		for (UEdGraphPin* Pin : OutputPin->LinkedTo)
		{
			OwnerPanel->AddPinToHoverSet(Pin);
		}
	}

	if (const UEdGraphPin* TaskPin = Node->GetTaskPin())
	{
		for (UEdGraphPin* Pin : TaskPin->LinkedTo)
		{
			OwnerPanel->AddPinToHoverSet(Pin);
		}
	}

	Super::OnMouseEnter(InGeometry, InMouseEvent);
}

void SStateMachineStateNode::OnMouseLeave(const FPointerEvent& InMouseEvent)
{
	// Remove manually added pins from the hover set
	const USceneStateMachineStateNode* Node = CastChecked<USceneStateMachineStateNode>(GraphNode);

	TSharedPtr<SGraphPanel> OwnerPanel = GetOwnerPanel();
	check(OwnerPanel.IsValid());

	if (const UEdGraphPin* OutputPin = Node->GetOutputPin())
	{
		for (UEdGraphPin* Pin : OutputPin->LinkedTo)
		{
			OwnerPanel->RemovePinFromHoverSet(Pin);
		}
	}

	if (const UEdGraphPin* TaskPin = Node->GetTaskPin())
	{
		for (UEdGraphPin* Pin : TaskPin->LinkedTo)
		{
			OwnerPanel->RemovePinFromHoverSet(Pin);
		}
	}

	Super::OnMouseLeave(InMouseEvent);
}

} // UE::SceneState::Editor

#undef LOCTEXT_NAMESPACE
