// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneStateEnums.h"
#include "SSceneStateMachineNode.h"

class USceneStateMachineStateNode;

namespace UE::SceneState::Editor
{

class SStateMachineStateNode : public SStateMachineNode
{
public:
	using Super = SStateMachineNode;

	SLATE_BEGIN_ARGS(SStateMachineStateNode) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, USceneStateMachineStateNode* InNode);

private:
	TSharedRef<SWidget> MakeNodeInnerWidget();

	FSlateColor GetStateBackgroundColor() const;

	//~ Begin SGraphNode
	virtual void UpdateGraphNode() override;
	virtual void AddPinWidgetToSlot(const TSharedRef<SGraphPin>& InPinWidget);
	virtual TSharedPtr<SToolTip> GetComplexTooltip() override;
	//~ End SGraphNode

	//~ Begin SNodePanel::SNode
	virtual void GetNodeInfoPopups(FNodeInfoContext* InContext, TArray<FGraphInformationPopupInfo>& OutPopups) const;
	//~ End SNodePanel::SNode

	//~ Begin SWidget
	virtual void OnMouseEnter(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent) override;
	virtual void OnMouseLeave(const FPointerEvent& InMouseEvent) override;
	//~ End SWidget

	TSharedPtr<SOverlay> OutputPinOverlay;

	FLinearColor ActiveColor = FLinearColor(ForceInit);
	FLinearColor FinishedColor = FLinearColor(ForceInit);
	FLinearColor InactiveColor = FLinearColor(ForceInit);

	/**
	 * Represents how 'active' the state is 
	 * 1 means fully active, 0 means fully inactive.
	 * When multiple instances are active, it will use the highest value.
	 */
	mutable float ActivePercentage = 0.f;

	/** Whether the state should show the finish color instead of the active color */
	mutable TOptional<EExecutionStatus> StateStatus;
};

} // UE::SceneState::Editor
