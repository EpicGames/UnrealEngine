// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSceneStateMachineTaskNode.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Nodes/SceneStateMachineTaskNode.h"
#include "SGraphPanel.h"
#include "SGraphPin.h"
#include "SGraphPreviewer.h"
#include "SSceneStateMachineOutputPin.h"
#include "SceneStateMachineEditorStyle.h"
#include "SceneStateMachineGraphSchema.h"
#include "SceneStateObject.h"
#include "SceneStateTemplateData.h"
#include "Styling/SlateIconFinder.h"
#include "Tasks/SceneStateBlueprintableTask.h"
#include "Tasks/SceneStateTaskInstance.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Notifications/SErrorText.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#define LOCTEXT_NAMESPACE "SSceneStateMachineTaskNode"

namespace UE::SceneState::Editor
{

void SStateMachineTaskNode::Construct(const FArguments& InArgs, USceneStateMachineTaskNode* InNode)
{
	GraphNode = InNode;
	TaskNodeWeak = InNode;

	const FStateMachineEditorStyle& Style = FStateMachineEditorStyle::Get();
	StatusColors =
		{
			{ EExecutionStatus::NotStarted, Style.GetColor("SpillColor.Task.Inactive") },
			{ EExecutionStatus::Running   , Style.GetColor("SpillColor.Task.Active") },
			{ EExecutionStatus::Finished  , Style.GetColor("SpillColor.Task.Finished") },
		};

	NodeTitle = SNew(SNodeTitle, GraphNode)
		.StyleSet(&Style)
		.Style(TEXT("Graph.TaskNode.Title"))
		.ExtraLineStyle(TEXT("Graph.Node.NodeTitleExtraLines"));

	InNode->OnPostEditTask().AddSP(NodeTitle.ToSharedRef(), &SNodeTitle::MarkDirty);

	SetCursor(EMouseCursor::CardinalCross);
	UpdateGraphNode();
}

SStateMachineTaskNode::~SStateMachineTaskNode()
{
	if (USceneStateMachineTaskNode* TaskNode = TaskNodeWeak.Get(/*bEvenIfPendingKill*/true))
	{
		TaskNode->OnPostEditTask().RemoveAll(this);
	}
}

TSharedRef<SWidget> SStateMachineTaskNode::MakeNodeInnerWidget()
{
	const USceneStateMachineTaskNode* Node = CastChecked<USceneStateMachineTaskNode>(GraphNode);

	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(0)
		[
			SAssignNew(ErrorReporting, SErrorText)
			.BackgroundColor(this, &SStateMachineTaskNode::GetErrorColor)
			.ToolTipText(this, &SStateMachineTaskNode::GetErrorMsgToolTip)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(0)
		[
			SNew(SImage)
			.Image(FSlateIconFinder::FindCustomIconForClass(Node->GetTask().GetScriptStruct(), TEXT("TaskIcon")).GetIcon())
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.Padding(FMargin(5.0f, 0.0f, 5.0f, 0.0f))
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Center)
			[
				SAssignNew(InlineEditableText, SInlineEditableTextBlock)
				.Style(FStateMachineEditorStyle::Get(), "Graph.TaskNode.TitleInlineEditableText")
				.Text(NodeTitle.Get(), &SNodeTitle::GetHeadTitle)
				.OnVerifyTextChanged(this, &SStateMachineTaskNode::OnVerifyNameTextChanged)
				.OnTextCommitted(this, &SStateMachineTaskNode::OnNameTextCommited)
				.IsReadOnly(this, &SStateMachineTaskNode::IsNameReadOnly)
				.IsSelected(this, &SStateMachineTaskNode::IsSelectedExclusively)
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Center)
			[
				NodeTitle.ToSharedRef()
			]
		];
}

FSlateColor SStateMachineTaskNode::GetTaskBackgroundColor() const
{
	return StatusColors[TaskStatus];
}

void SStateMachineTaskNode::UpdateGraphNode()
{
	Super::UpdateGraphNode();

	GetOrAddSlot(ENodeZone::Center)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Graph.StateNode.Body"))
			.BorderBackgroundColor(this, &SStateMachineTaskNode::GetTaskBackgroundColor)
			.Padding(0)
			[
				SNew(SOverlay)
				// Pin Area
				+ SOverlay::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SAssignNew(RightNodeBox, SVerticalBox)
				]
				// Task Name Area
				+ SOverlay::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Padding(10.f)
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("WhiteBrush"))
					.BorderBackgroundColor(FStateMachineEditorStyle::Get().GetColor("NodeColor.Task"))
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Padding(5)
					[
						MakeNodeInnerWidget()
					]
				]
			]
		];

	ErrorReporting->SetError(ErrorMsg);
	CreatePinWidgets();
}

TSharedPtr<SToolTip> SStateMachineTaskNode::GetComplexTooltip()
{
	USceneStateMachineTaskNode* Node = CastChecked<USceneStateMachineTaskNode>(GraphNode);

	UEdGraph* BoundGraph = Node->GetBoundGraph();
	if (!BoundGraph)
	{
		return nullptr;
	}

	return SNew(SToolTip)
		[
			// Create the tooltip preview, ensure to disable state overlays to stop PIE and read-only borders obscuring the graph
			SNew(SGraphPreviewer, BoundGraph)
			.CornerOverlayText(FText::FromName(Node->GetNodeName()))
			.ShowGraphStateOverlay(false)
		];
}

void SStateMachineTaskNode::GetNodeInfoPopups(FNodeInfoContext* InContext, TArray<FGraphInformationPopupInfo>& OutPopups) const
{
	TaskStatus = EExecutionStatus::NotStarted;

	const USceneStateMachineTaskNode* const Node = Cast<USceneStateMachineTaskNode>(GraphNode);
	if (!Node || !Node->GetCompiledIndex().IsSet())
	{
		return;
	}

	const UBlueprint* const Blueprint = FBlueprintEditorUtils::FindBlueprintForNode(GraphNode);
	if (!Blueprint)
	{
		return;
	}

	const USceneStateObject* const RootState = Cast<USceneStateObject>(Blueprint->GetObjectBeingDebugged());
	if (!RootState)
	{
		return;
	}

	const USceneStateTemplateData* TemplateData = RootState->GetTemplateData();
	if (!TemplateData)
	{
		return;
	}

	TOptional<EExecutionStatus> ResultStatus;

	TemplateData->ForEachTaskStatus(RootState->GetContextRegistry(), *Node->GetCompiledIndex(),
		[&OutPopups, &ResultStatus, &StatusColors = StatusColors](UE::SceneState::EExecutionStatus InTaskStatus)
		{
			// Get the lowest status to show
			if (InTaskStatus <= ResultStatus.Get(EExecutionStatus::Finished))
			{
				ResultStatus = InTaskStatus;
			}

			FText Message;
			switch (InTaskStatus)
			{
			case EExecutionStatus::Running:
				Message = LOCTEXT("RunningStatusFormat", "Running");
				break;

			case EExecutionStatus::Finished:
				Message = LOCTEXT("FinishedStatusFormat", "Finished");
				break;

			default:
				return;
			}

			OutPopups.Emplace(nullptr, StatusColors[InTaskStatus], Message.ToString());
		});

	// If the result status was never gotten (i.e. no Task Instances), set to Not-Started
	TaskStatus = ResultStatus.Get(EExecutionStatus::NotStarted);
}

void SStateMachineTaskNode::OnMouseEnter(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent)
{
	// Add pins to the hover set so outgoing transitions arrows remains highlighted while the mouse is over the state node
	const USceneStateMachineTaskNode* Node = CastChecked<USceneStateMachineTaskNode>(GraphNode);

	if (const UEdGraphPin* OutputPin = Node->GetOutputPin())
	{
		TSharedPtr<SGraphPanel> OwnerPanel = GetOwnerPanel();
		check(OwnerPanel.IsValid());

		for (UEdGraphPin* Pin : OutputPin->LinkedTo)
		{
			OwnerPanel->AddPinToHoverSet(Pin);
		}
	}

	Super::OnMouseEnter(InGeometry, InMouseEvent);
}

void SStateMachineTaskNode::OnMouseLeave(const FPointerEvent& InMouseEvent)
{
	// Remove manually added pins from the hover set
	const USceneStateMachineTaskNode* Node = CastChecked<USceneStateMachineTaskNode>(GraphNode);

	if (const UEdGraphPin* OutputPin = Node->GetOutputPin())
	{
		TSharedPtr<SGraphPanel> OwnerPanel = GetOwnerPanel();
		check(OwnerPanel.IsValid());

		for (UEdGraphPin* Pin : OutputPin->LinkedTo)
		{
			OwnerPanel->RemovePinFromHoverSet(Pin);
		}
	}

	Super::OnMouseLeave(InMouseEvent);
}

} // UE::SceneState::Editor

#undef LOCTEXT_NAMESPACE
