// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SSceneStateMachineNode.h"
#include "SceneStateEnums.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class SNodeTitle;
class USceneStateMachineTaskNode;
class USceneStateBlueprintableTask;

namespace UE::SceneState::Editor
{

class SStateMachineTaskNode : public SStateMachineNode
{
public:
	using Super = SStateMachineNode;

	SLATE_BEGIN_ARGS(SStateMachineTaskNode) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, USceneStateMachineTaskNode* InNode);

	virtual ~SStateMachineTaskNode() override;

private:
	TSharedRef<SWidget> MakeNodeInnerWidget();

	FSlateColor GetTaskBackgroundColor() const;

	//~ Begin SGraphNode
	virtual void UpdateGraphNode() override;
	virtual TSharedPtr<SToolTip> GetComplexTooltip() override;
	//~ End SGraphNode

	//~ Begin SNodePanel::SNode
	virtual void GetNodeInfoPopups(FNodeInfoContext* InContext, TArray<FGraphInformationPopupInfo>& OutPopups) const;
	//~ End SNodePanel::SNode

	//~ Begin SWidget
	void OnMouseEnter(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent) override;
	void OnMouseLeave(const FPointerEvent& InMouseEvent) override;
	//~ End SWidget

	TSharedPtr<SNodeTitle> NodeTitle;

	TWeakObjectPtr<USceneStateMachineTaskNode> TaskNodeWeak;

	TMap<EExecutionStatus, FLinearColor> StatusColors;

	mutable EExecutionStatus TaskStatus = EExecutionStatus::NotStarted;
};

} // UE::SceneState::Editor