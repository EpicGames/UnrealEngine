// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSceneStateMachineTransitionNode.h"
#include "ConnectionDrawingPolicy.h"
#include "Nodes/SceneStateMachineTransitionNode.h"
#include "SGraphPanel.h"
#include "SGraphPreviewer.h"
#include "SceneStateTransitionGraph.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SSceneStateMachineTransitionNode"

namespace UE::SceneState::Editor
{

void SStateMachineTransitionNode::Construct(const FArguments& InArgs, USceneStateMachineTransitionNode* InNode)
{
	GraphNode = InNode;
	UpdateGraphNode();
}

void SStateMachineTransitionNode::MoveTo(const FVector2f& InNewPosition, FNodeSet& InNodeFilter, bool bInMarkDirty)
{
	// Deliberately do nothing as this is a node anchored to the transition path between two state nodes
}

bool SStateMachineTransitionNode::RequiresSecondPassLayout() const
{
	// Second pass required to adjust node position to be anchored between two state nodes
	return true;
}

void SStateMachineTransitionNode::PerformSecondPassLayout(const TMap<UObject*, TSharedRef<SNode>>& InNodeToWidgetLookup) const
{
	USceneStateMachineTransitionNode* TransitionNode = CastChecked<USceneStateMachineTransitionNode>(GraphNode);

	// Find the geometry of the state nodes we're connecting
	FGeometry StartGeometry;
	FGeometry EndGeometry;

	int32 TransitionIndex = 0;
	int32 TransitionCount = 1;

	USceneStateMachineNode* SourceState = TransitionNode->GetSourceNode();
	USceneStateMachineNode* TargetState = TransitionNode->GetTargetNode();

	if (SourceState && TargetState)
	{
		const TSharedRef<SNode>* SourceNodeWidget = InNodeToWidgetLookup.Find(SourceState);
		const TSharedRef<SNode>* TargetNodeWidget = InNodeToWidgetLookup.Find(TargetState);

		if (SourceNodeWidget && TargetNodeWidget)
		{
			StartGeometry = FGeometry(SourceState->GetNodePosition(), FVector2D::ZeroVector, (*SourceNodeWidget)->GetDesiredSize(), 1.0f);
			EndGeometry   = FGeometry(TargetState->GetNodePosition(), FVector2D::ZeroVector, (*TargetNodeWidget)->GetDesiredSize(), 1.0f);

			TArray<USceneStateMachineTransitionNode*> Transitions = SourceState->GatherTransitions();

			Transitions = Transitions.FilterByPredicate(
				[TargetState](USceneStateMachineTransitionNode* InTransition) -> bool
				{
					return InTransition->GetTargetNode() == TargetState;
				});

			TransitionIndex = Transitions.IndexOfByKey(TransitionNode);
			TransitionCount = Transitions.Num();

			SourceNodeWidgetWeak = *SourceNodeWidget;
		}
	}

	// Position Node
	PositionBetweenTwoNodesWithOffset(StartGeometry, EndGeometry, TransitionIndex, TransitionCount);
}

void SStateMachineTransitionNode::UpdateGraphNode()
{
	Super::UpdateGraphNode();

	GetOrAddSlot(ENodeZone::Center)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SOverlay)
			+SOverlay::Slot()
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Graph.TransitionNode.ColorSpill"))
			]
			+SOverlay::Slot()
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Graph.TransitionNode.Icon"))
			]
		];

	CreatePinWidgets();
}

TSharedPtr<SToolTip> SStateMachineTransitionNode::GetComplexTooltip()
{
	USceneStateMachineTransitionNode* Node = CastChecked<USceneStateMachineTransitionNode>(GraphNode);

	UEdGraph* BoundGraph = Node->GetBoundGraph();
	if (!BoundGraph)
	{
		return nullptr;
	}

	return SNew(SToolTip)
		[
			// Create the tooltip preview, ensure to disable state overlays to stop PIE and read-only borders obscuring the graph
			SNew(SGraphPreviewer, BoundGraph)
			.CornerOverlayText(FText::FromName(Node->GetNodeName()))
			.ShowGraphStateOverlay(false)
		];
}

void SStateMachineTransitionNode::OnMouseEnter(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent)
{
	USceneStateMachineTransitionNode* TransitionNode = CastChecked<USceneStateMachineTransitionNode>(GraphNode);
	if (UEdGraphPin* Pin = TransitionNode->GetInputPin())
	{
		GetOwnerPanel()->AddPinToHoverSet(Pin);
	}

	Super::OnMouseEnter(InGeometry, InMouseEvent);
}

void SStateMachineTransitionNode::OnMouseLeave(const FPointerEvent& InMouseEvent)
{
	USceneStateMachineTransitionNode* TransitionNode = CastChecked<USceneStateMachineTransitionNode>(GraphNode);
	if (UEdGraphPin* Pin = TransitionNode->GetInputPin())
	{
		GetOwnerPanel()->RemovePinFromHoverSet(Pin);
	}

	Super::OnMouseLeave(InMouseEvent);
}

void SStateMachineTransitionNode::PositionBetweenTwoNodesWithOffset(const FGeometry& InStartGeometry, const FGeometry& InEndGeometry, int32 InNodeIndex, int32 InMaxNodes) const
{
	// Get a reasonable seed point (halfway between the boxes)
	const FVector2D StartCenter = FGeometryHelper::CenterOf(InStartGeometry);
	const FVector2D EndCenter   = FGeometryHelper::CenterOf(InEndGeometry);
	const FVector2D SeedPoint   = (StartCenter + EndCenter) * 0.5;

	// Find the (approximate) closest points between the two boxes
	const FVector2D StartAnchorPoint = FGeometryHelper::FindClosestPointOnGeom(InStartGeometry, SeedPoint);
	const FVector2D EndAnchorPoint   = FGeometryHelper::FindClosestPointOnGeom(InEndGeometry, SeedPoint);

	// Position ourselves halfway along the connecting line between the nodes, elevated away perpendicular to the direction of the line
	constexpr float Height = 30.f;

	const FVector2D DesiredNodeSize = GetDesiredSize();

	FVector2D DeltaPosition(EndAnchorPoint - StartAnchorPoint);
	if (DeltaPosition.IsNearlyZero())
	{
		DeltaPosition = FVector2D(10.0, 0.0);
	}

	const FVector2D Normal      = FVector2D(DeltaPosition.Y, -DeltaPosition.X).GetSafeNormal();
	const FVector2D NewCenter   = StartAnchorPoint + (0.5f * DeltaPosition) + (Height * Normal);
	const FVector2D DeltaNormal = DeltaPosition.GetSafeNormal();

	// Calculate node offset in the case of multiple transitions between the same two nodes
	// MultiNodeOffset: the offset where 0 is the centre of the transition, -1 is 1 <size of node>
	// towards the PrevStateNode and +1 is 1 <size of node> towards the NextStateNode.

	constexpr float MultiNodeSpace = 0.2f; // Space between multiple transition nodes (in units of <size of node>)
	constexpr float MultiNodeStep  = 1.0f + MultiNodeSpace; //Step between node centres (Size of node + size of node spacer)

	const float MultiNodeStart = -((InMaxNodes - 1) * MultiNodeStep) * 0.5f;
	const float MultiNodeOffset = MultiNodeStart + (InNodeIndex * MultiNodeStep);

	// Now we need to adjust the new center by the node size, zoom factor and multi node offset
	const FVector2D NewCorner = NewCenter - (0.5 * DesiredNodeSize) + (DeltaNormal * MultiNodeOffset * DesiredNodeSize.Size());

	GraphNode->NodePosX = static_cast<int32>(NewCorner.X);
	GraphNode->NodePosY = static_cast<int32>(NewCorner.Y);
}

TSharedRef<SWidget> SStateMachineTransitionNode::GenerateTooltip()
{
	return SNew(STextBlock)
		.TextStyle(FAppStyle::Get(), TEXT("Graph.TransitionNode.TooltipName"))
		.Text(GetPreviewCornerText());
}

FText SStateMachineTransitionNode::GetPreviewCornerText() const
{
	USceneStateMachineTransitionNode* TransitionNode = CastChecked<USceneStateMachineTransitionNode>(GraphNode);

	if (!TransitionNode->GetBoundGraph())
	{
		return LOCTEXT("InvalidGraphTooltip", "Error: No graph");
	}

	USceneStateMachineNode* SourceState = TransitionNode->GetSourceNode();
	USceneStateMachineNode* TargetState = TransitionNode->GetTargetNode();

	if (!SourceState || !TargetState)
	{
		return LOCTEXT("BadTransition", "Bad transition (missing source or target)");
	}

	TArray<USceneStateMachineTransitionNode*> Transitions = SourceState->GatherTransitions();

	// Show the priority number if there is any ambiguity
	for (USceneStateMachineTransitionNode* Transition : Transitions)
	{
		if (Transition->GetPriority() != TransitionNode->GetPriority())
		{
			return FText::Format(LOCTEXT("TransitionTooltipWithPriority", "{0} to {1} (Priority {2})")
				, FText::FromName(SourceState->GetNodeName())
				, FText::FromName(TargetState->GetNodeName())
				, FText::AsNumber(TransitionNode->GetPriority()));
		}
	}

	return FText::Format(LOCTEXT("TransitionTooltip", "{0} to {1}")
		, FText::FromName(SourceState->GetNodeName())
		, FText::FromName(TargetState->GetNodeName()));
}

}

#undef LOCTEXT_NAMESPACE
