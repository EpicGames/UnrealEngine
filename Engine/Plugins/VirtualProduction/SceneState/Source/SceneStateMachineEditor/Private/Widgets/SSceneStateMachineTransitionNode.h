// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SSceneStateMachineNode.h"

class USceneStateMachineTransitionNode;

namespace UE::SceneState::Editor
{

class SStateMachineTransitionNode : public SStateMachineNode
{
public:
	using Super = SStateMachineNode;

	SLATE_BEGIN_ARGS(SStateMachineTransitionNode) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, USceneStateMachineTransitionNode* InNode);

private:
	//~ Begin SNodePanel::SNode
	virtual void MoveTo(const FVector2f& InNewPosition, FNodeSet& InNodeFilter, bool bInMarkDirty) override;
	virtual bool RequiresSecondPassLayout() const override;
	virtual void PerformSecondPassLayout(const TMap<UObject*, TSharedRef<SNode>>& InNodeToWidgetLookup) const override;
	//~ End SNodePanel::SNode

	//~ Begin SGraphNode
	virtual void UpdateGraphNode() override;
	virtual TSharedPtr<SToolTip> GetComplexTooltip() override;
	//~ End SGraphNode

	//~ Begin SWidget
	void OnMouseEnter(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent) override;
	void OnMouseLeave(const FPointerEvent& InMouseEvent) override;
	//~ End SWidget

	/** Calculate position for multiple nodes to be placed between a start and end point, by providing this nodes index and max expected nodes */
	void PositionBetweenTwoNodesWithOffset(const FGeometry& InStartGeometry, const FGeometry& InEndGeometry, int32 InNodeIndex, int32 InMaxNodes) const;

	TSharedRef<SWidget> GenerateTooltip();

	FText GetPreviewCornerText() const;

	TSharedPtr<STextEntryPopup> TextEntryWidget;

	/** Cache of the widget representing the source state node */
	mutable TWeakPtr<SNode> SourceNodeWidgetWeak;

	static const FLinearColor ActiveColor;
	static const FLinearColor HoverColor;
};

} // UE::SceneState::Editor
