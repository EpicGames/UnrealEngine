// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "UObject/NameTypes.h"

class USceneStateTemplateData;
class USceneStateTransitionGraph;

namespace UE::SceneState::Editor
{

enum class ETransitionGraphCompileReturnCode : uint8
{
	/** Succeeded compiling transition graph */
	Success,

	/** Failed to compile transition graph due to errors */
	Failed,

	/** Transition graph not compiled because it's always going to evaluate to false */
	SkippedAlwaysFalse,

	/** Transition graph not compiled because it's always going to evaluate to true */
	SkippedAlwaysTrue,
};

struct FTransitionGraphCompileResult
{
	/** Return code of the graph compilation, whether it succeeded, failed, etc. */
	ETransitionGraphCompileReturnCode ReturnCode;
	/** Compiled event name to call (if any) */
	FName EventName;
	/** Name of the result property that the event name will write to when called */
	FName ResultPropertyName;
};

/** Interface to get data or execute functionality out of the scope of the compiler */
class IStateMachineCompilerContext
{
public:
	/** Gets the template data to compile the objects to */
	virtual USceneStateTemplateData* GetTemplateData() = 0;

	/** Requests compilation of the given transition graph */
	virtual FTransitionGraphCompileResult CompileTransitionGraph(USceneStateTransitionGraph* InTransitionGraph) = 0;
};

} // UE::SceneState::Editor
