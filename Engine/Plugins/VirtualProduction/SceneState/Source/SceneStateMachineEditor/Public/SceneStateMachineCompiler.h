// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Conduit/SceneStateConduit.h"
#include "Conduit/SceneStateConduitLink.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "SceneState.h"
#include "UObject/ObjectKey.h"
#include "UObject/ObjectMacros.h"

#define UE_API SCENESTATEMACHINEEDITOR_API

class UBlueprint;
class UObject;
class USceneStateMachineConduitNode;
class USceneStateMachineGraph;
class USceneStateMachineStateNode;
class USceneStateMachineTransitionNode;
class USceneStateTemplateData;
struct FSceneStateMachine;

namespace UE::SceneState::Editor
{
	class IStateMachineCompilerContext;
	struct FStateMachineTransitionCompileResult;
}

namespace UE::SceneState::Editor
{

class FStateMachineCompiler
{
public:
	UE_API explicit FStateMachineCompiler(USceneStateMachineGraph* InGraph, IStateMachineCompilerContext& InCompilerContext);

	UE_API FSceneStateMachine Compile();

private:
	/** Compiles a single state (compiles state's tasks, substate machines) */
	void CompileState(const USceneStateMachineStateNode* InStateNode);

	/** Follows the given transitions to gather more states and conduits to process */
	void FollowTransitions(TConstArrayView<USceneStateMachineTransitionNode*> InExitTransitions);

	/** Compiles the substate machines for a given state */
	void CompileSubStateMachines(FSceneState& InNewState, const USceneStateMachineStateNode* InStateNode);

	/** Adds the given conduit for later compilation */
	void AddConduitToCompile(const USceneStateMachineConduitNode* InConduitNode);

	/** Compiles the tasks connected to the state node, and writes the result into the given new state  */
	void CompileTasks(FSceneState& InNewState, const USceneStateMachineStateNode* InStateNode);

	/** Compiles the events handlers in the state node, and writes the result into the given new state  */
	void CompileEventHandlers(FSceneState& InNewState, const USceneStateMachineStateNode* InStateNode);

	/** Adds the compile result's transitions to the generated class, and returns the transition range mapping these transitions */
	void FinishTransitionCompilation(FSceneStateRange& OutTransitionRange, FStateMachineTransitionCompileResult&& InCompileResult);

	/** Compiles the transitions exiting the given state node, and writes the result into the given new state */
	void CompileStateTransitions(FSceneState& InNewState, const USceneStateMachineStateNode* InStateNode);

	/** Compiles the given conduit node, and writes the result into the given new conduit */
	void CompileConduit(const USceneStateMachineConduitNode* InConduitNode);

	UObject* DuplicateObject(FObjectDuplicationParameters& InDuplicationParams);

	/** Template data where all the compiled objects will be allocated to */
	TObjectPtr<USceneStateTemplateData> TemplateData;

	/** Reference Graph that will build out the runtime State Machine object */
	TObjectPtr<USceneStateMachineGraph> StateMachineGraph;

	/** Blueprint owning the state machine graph, if any */
	TObjectPtr<UBlueprint> Blueprint;

	/** Context used for compiling the State Machine Graph */
	IStateMachineCompilerContext& Context;

	/**
	 * Map of a State Node to the index of the States array below
	 * This is also used to avoid re-processing the same state
	 */
	TMap<FObjectKey, uint16> StateNodeIndexMap;

	/** Map of a State's parameter id to the index of the states array below */
	TMap<FGuid, uint16> StateParameterIdIndexMap;

	/** Compiled States */
	TArray<FSceneState> States;

	/** Compiled State Metadata */
	TArray<FSceneStateMetadata> StateMetadata;

	/** State nodes to process in an iteration (does not contain all the nodes) */
	TArray<const USceneStateMachineStateNode*> StateNodesToProcess;

	/** Conduit nodes to process in an iteration. Should contain all conduit nodes to compile */
	TArray<const USceneStateMachineConduitNode*> ConduitNodesToCompile;

	/**
	 * Map of a Conduit Node to the index of the States array below
	 * This is also used to avoid re-processing the same state
	 */
	TMap<FObjectKey, uint16> ConduitNodeIndexMap;

	/** Compiled Conduits */
	TArray<FSceneStateConduit> Conduits;

	/** Compiled Conduit Links */
	TArray<FSceneStateConduitLink> ConduitLinks;
};

} // UE::SceneState::Editor

#undef UE_API
