// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Conduit/SceneStateConduit.h"
#include "Conduit/SceneStateConduitLink.h"
#include "SceneStateMachineTransitionCompiler.h"

class USceneStateMachineConduitNode;
struct FObjectKey;

namespace UE::SceneState::Editor
{

struct FStateMachineConduitCompileResult
{
	/** Compiled conduit. Without the missing Transition Range (to be generated by the finish compilation process) */
	FSceneStateConduit Conduit;
	/** Compiled conduit link */
	FSceneStateConduitLink ConduitLink;
	/** Compiled conduit exit transitions */
	FStateMachineTransitionCompileResult TransitionCompileResult;
};

/** Compiles the given conduit node into transitions, links, etc */
class FStateMachineConduitCompiler
{
public:
	struct FCompileParams
	{
		/** Compiler context interface for compilation required outside the scope of this compiler */
		IStateMachineCompilerContext& Context;
		/** Conduit node to compile, along with its exit transitions */
		const USceneStateMachineConduitNode* ConduitNode;
		/** Used to look up the index, relative in state machine space, for a given state node */
		const TMap<FObjectKey, uint16>& StateNodeIndexMap;
		/** Used to look up the index, relative in state machine space, for a given conduit node */
		const TMap<FObjectKey, uint16>& ConduitNodeIndexMap;
	};
	explicit FStateMachineConduitCompiler(const FCompileParams& InParams);

	/** Compiles the conduit node */
	bool Compile(FStateMachineConduitCompileResult& OutResult);

private:
	/** Compiles the transition graph within the conduit itself (not the exit transitions') */
	bool CompileConduitGraph();

	/** Compiles the exit transitions of the conduit */
	void CompileExitTransitions();

	/** Parameters for compilation */
	const FCompileParams& Params;

	/** Result of the compilation */
	FStateMachineConduitCompileResult Result;
};

} // UE::SceneState::Editor
