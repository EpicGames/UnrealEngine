// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "StructUtils/StructView.h"
#include "Tasks/SceneStateTaskMetadata.h"
#include "UObject/ObjectKey.h"

class UBlueprint;
class UEdGraphPin;
class UObject;
class USceneStateMachineTaskNode;
struct FSceneStateTask;
struct FSceneStateTaskInstance;

namespace UE::SceneState::Editor
{

class FStateMachineTaskCompiler
{
	/** Information about a node to compile */
	struct FTaskInfo
	{
		/** View of the task in the task node to compile */
		TConstStructView<FSceneStateTask> Task;
		/** View of the task instance in the task node to compile */
		TConstStructView<FSceneStateTaskInstance> TaskInstance;
		/** Metadata compiled from the node for editor-only runtime  */
		FSceneStateTaskMetadata Metadata;
		/** The task node to compile */
		TObjectPtr<const USceneStateMachineTaskNode> Node;
		/** Prerequisite nodes that need to complete before this task executes */
		TArray<const USceneStateMachineTaskNode*> Prerequisites;
	};

public:
	explicit FStateMachineTaskCompiler(UEdGraphPin* InSourceOutputPin, UObject* InOuter);

	struct FCompileResult
	{
		/** Compiled FSceneStateTask instances */
		TArray<FInstancedStruct> Tasks;
		/** Compiled Task prerequisites */
		TArray<uint16> TaskPrerequisites;
		/** The nodes' task instance views */
		TArray<FConstStructView> TaskInstances;
		/** Additional editor-only metadata gotten from the node */
		TArray<FSceneStateTaskMetadata> TaskMetadata;
		/** Map of the task node to the index in the task array */
		TMap<FObjectKey, uint16> TaskToIndexMap;
	};
	void Compile(FCompileResult& OutCompileResult);

private:
	static FTaskInfo MakeTaskInfo(USceneStateMachineTaskNode* InNode);

	void GatherTasks(UEdGraphPin* InOutputPin);

	void CompileTasks(FCompileResult& OutCompilationResult);

	UEdGraphPin* SourceOutputPin;

	TObjectPtr<UObject> Outer;

	TArray<FTaskInfo> TaskInfos;

	TMap<FObjectKey, uint16> ProcessedNodes;
};

} // UE::SceneState::Editor
