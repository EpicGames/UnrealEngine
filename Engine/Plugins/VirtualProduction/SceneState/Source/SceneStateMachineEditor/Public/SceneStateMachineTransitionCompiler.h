// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ContainersFwd.h"
#include "Transition/SceneStateTransition.h"
#include "Transition/SceneStateTransitionLink.h"
#include "Transition/SceneStateTransitionMetadata.h"

class USceneStateMachineNode;
class USceneStateMachineTransitionNode;
struct FObjectKey;

namespace UE::SceneState
{
	namespace Editor
	{
		class IStateMachineCompilerContext;
		struct FTransitionGraphCompileResult;
	}

	namespace Graph
	{
		enum class EStateMachineNodeType : uint8;	
	}
}

namespace UE::SceneState::Editor
{

struct FStateMachineTransitionCompileResult
{
	TArray<FSceneStateTransition> Transitions;
	TArray<FSceneStateTransitionLink> Links;
	TArray<FSceneStateTransitionMetadata> Metadata;
	TArray<FInstancedPropertyBag> Parameters;
};

/** Compiles the exit transitions of a given state node into a series of arrays */
class FStateMachineTransitionCompiler
{
public:
	struct FCompileParams
	{
		/** Compiler context interface for compilation required outside the scope of this compiler */
		IStateMachineCompilerContext& Context;
		/** Node containing the exit transitions to compile */
		const USceneStateMachineNode* Node;
		/** Used to look up the index, relative in state machine space, for a given state node */
		const TMap<FObjectKey, uint16>& StateNodeIndexMap;
		/** Used to look up the index, relative in state machine space, for a given conduit node */
		const TMap<FObjectKey, uint16>& ConduitNodeIndexMap;
	};
	explicit FStateMachineTransitionCompiler(const FCompileParams& InParams);

	void Compile(FStateMachineTransitionCompileResult& OutResult);

private:
	/** Determines whether the given node has a valid target */
	bool IsNodeValid(const USceneStateMachineTransitionNode* InTransitionNode) const;

	/** Compiles a single transition node */
	void CompileTransitionNode(const USceneStateMachineTransitionNode* InTransitionNode);

	/** Builds the transition target based on the transition node's target node. Does not deal with the Transition Target yet */
	FSceneStateTransition BuildTransition(const USceneStateMachineTransitionNode* InTransitionNode, const FTransitionGraphCompileResult& InGraphCompileResult);

	/** Parameters for compilation */
	const FCompileParams& Params;

	/** Result of the compilation */
	FStateMachineTransitionCompileResult Result;
};

} // UE::SceneState::Editor
