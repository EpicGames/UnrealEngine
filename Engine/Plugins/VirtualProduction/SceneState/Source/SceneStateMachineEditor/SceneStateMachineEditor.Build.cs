// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SceneStateMachineEditor : ModuleRules
{
    public SceneStateMachineEditor(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "Core",
                "CoreUObject",
                "Engine",
                "SceneStateMachineGraph",
            }
        );

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "GraphEditor",
                "Projects",
                "PropertyEditor",
                "SceneState",
                "SceneStateTransitionGraph",
                "Slate",
                "SlateCore",
                "ToolMenus",
                "UnrealEd",
            }
        );
    }
}
