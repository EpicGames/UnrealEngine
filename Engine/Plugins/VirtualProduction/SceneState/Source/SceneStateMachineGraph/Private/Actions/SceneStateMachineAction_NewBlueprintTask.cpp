// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateMachineAction_NewBlueprintTask.h"
#include "Engine/Blueprint.h"
#include "Nodes/SceneStateMachineTaskNode.h"
#include "SceneStateMachineAction_NewNode.h"
#include "Tasks/SceneStateBlueprintableTask.h"

namespace UE::SceneState::Graph
{

FStateMachineAction_NewBlueprintTask::FStateMachineAction_NewBlueprintTask(const FAssetData& InTaskAsset, int32 InSectionId)
	: BlueprintTaskAsset(InTaskAsset)
{
	SectionID = InSectionId;

	auto GetTag = [&InTaskAsset](FName InTagKey, const FText& InDefaultValue)
		{
			FString TagValue;
			if (InTaskAsset.GetTagValue(InTagKey, TagValue))
			{
				return FText::FromString(TagValue);
			}
			return InDefaultValue;
		};

	const FText AssetNameText = FText::FromName(InTaskAsset.AssetName);
	UpdateSearchData(GetTag(FBlueprintTags::BlueprintDisplayName, AssetNameText)
		, GetTag(FBlueprintTags::BlueprintDescription, FText::GetEmpty())
		, GetTag(FBlueprintTags::BlueprintCategory, FText::GetEmpty())
		, FText::GetEmpty());
}

UEdGraphNode* FStateMachineAction_NewBlueprintTask::PerformAction(UEdGraph* InParentGraph, UEdGraphPin* InSourcePin, const FVector2f& InLocation, bool bInSelectNewNode)
{
	TSubclassOf<USceneStateBlueprintableTask> TaskBlueprintClass = ResolveBlueprintTaskClass();
	if (!TaskBlueprintClass)
	{
		return nullptr;
	}

	USceneStateMachineTaskNode* TaskNodeTemplate = NewObject<USceneStateMachineTaskNode>();
	TaskNodeTemplate->SetTaskBlueprintClass(TaskBlueprintClass);

	return FStateMachineAction_NewNode::SpawnNode<USceneStateMachineTaskNode>(InParentGraph, TaskNodeTemplate, InSourcePin, InLocation);
}

TSubclassOf<USceneStateBlueprintableTask> FStateMachineAction_NewBlueprintTask::ResolveBlueprintTaskClass() const
{
	UObject* TaskObject = BlueprintTaskAsset.GetAsset();
	if (!TaskObject)
	{
		return nullptr;
	}

	if (TSubclassOf<USceneStateBlueprintableTask> TaskClass = Cast<UClass>(TaskObject))
	{
		return TaskClass;
	}

	if (UBlueprint* Blueprint = Cast<UBlueprint>(TaskObject))
	{
		return Cast<UClass>(Blueprint->GeneratedClass);
	}

	return nullptr;
}

} // UE::SceneState::Graph
