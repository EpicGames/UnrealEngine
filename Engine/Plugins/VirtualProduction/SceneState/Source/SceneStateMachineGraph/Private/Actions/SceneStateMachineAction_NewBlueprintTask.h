// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "EdGraph/EdGraphSchema.h"

class USceneStateBlueprintableTask;

namespace UE::SceneState::Graph
{

/** Adds a new task node with a set blueprint class to the State Machine Graph */
struct FStateMachineAction_NewBlueprintTask : public FEdGraphSchemaAction
{
	FStateMachineAction_NewBlueprintTask() = default;

	explicit FStateMachineAction_NewBlueprintTask(const FAssetData& InBlueprintTaskAsset, int32 InSectionId);

	//~ Begin FEdGraphSchemaAction
	virtual UEdGraphNode* PerformAction(UEdGraph* InParentGraph, UEdGraphPin* InSourcePin, const FVector2f& InLocation, bool bInSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction

	TSubclassOf<USceneStateBlueprintableTask> ResolveBlueprintTaskClass() const;

private:
	FAssetData BlueprintTaskAsset;
};

} // UE::SceneState::Graph
