// Copyright Epic Games, Inc. All Rights Reserved.

#include "Actions/SceneStateMachineAction_NewComment.h"
#include "Actions/SceneStateMachineAction_NewNode.h"
#include "EdGraphNode_Comment.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"

namespace UE::SceneState::Graph
{

FStateMachineAction_NewComment::FStateMachineAction_NewComment(const FText& InNodeCategory, const FText& InMenuDesc, const FText& InToolTip, int32 InGrouping)
	: FEdGraphSchemaAction(InNodeCategory, InMenuDesc, InToolTip, InGrouping)
{
}

UEdGraphNode* FStateMachineAction_NewComment::PerformAction(UEdGraph* InParentGraph, UEdGraphPin* InSourcePin, const FVector2f& InLocation, bool bInSelectNewNode)
{
	UEdGraphNode_Comment* CommentTemplate = NewObject<UEdGraphNode_Comment>();

	FVector2f SpawnLocation = InLocation;

	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(InParentGraph);

	FSlateRect Bounds;
	if (Blueprint && FKismetEditorUtilities::GetBoundsForSelectedNodes(Blueprint, Bounds, 50.0f))
	{
		CommentTemplate->SetBounds(Bounds);
		SpawnLocation.X = CommentTemplate->NodePosX;
		SpawnLocation.Y = CommentTemplate->NodePosY;
	}

	return FStateMachineAction_NewNode::SpawnNode<UEdGraphNode_Comment>(InParentGraph, CommentTemplate, InSourcePin, SpawnLocation);
}

} // UE::SceneState::Graph
