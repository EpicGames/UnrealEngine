// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraphSchema.h"

namespace UE::SceneState::Graph
{

/** Adds a new comment to the State Machine Graph */
struct FStateMachineAction_NewComment : public FEdGraphSchemaAction
{
	FStateMachineAction_NewComment() = default;

	explicit FStateMachineAction_NewComment(const FText& InNodeCategory, const FText& InMenuDesc, const FText& InToolTip, int32 InGrouping);

	//~ Begin FEdGraphSchemaAction
	virtual UEdGraphNode* PerformAction(UEdGraph* InParentGraph, UEdGraphPin* InSourcePin, const FVector2f& InLocation, bool bInSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction
};

} // UE::SceneState::Graph
