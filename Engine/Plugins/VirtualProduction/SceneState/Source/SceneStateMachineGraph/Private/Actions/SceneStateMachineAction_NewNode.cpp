// Copyright Epic Games, Inc. All Rights Reserved.

#include "Actions/SceneStateMachineAction_NewNode.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "ScopedTransaction.h"
#include "Settings/EditorStyleSettings.h"

#define LOCTEXT_NAMESPACE "SceneStateMachineAction_NewNode"

namespace UE::SceneState::Graph
{

FStateMachineAction_NewNode::FStateMachineAction_NewNode(UEdGraphNode* InTemplateNode)
	: TemplateNode(InTemplateNode)
{
}

FStateMachineAction_NewNode::FStateMachineAction_NewNode(UEdGraphNode* InTemplateNode, const FText& InNodeCategory, const FText& InMenuDesc, const FText& InTooltip, int32 InGrouping)
	: FEdGraphSchemaAction(InNodeCategory, InMenuDesc, InTooltip, InGrouping)
	, TemplateNode(InTemplateNode)
{
}

UEdGraphNode* FStateMachineAction_NewNode::PerformAction(UEdGraph* InParentGraph, UEdGraphPin* InSourcePin, const FVector2f& InLocation, bool bInSelectNewNode)
{
	if (!TemplateNode || !InParentGraph)
	{
		return nullptr;
	}

	const FScopedTransaction Transaction(LOCTEXT("AddNode", "Add Node"));
	InParentGraph->Modify();

	if (InSourcePin)
	{
		InSourcePin->Modify();
	}

	UEdGraphNode* ResultNode = DuplicateObject<UEdGraphNode>(TemplateNode, InParentGraph);
	check(ResultNode);
	ResultNode->SetFlags(RF_Transactional);

	InParentGraph->AddNode(ResultNode, /*bUserAction*/true, bInSelectNewNode);

	ResultNode->CreateNewGuid();
	ResultNode->PostPlacedNewNode();
	ResultNode->AllocateDefaultPins();
	ResultNode->AutowireNewNode(InSourcePin);

	ResultNode->NodePosX = static_cast<int32>(InLocation.X);
	ResultNode->NodePosY = static_cast<int32>(InLocation.Y);
	ResultNode->SnapToGrid(GetDefault<UEditorStyleSettings>()->GridSnapSize);

	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraphChecked(InParentGraph);
	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);

	return ResultNode;
}

void FStateMachineAction_NewNode::AddReferencedObjects(FReferenceCollector& InCollector)
{
	FEdGraphSchemaAction::AddReferencedObjects(InCollector);
	InCollector.AddReferencedObject(TemplateNode);
}

} // UE::SceneState::Graph

#undef LOCTEXT_NAMESPACE
