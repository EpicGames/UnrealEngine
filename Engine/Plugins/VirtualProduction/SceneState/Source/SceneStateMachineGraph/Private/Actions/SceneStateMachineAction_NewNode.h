// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraphSchema.h"

namespace UE::SceneState::Graph
{

/** Adds a new node to the State Machine Graph */
struct FStateMachineAction_NewNode : public FEdGraphSchemaAction
{
	FStateMachineAction_NewNode() = default;

	explicit FStateMachineAction_NewNode(UEdGraphNode* InTemplateNode);

	explicit FStateMachineAction_NewNode(UEdGraphNode* InTemplateNode, const FText& InNodeCategory, const FText& InMenuDesc, const FText& InTooltip, int32 InGrouping);

	//~ Begin FEdGraphSchemaAction
	virtual UEdGraphNode* PerformAction(UEdGraph* InParentGraph, UEdGraphPin* InSourcePin, const FVector2f& InLocation, bool bInSelectNewNode = true) override;
	virtual void AddReferencedObjects(FReferenceCollector& InCollector) override;
	//~ End FEdGraphSchemaAction

	template <typename InNodeType>
	static InNodeType* SpawnNode(UEdGraph* InParentGraph, InNodeType* InNewNode, UEdGraphPin* InSourcePin, const UE::Slate::FDeprecateVector2DParameter& InLocation = FVector2f::ZeroVector, bool bInSelectNewNode = true)
	{
		return Cast<InNodeType>(FStateMachineAction_NewNode(InNewNode).PerformAction(InParentGraph, InSourcePin, InLocation, bInSelectNewNode));
	}

private:
	TObjectPtr<UEdGraphNode> TemplateNode;
};

} // UE::SceneState::Graph
