// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateMachineAction_NewTask.h"
#include "Nodes/SceneStateMachineTaskNode.h"
#include "SceneStateMachineAction_NewNode.h"

namespace UE::SceneState::Graph
{

FStateMachineAction_NewTask::FStateMachineAction_NewTask(const UScriptStruct* InTaskStruct, int32 InSectionId)
	: TaskStruct(InTaskStruct)
{
	SectionID = InSectionId;

	UpdateSearchData(InTaskStruct->GetDisplayNameText()
		, InTaskStruct->GetToolTipText()
		, FText::FromString(InTaskStruct->GetMetaData(TEXT("Category")))
		, FText::GetEmpty());
}

UEdGraphNode* FStateMachineAction_NewTask::PerformAction(UEdGraph* InParentGraph, UEdGraphPin* InSourcePin, const FVector2f& InLocation, bool bInSelectNewNode)
{
	USceneStateMachineTaskNode* TaskNodeTemplate = NewObject<USceneStateMachineTaskNode>();
	TaskNodeTemplate->SetTaskStruct(TaskStruct);

	return FStateMachineAction_NewNode::SpawnNode<USceneStateMachineTaskNode>(InParentGraph, TaskNodeTemplate, InSourcePin, InLocation);
}

void FStateMachineAction_NewTask::AddReferencedObjects(FReferenceCollector& InCollector)
{
	FEdGraphSchemaAction::AddReferencedObjects(InCollector);
	InCollector.AddReferencedObject(TaskStruct);
}

} // UE::SceneState::Graph
