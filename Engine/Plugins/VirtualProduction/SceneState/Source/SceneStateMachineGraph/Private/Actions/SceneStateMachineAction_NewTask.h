// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraphSchema.h"

namespace UE::SceneState::Graph
{

/** Adds a new task node to the State Machine Graph */
struct FStateMachineAction_NewTask : public FEdGraphSchemaAction
{
	FStateMachineAction_NewTask() = default;

	explicit FStateMachineAction_NewTask(const UScriptStruct* InTaskStruct, int32 InSectionId);

	//~ Begin FEdGraphSchemaAction
	virtual UEdGraphNode* PerformAction(UEdGraph* InParentGraph, UEdGraphPin* InSourcePin, const FVector2f& InLocation, bool bInSelectNewNode = true) override;
	virtual void AddReferencedObjects(FReferenceCollector& InCollector) override;
	//~ End FEdGraphSchemaAction

private:
	TObjectPtr<const UScriptStruct> TaskStruct;
};

} // UE::SceneState::Graph
