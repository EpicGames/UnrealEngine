// Copyright Epic Games, Inc. All Rights Reserved.

#include "Actions/SceneStateMachineNodeAction.h"
#include "Nodes/SceneStateMachineNode.h"

#define LOCTEXT_NAMESPACE "SceneStateBlueprintAction_StateNode"

namespace UE::SceneState::Graph
{

FStateMachineAction_Node::FStateMachineAction_Node(const FArguments& InArgs)
{
	NodeWeak = InArgs.Node;
	SectionID = InArgs.SectionID;

	if (InArgs.Node)
	{
		UpdateSearchData(FText::FromName(InArgs.Node->GetNodeName()), FText(),  InArgs.Category, FText());
	}
}

FName FStateMachineAction_Node::StaticGetTypeId()
{
	static FName Type = TEXT("UE::SceneState::Graph::FBlueprintAction_StateMachineNode");
	return Type;
}

bool FStateMachineAction_Node::IsParentable() const
{
	return true;
}

bool FStateMachineAction_Node::CanBeRenamed() const
{
	return false;
}

FName FStateMachineAction_Node::GetTypeId() const
{
	return StaticGetTypeId();
}

FEdGraphSchemaActionDefiningObject FStateMachineAction_Node::GetPersistentItemDefiningObject() const
{
	if (USceneStateMachineNode* Node = NodeWeak.Get())
	{
		return FEdGraphSchemaActionDefiningObject(Node->GetOuter());
	}
	return FEdGraphSchemaActionDefiningObject(nullptr);
}

} // UE::SceneState::Graph

#undef LOCTEXT_NAMESPACE
