// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/SceneStateMachineConduitNode.h"
#include "EdGraphUtilities.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "SceneStateConduitGraph.h"
#include "SceneStateConduitGraphSchema.h"
#include "SceneStateMachineGraphSchema.h"

USceneStateMachineConduitNode::USceneStateMachineConduitNode()
{
	NodeName = TEXT("Conduit");
	NodeType = UE::SceneState::Graph::EStateMachineNodeType::Conduit;

	bCanRenameNode = true;
}

UEdGraph* USceneStateMachineConduitNode::CreateBoundGraphInternal()
{
	UEdGraph* NewGraph = FBlueprintEditorUtils::CreateNewGraph(this
		, NAME_None
		, USceneStateConduitGraph::StaticClass()
		, USceneStateConduitGraphSchema::StaticClass());

	check(NewGraph);

	FEdGraphUtilities::RenameGraphToNameOrCloseToName(NewGraph, TEXT("ConduitGraph"));
	return NewGraph;
}

void USceneStateMachineConduitNode::AllocateDefaultPins()
{
	UEdGraphPin* InputPin = CreatePin(EGPD_Input, USceneStateMachineGraphSchema::PC_Transition, USceneStateMachineGraphSchema::PN_In);
	check(InputPin);
	InputPin->bHidden = true;

	CreatePin(EGPD_Output, USceneStateMachineGraphSchema::PC_Transition, USceneStateMachineGraphSchema::PN_Out);
}

bool USceneStateMachineConduitNode::CanDuplicateNode() const
{
	return true;
}

void USceneStateMachineConduitNode::PostPasteNode()
{
	GenerateNewNodeName();

	// fail-safe, create empty conduit graph
	ConditionallyCreateBoundGraph();
	check(GetBoundGraph());

	Super::PostPasteNode();
}

void USceneStateMachineConduitNode::PostPlacedNewNode()
{
	Super::PostPlacedNewNode();
	GenerateNewNodeName();
	ConditionallyCreateBoundGraph();
}

FText USceneStateMachineConduitNode::GetTitle() const
{
	return GetNodeTitle(ENodeTitleType::Type::MenuTitle);
}

bool USceneStateMachineConduitNode::IsBoundToGraphLifetime(UEdGraph& InGraph) const
{
	return &InGraph == GetBoundGraph();
}

UEdGraphNode* USceneStateMachineConduitNode::AsNode()
{
	return this;
}
