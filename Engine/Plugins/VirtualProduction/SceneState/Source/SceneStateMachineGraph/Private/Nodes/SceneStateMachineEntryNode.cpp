// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/SceneStateMachineEntryNode.h"
#include "EdGraph/EdGraph.h"
#include "EdGraphSchema_K2.h"
#include "Nodes/SceneStateMachineStateNode.h"
#include "SceneStateMachineGraphSchema.h"

USceneStateMachineEntryNode::USceneStateMachineEntryNode()
{
	NodeName = TEXT("Entry");
	NodeType = UE::SceneState::Graph::EStateMachineNodeType::Entry;
}

USceneStateMachineStateNode* USceneStateMachineEntryNode::GetStateNode() const
{
	UEdGraphPin* OutputPin = GetOutputPin();
	if (!OutputPin || OutputPin->LinkedTo.IsEmpty())
	{
		return nullptr;
	}

	ensure(OutputPin->LinkedTo.Num() == 1);

	if (UEdGraphPin* LinkedPin = OutputPin->LinkedTo[0])
	{
		return Cast<USceneStateMachineStateNode>(LinkedPin->GetOwningNode());
	}

	return nullptr;
}

UEdGraphPin* USceneStateMachineEntryNode::GetInputPin() const
{
	return nullptr;
}

UEdGraphPin* USceneStateMachineEntryNode::GetOutputPin() const
{
	return Pins[0];
}

bool USceneStateMachineEntryNode::HasValidPins() const
{
	return !!GetOutputPin();
}

void USceneStateMachineEntryNode::AllocateDefaultPins()
{
	CreatePin(EGPD_Output, USceneStateMachineGraphSchema::PC_Transition, TEXT("Entry"));
}
