// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/SceneStateMachineExitNode.h"
#include "SceneStateMachineGraphSchema.h"

USceneStateMachineExitNode::USceneStateMachineExitNode()
{
	NodeName = TEXT("Exit");
	NodeType = UE::SceneState::Graph::EStateMachineNodeType::Exit;
}

UEdGraphPin* USceneStateMachineExitNode::GetInputPin() const
{
	return Pins[0];
}

UEdGraphPin* USceneStateMachineExitNode::GetOutputPin() const
{
	return nullptr;
}

bool USceneStateMachineExitNode::HasValidPins() const
{
	return !!GetInputPin();
}

void USceneStateMachineExitNode::AllocateDefaultPins()
{
	CreatePin(EGPD_Input, USceneStateMachineGraphSchema::PC_Transition, TEXT("Exit"));
}
