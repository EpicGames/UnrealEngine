// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/SceneStateMachineNode.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Nodes/SceneStateMachineTransitionNode.h"
#include "SceneStateMachineGraph.h"
#include "SceneStateMachineGraphSchema.h"
#include "SceneStateMachineGraphUtils.h"
#include "SceneStateMachineNodeNameValidator.h"

TArray<USceneStateMachineTransitionNode*> USceneStateMachineNode::GatherTransitions(bool bInSortList) const
{
	TArray<USceneStateMachineTransitionNode*> Transitions;

	UEdGraphPin* OutputPin = GetOutputPin();
	check(OutputPin);

	// Worst case: All output links are transitions
	Transitions.Reserve(OutputPin->LinkedTo.Num());

	// Normal transitions (can only go out)
	for (UEdGraphPin* Link : OutputPin->LinkedTo)
	{
		check(Link);
		if (USceneStateMachineTransitionNode* Transition = Cast<USceneStateMachineTransitionNode>(Link->GetOwningNode()))
		{
			checkSlow(Transition->GetSourceNode() == this);
			Transitions.Add(Transition);
		}
	}

	// Sort the transitions by priority order, lower numbers are higher priority
	if (bInSortList)
	{
		Transitions.StableSort(
			[](const USceneStateMachineTransitionNode& A, const USceneStateMachineTransitionNode& B)
			{
				return A.GetPriority() < B.GetPriority();
			});
	}

	return Transitions;
}

UEdGraph* USceneStateMachineNode::GetBoundGraph() const
{
	if (BoundGraphs.IsEmpty())
	{
		return nullptr;
	}
	return BoundGraphs[0];
}

bool USceneStateMachineNode::ConditionallyCreateBoundGraph()
{
	CleanInvalidBoundGraphs();

	// Don't create bound graph if there's already an existing valid bound graph
	if (!BoundGraphs.IsEmpty())
	{
		return false;
	}

	UEdGraph* BoundGraph = CreateBoundGraphInternal();
	if (!BoundGraph)
	{
		return false;
	}

	BoundGraphs.Add(BoundGraph);

	// Initialize the Graph
	const UEdGraphSchema* Schema = BoundGraph->GetSchema();
	check(Schema);
	Schema->CreateDefaultNodesForGraph(*BoundGraph);

	// Add the new graph as a child of our parent graph
	UEdGraph* ParentGraph = GetGraph();
	check(ParentGraph);
	ParentGraph->SubGraphs.AddUnique(BoundGraph);
	return true;
}

void USceneStateMachineNode::CleanInvalidBoundGraphs()
{
	BoundGraphs.RemoveAll(
		[This = this](UEdGraph* InBoundGraph)
		{
			// Remove nulls
			if (!InBoundGraph)
			{
				return true;
			}

			// Remove graphs that aren't outered to this node
			if (InBoundGraph->GetOuter() != This)
			{
				if (UEdGraph* ParentGraph = InBoundGraph->GetTypedOuter<UEdGraph>())
				{
					ParentGraph->SubGraphs.Remove(InBoundGraph);
				}
				return true;
			}
			return false;
		});
}

void USceneStateMachineNode::PostPasteNode()
{
	Super::PostPasteNode();

	CleanInvalidBoundGraphs();

	for (UEdGraph* BoundGraph : BoundGraphs)
	{
		for (UEdGraphNode* GraphNode : BoundGraph->Nodes)
		{
			GraphNode->CreateNewGuid();
			GraphNode->PostPasteNode();
			GraphNode->ReconstructNode();
		}

		// Add the new graph as a child of our parent graph
		if (UEdGraph* ParentGraph = GetGraph())
		{
			ParentGraph->SubGraphs.AddUnique(BoundGraph);
		}

		// Restore transactional flag that is lost during copy/paste process
		BoundGraph->SetFlags(RF_Transactional);
	}

	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNodeChecked(this);
	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
}

void USceneStateMachineNode::OnRenameNode(const FString& InNodeName)
{
	NodeName = *InNodeName;
}

void USceneStateMachineNode::AutowireNewNode(UEdGraphPin* InSourcePin)
{
	if (!InSourcePin)
	{
		return;
	}

	UEdGraphPin* const InputPin = GetInputPin();
	if (!InputPin)
	{
		return;
	}

	const UEdGraphSchema* Schema = GetSchema();
	if (Schema && Schema->TryCreateConnection(InSourcePin, InputPin))
	{
		NodeConnectionListChanged();
	}
}

FText USceneStateMachineNode::GetNodeTitle(ENodeTitleType::Type InTitleType) const
{
	return FText::FromName(NodeName);
}

void USceneStateMachineNode::DestroyNode()
{
	Super::DestroyNode();

	for (UEdGraph* BoundGraph : BoundGraphs)
	{
		UE::SceneState::Graph::RemoveGraph(BoundGraph);
	}
	BoundGraphs.Reset();

	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNodeChecked(this);
	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
}

TArray<UEdGraph*> USceneStateMachineNode::GetSubGraphs() const
{
	return BoundGraphs;
}

UObject* USceneStateMachineNode::GetJumpTargetForDoubleClick() const
{
	return !BoundGraphs.IsEmpty() ? BoundGraphs[0] : nullptr;
}

bool USceneStateMachineNode::CanJumpToDefinition() const
{
	return !!GetJumpTargetForDoubleClick();
}

void USceneStateMachineNode::JumpToDefinition() const
{
	if (UObject* HyperlinkTarget = GetJumpTargetForDoubleClick())
	{
		FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(HyperlinkTarget);
	}
}

bool USceneStateMachineNode::CanCreateUnderSpecifiedSchema(const UEdGraphSchema* InSchema) const
{
	check(InSchema);
	return InSchema->IsA<USceneStateMachineGraphSchema>();
}

TSharedPtr<INameValidatorInterface> USceneStateMachineNode::MakeNameValidator() const
{
	return MakeShared<UE::SceneState::Graph::FStateMachineNodeNameValidator>(this);
}

void USceneStateMachineNode::PostLoad()
{
	Super::PostLoad();
	CleanInvalidBoundGraphs();
}

void USceneStateMachineNode::HidePins(TConstArrayView<FName> InPinNames)
{
	for (FName PinName : InPinNames)
	{
		if (UEdGraphPin* Pin = FindPin(PinName))
		{
			Pin->bHidden = true;
		}
	}
}

void USceneStateMachineNode::GenerateNewNodeName()
{
	const TSharedRef<INameValidatorInterface> NameValidator = FNameValidatorFactory::MakeValidator(this).ToSharedRef();

	FString NewNodeName = NodeName.GetPlainNameString();
	NameValidator->FindValidString(NewNodeName);
	NodeName = *NewNodeName;
}
