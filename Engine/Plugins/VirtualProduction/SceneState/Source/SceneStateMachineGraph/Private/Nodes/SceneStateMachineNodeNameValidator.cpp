// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateMachineNodeNameValidator.h"
#include "Algo/Count.h"
#include "Nodes/SceneStateMachineNode.h"
#include "SceneStateMachineGraph.h"

namespace UE::SceneState::Graph
{

FStateMachineNodeNameValidator::FStateMachineNodeNameValidator(const USceneStateMachineNode* InNodeToValidate)
{
	check(InNodeToValidate);

	// Only consider nodes that share the same type (i.e. name conflicts should only happen between nodes of the same type)
	auto IsNodeRelevant = [NodeType = InNodeToValidate->GetNodeType()](const USceneStateMachineNode* InStateMachineNode)
		{
			return InStateMachineNode && InStateMachineNode->GetNodeType() == NodeType;
		};

	const USceneStateMachineGraph* Graph = CastChecked<USceneStateMachineGraph>(InNodeToValidate->GetOuter());

	Names.Reserve(Algo::CountIf(Graph->Nodes,
		[&IsNodeRelevant](const UEdGraphNode* InNode)
		{
			return IsNodeRelevant(Cast<USceneStateMachineNode>(InNode));
		}));

	for (const UEdGraphNode* Node : Graph->Nodes)
	{
		const USceneStateMachineNode* StateMachineNode = Cast<USceneStateMachineNode>(Node);
		if (IsNodeRelevant(StateMachineNode))
		{
			Names.Add(StateMachineNode->GetNodeName());
		}
	}
}

EValidatorResult FStateMachineNodeNameValidator::IsValid(const FName& InName, bool bInOriginal)
{
	if (InName.IsNone())
	{
		return EValidatorResult::EmptyName;
	}

	if (Names.Contains(InName))
	{
		return EValidatorResult::AlreadyInUse;
	}

	return EValidatorResult::Ok;
}

EValidatorResult FStateMachineNodeNameValidator::IsValid(const FString& InName, bool bInOriginal)
{
	return IsValid(FName(InName), bInOriginal);
}

} // UE::SceneState::Graph
