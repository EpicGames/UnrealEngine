// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "UObject/NameTypes.h"

class USceneStateMachineNode;

namespace UE::SceneState::Graph
{

class FStateMachineNodeNameValidator : public INameValidatorInterface
{
public:
	explicit FStateMachineNodeNameValidator(const USceneStateMachineNode* InNodeToValidate);

private:
	//~ Begin INameValidatorInterface
	virtual EValidatorResult IsValid(const FName& InName, bool bInOriginal) override;
	virtual EValidatorResult IsValid(const FString& InName, bool bInOriginal) override;
	//~ End INameValidatorInterface

	TSet<FName> Names;
};

} // UE::SceneState::Graph
