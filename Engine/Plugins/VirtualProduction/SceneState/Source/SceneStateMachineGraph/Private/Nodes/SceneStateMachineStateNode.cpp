// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/SceneStateMachineStateNode.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "Nodes/SceneStateMachineTaskNode.h"
#include "SceneStateBindingUtils.h"
#include "SceneStateMachineGraph.h"
#include "SceneStateMachineGraphSchema.h"

USceneStateMachineStateNode::FOnParametersChanged USceneStateMachineStateNode::OnParametersChangedDelegate;

USceneStateMachineStateNode::USceneStateMachineStateNode()
{
	NodeName = TEXT("State");
	NodeType = UE::SceneState::Graph::EStateMachineNodeType::State;

	bCanRenameNode = true;
}

TMulticastDelegate<void(USceneStateMachineStateNode*)>::RegistrationType& USceneStateMachineStateNode::OnParametersChanged()
{
	return OnParametersChangedDelegate;
}

void USceneStateMachineStateNode::NotifyParametersChanged()
{
	OnParametersChangedDelegate.Broadcast(this);
}

TOptional<uint16> USceneStateMachineStateNode::GetCompiledIndex() const
{
	return CompiledIndex;
}

void USceneStateMachineStateNode::SetCompiledIndex(uint16 InCompiledIndex)
{
	CompiledIndex = InCompiledIndex;
}

bool USceneStateMachineStateNode::FindEventHandlerId(const FSceneStateEventSchemaHandle& InEventSchemaHandle, FGuid& OutHandlerId) const
{
	for (const FSceneStateEventHandler& EventHandler : EventHandlers)
	{
		if (EventHandler.GetEventSchemaHandle() == InEventSchemaHandle)
		{
			OutHandlerId = EventHandler.GetHandlerId();
			return true;
		}
	}

	return false;
}

bool USceneStateMachineStateNode::HasValidPins() const
{
	return Super::HasValidPins() && GetTaskPin();
}

UEdGraph* USceneStateMachineStateNode::CreateBoundGraphInternal()
{
	UEdGraph* const NewGraph = FBlueprintEditorUtils::CreateNewGraph(this
		, TEXT("SceneStateMachine")
		, USceneStateMachineGraph::StaticClass()
		, USceneStateMachineGraphSchema::StaticClass());

	check(NewGraph);

	FBlueprintEditorUtils::RenameGraphWithSuggestion(NewGraph, FNameValidatorFactory::MakeValidator(this), TEXT("SubStateMachine"));
	return NewGraph;
}

void USceneStateMachineStateNode::AllocateDefaultPins()
{
	CreatePin(EGPD_Input, USceneStateMachineGraphSchema::PC_Transition, USceneStateMachineGraphSchema::PN_In);
	CreatePin(EGPD_Output, USceneStateMachineGraphSchema::PC_Transition, USceneStateMachineGraphSchema::PN_Out);
	CreatePin(EGPD_Output, USceneStateMachineGraphSchema::PC_Task, USceneStateMachineGraphSchema::PN_Task);

	// Hide pins that should be hidden
	HidePins(MakeArrayView(&USceneStateMachineGraphSchema::PN_In, 1));
}

bool USceneStateMachineStateNode::CanDuplicateNode() const
{
	return true;
}

void USceneStateMachineStateNode::PostPasteNode()
{
	Super::PostPasteNode();
	GenerateNewNodeName();
}

void USceneStateMachineStateNode::PostPlacedNewNode()
{
	Super::PostPlacedNewNode();
	GenerateNewNodeName();
}

void USceneStateMachineStateNode::PostInitProperties()
{
	Super::PostInitProperties();

	if (!IsTemplate())
	{
		ParametersId = FGuid::NewGuid();
	}
}

void USceneStateMachineStateNode::PostLoad()
{
	Super::PostLoad();

	// Hide pins that should be hidden
	HidePins(MakeArrayView(&USceneStateMachineGraphSchema::PN_In, 1));

	// Move main graph to bound graph
	if (MainGraph && MainGraph->GetOuter() == this)
	{
		BoundGraphs.Empty(0);
		BoundGraphs.Add(MainGraph);
	}
	MainGraph = nullptr;
}

void USceneStateMachineStateNode::PostDuplicate(bool bInDuplicateForPIE)
{
	Super::PostDuplicate(bInDuplicateForPIE);
	GenerateNewParametersId();
}

void USceneStateMachineStateNode::PostEditImport()
{
	Super::PostEditImport();
	GenerateNewParametersId();
}

void USceneStateMachineStateNode::GenerateNewParametersId()
{
	const FGuid OldParametersId = ParametersId;
	ParametersId = FGuid::NewGuid();
	UE::SceneState::HandleStructIdChanged(*this, OldParametersId, ParametersId);
}
