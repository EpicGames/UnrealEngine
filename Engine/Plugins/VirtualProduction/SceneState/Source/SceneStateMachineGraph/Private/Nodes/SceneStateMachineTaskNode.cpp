// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/SceneStateMachineTaskNode.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "SceneStateBindingDelegates.h"
#include "SceneStateBindingUtils.h"
#include "SceneStateMachineGraphSchema.h"
#include "SceneStateUtils.h"
#include "Tasks/SceneStateBlueprintableTask.h"
#include "Tasks/SceneStateBlueprintableTaskWrapper.h"
#include "Tasks/SceneStateTaskBindingExtension.h"
#include "Tasks/SceneStateTaskDesc.h"
#include "Tasks/SceneStateTaskDescRegistry.h"
#include "Tasks/SceneStateTaskEditChange.h"

#define LOCTEXT_NAMESPACE "SceneStateMachineTaskNode"

USceneStateMachineTaskNode::USceneStateMachineTaskNode()
{
	NodeName = NAME_None;
	NodeType = UE::SceneState::Graph::EStateMachineNodeType::Task;
	bCanRenameNode = true;

	UE::SceneState::OnStructIdChanged.AddUObject(this, &USceneStateMachineTaskNode::OnStructIdChanged);
}

bool USceneStateMachineTaskNode::SetTaskStruct(const UScriptStruct* InTaskStruct)
{
	if (Task.GetScriptStruct() != InTaskStruct)
	{
		if (InTaskStruct)
		{
			Task.InitializeAsScriptStruct(InTaskStruct);
		}
		else
		{
			Task.Reset();
		}
		OnTaskChanged();
		return true;
	}
	return false;
}

void USceneStateMachineTaskNode::SetTaskBlueprintClass(TSubclassOf<USceneStateBlueprintableTask> InTaskClass)
{
	bool bTaskChanged = false;

	if (Task.GetScriptStruct() != FSceneStateBlueprintableTaskWrapper::StaticStruct())
	{
		Task.InitializeAsScriptStruct(FSceneStateBlueprintableTaskWrapper::StaticStruct());
		bTaskChanged = true;
	}

	FSceneStateBlueprintableTaskWrapper& TaskWrapper = Task.GetMutable<FSceneStateBlueprintableTaskWrapper>();
	bTaskChanged |= TaskWrapper.SetTaskClass(InTaskClass);

	if (bTaskChanged)
	{
		OnTaskChanged();
	}
}

bool USceneStateMachineTaskNode::FindDataViewById(const FGuid& InStructId, FStructView& OutDataView)
{
	if (TaskId == InStructId)
	{
		OutDataView = GetTaskInstance();
		return true;
	}

	const FSceneStateTask* ResolvedTask = Task.GetPtr<FSceneStateTask>();
	if (!ResolvedTask)
	{
		return false;
	}

	if (const FSceneStateTaskBindingExtension* BindingExtension = ResolvedTask->GetBindingExtension())
	{
		uint16 DataIndex;
		return BindingExtension->FindDataById(GetTaskInstance(), InStructId, OutDataView, /*out*/DataIndex);
	}

	return false;
}

TOptional<uint16> USceneStateMachineTaskNode::GetCompiledIndex() const
{
	return CompiledIndex;
}

void USceneStateMachineTaskNode::SetCompiledIndex(uint16 InCompiledIndex)
{
	CompiledIndex = InCompiledIndex;
}

void USceneStateMachineTaskNode::AllocateDefaultPins()
{
	CreatePin(EGPD_Input, USceneStateMachineGraphSchema::PC_Task, USceneStateMachineGraphSchema::PN_In);
	CreatePin(EGPD_Output, USceneStateMachineGraphSchema::PC_Task, USceneStateMachineGraphSchema::PN_Out);

	// Hide pins that should be hidden
	HidePins(MakeArrayView(&USceneStateMachineGraphSchema::PN_In, 1));
}

void USceneStateMachineTaskNode::OnRenameNode(const FString& InNodeName)
{
	Super::OnRenameNode(InNodeName);
	CachedTaskDisplayName = FText::GetEmpty();
	OnPostEditTaskDelegate.Broadcast();
}

FText USceneStateMachineTaskNode::GetNodeTitle(ENodeTitleType::Type InTitleType) const
{
	if (CachedTaskDisplayName.IsEmpty())
	{
		if (Task.IsValid())
		{
			const FSceneStateTaskDesc& TaskDesc = FSceneStateTaskDescRegistry::Get().GetTaskDesc(Task.GetScriptStruct());
			CachedTaskDisplayName = TaskDesc.GetDisplayName(MakeContext());
		}
		else
		{
			CachedTaskDisplayName = LOCTEXT("NodeTitle_InvalidTaskClass", "Invalid Task Class");
		}
	}

	if (NodeName.IsNone())
	{
		return CachedTaskDisplayName;
	}

	const FText NodeNameText = FText::FromName(NodeName);

	if (InTitleType == ENodeTitleType::FullTitle && !NodeNameText.EqualTo(CachedTaskDisplayName))
	{
		return FText::FormatNamed(LOCTEXT("NodeTitle_Full", "{NodeName}\n{TaskDisplayName}")
			, TEXT("NodeName"), NodeNameText
			, TEXT("TaskDisplayName"), CachedTaskDisplayName);
	}

	return NodeNameText;
}

FText USceneStateMachineTaskNode::GetTooltipText() const
{
	const FSceneStateTaskDesc& TaskDesc = FSceneStateTaskDescRegistry::Get().GetTaskDesc(Task.GetScriptStruct());
	return TaskDesc.GetTooltip(MakeContext());
}

bool USceneStateMachineTaskNode::CanDuplicateNode() const
{
	return true;
}

void USceneStateMachineTaskNode::PostPasteNode()
{
	Super::PostPasteNode();
	GenerateNewTaskId();
}

UObject* USceneStateMachineTaskNode::GetJumpTargetForDoubleClick() const
{
	const FSceneStateTaskDesc& TaskDesc = FSceneStateTaskDescRegistry::Get().GetTaskDesc(Task.GetScriptStruct());
	return TaskDesc.GetJumpTarget(MakeContext());
}

TSharedPtr<INameValidatorInterface> USceneStateMachineTaskNode::MakeNameValidator() const
{
	return MakeShared<FDummyNameValidator>(EValidatorResult::Ok);
}

void USceneStateMachineTaskNode::PostLoad()
{
	Super::PostLoad();

	// Hide pins that should be hidden
	HidePins(MakeArrayView(&USceneStateMachineGraphSchema::PN_In, 1));
}

void USceneStateMachineTaskNode::BeginDestroy()
{
	Super::BeginDestroy();
	UE::SceneState::OnStructIdChanged.RemoveAll(this);
}

void USceneStateMachineTaskNode::PostEditChangeChainProperty(FPropertyChangedChainEvent& InPropertyChangedEvent)
{
	using namespace UE::SceneState;

	Super::PostEditChangeChainProperty(InPropertyChangedEvent);

	CachedTaskDisplayName = FText::GetEmpty();

	const ETaskObjectType ChangedObject = GetChangedObject(InPropertyChangedEvent);

	if (ChangedObject != ETaskObjectType::None)
	{
		OnTaskChanged();

		// If the change chain is deeper than just the task, it means that the property is within the task or task instance
		// so propagate this edit change event to the task
		if (InPropertyChangedEvent.PropertyChain.Num() > 1)
		{
			PostEditChangeTask(InPropertyChangedEvent, ChangedObject);
		}
	}

	OnPostEditTaskDelegate.Broadcast();
}

void USceneStateMachineTaskNode::PostDuplicate(bool bInDuplicateForPIE)
{
	Super::PostDuplicate(bInDuplicateForPIE);

	if (!bInDuplicateForPIE)
	{
		GenerateNewTaskId();
		RebuildTaskInstance();
	}
}

void USceneStateMachineTaskNode::PostEditImport()
{
	Super::PostEditImport();
	GenerateNewTaskId();
	RebuildTaskInstance();
}

UE::SceneState::ETaskObjectType USceneStateMachineTaskNode::GetChangedObject(FPropertyChangedChainEvent& InPropertyChangedEvent)
{
	using namespace UE::SceneState;

	const TDoubleLinkedList<FProperty*>::TDoubleLinkedListNode* const MemberPropertyNode = InPropertyChangedEvent.PropertyChain.GetHead();
	if (!MemberPropertyNode)
	{
		return ETaskObjectType::None;
	}

	const FProperty* const MemberProperty = MemberPropertyNode->GetValue();
	if (!MemberProperty)
	{
		return ETaskObjectType::None;
	}

	const FName MemberPropertyName = MemberProperty->GetFName();

	if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(USceneStateMachineTaskNode, Task))
	{
		return ETaskObjectType::Task;
	}

	if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(USceneStateMachineTaskNode, TaskInstance))
	{
		return ETaskObjectType::TaskInstance;
	}

	return ETaskObjectType::None;
}

void USceneStateMachineTaskNode::PostEditChangeTask(FPropertyChangedChainEvent& InPropertyChangedEvent, UE::SceneState::ETaskObjectType InChangedObject)
{
	if (!TaskInstance.IsValid())
	{
		return;
	}

	FSceneStateTask* const ResolvedTask = Task.GetMutablePtr<FSceneStateTask>();
	if (!ResolvedTask)
	{
		return;
	}

	const TDoubleLinkedList<FProperty*>::TDoubleLinkedListNode* MemberPropertyNode = InPropertyChangedEvent.PropertyChain.GetHead();
	const TDoubleLinkedList<FProperty*>::TDoubleLinkedListNode* ChangedPropertyNode = InPropertyChangedEvent.PropertyChain.GetTail();
	if (!MemberPropertyNode || !ChangedPropertyNode)
	{
		return;
	}

	// Move the member property as it should now be 1 level down
	MemberPropertyNode = MemberPropertyNode->GetNextNode();
	if (!MemberPropertyNode)
	{
		return;
	}

	UE::SceneState::FTaskEditChange TaskEditChange;
	TaskEditChange.ChangeType = InPropertyChangedEvent.ChangeType;
	TaskEditChange.MemberProperty = MemberPropertyNode->GetValue();
	TaskEditChange.Property = ChangedPropertyNode->GetValue();

	TaskEditChange.Outer = this;
	TaskEditChange.ChangedObject = InChangedObject;

	FStructView TaskInstanceView(TaskInstance.GetScriptStruct(), TaskInstance.GetMutableMemory());
	ResolvedTask->PostEditChange(TaskEditChange, TaskInstanceView);
}

FSceneStateTaskDescContext USceneStateMachineTaskNode::MakeContext() const
{
	FSceneStateTaskDescContext Context;
	Context.Task = Task;
	Context.TaskInstance = TaskInstance;
	Context.ContextObject = this;
	return Context;
}

FSceneStateTaskDescMutableContext USceneStateMachineTaskNode::MakeMutableContext()
{
	FSceneStateTaskDescMutableContext Context;
	Context.Task = Task;
	Context.TaskInstance = TaskInstance;
	Context.ContextObject = this;
	return Context;
}

void USceneStateMachineTaskNode::RebuildTaskInstance()
{
	if (const FSceneStateTask* ResolvedTask = Task.GetPtr<FSceneStateTask>())
	{
		FStructView TaskInstanceView(TaskInstance.GetScriptStruct(), TaskInstance.GetMutableMemory());
		ResolvedTask->BuildTaskInstance(this, TaskInstanceView);
	}
}

void USceneStateMachineTaskNode::OnTaskChanged()
{
	if (const FSceneStateTask* ResolvedTask = Task.GetPtr<FSceneStateTask>())
	{
		if (const UScriptStruct* InstanceDataType = ResolvedTask->GetTaskInstanceType())
		{
			if (TaskInstance.GetScriptStruct() != InstanceDataType)
			{
				TaskInstance.InitializeAsScriptStruct(InstanceDataType);

				FStructView TaskInstanceView(TaskInstance.GetScriptStruct(), TaskInstance.GetMutableMemory());
				ResolvedTask->BuildTaskInstance(this, TaskInstanceView);
			}
		}
		else
		{
			TaskInstance.Reset();
		}
	}
	else
	{
		TaskInstance.Reset();
	}
}

void USceneStateMachineTaskNode::OnStructIdChanged(const UE::SceneState::FStructIdChange& InChange)
{
	// Only consider changes if they come from a binding owner that contains this task node
	if (IsIn(InChange.BindingOwner))
	{
		const FSceneStateTaskDesc& TaskDesc = FSceneStateTaskDescRegistry::Get().GetTaskDesc(Task.GetScriptStruct());
		TaskDesc.NotifyStructIdsChanged(MakeMutableContext(), InChange);
	}
}

void USceneStateMachineTaskNode::GenerateNewTaskId()
{
	const FGuid OldTaskId = TaskId;
	TaskId = FGuid::NewGuid();

	UE::SceneState::HandleStructIdChanged(*this, OldTaskId, TaskId);
}

#undef LOCTEXT_NAMESPACE
