// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/SceneStateMachineTransitionNode.h"
#include "EdGraphUtilities.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "SceneStateBindingUtils.h"
#include "SceneStateMachineGraphSchema.h"
#include "SceneStateTransitionGraph.h"
#include "SceneStateTransitionGraphSchema.h"

#define LOCTEXT_NAMESPACE "SceneStateMachineTransitionNode"

USceneStateMachineTransitionNode::FOnParametersChanged USceneStateMachineTransitionNode::OnParametersChangedDelegate;

USceneStateMachineTransitionNode::USceneStateMachineTransitionNode()
{
	NodeName = TEXT("Transition");
	NodeType = UE::SceneState::Graph::EStateMachineNodeType::Transition;
}

USceneStateMachineTransitionNode::FOnParametersChanged::RegistrationType& USceneStateMachineTransitionNode::OnParametersChanged()
{
	return OnParametersChangedDelegate;
}

void USceneStateMachineTransitionNode::NotifyParametersChanged()
{
	OnParametersChangedDelegate.Broadcast(*this);
}

TArray<USceneStateMachineTransitionNode*> USceneStateMachineTransitionNode::GetTransitionsToRelink(UEdGraphPin* InSourcePin, UEdGraphPin* InOldTargetPin, TConstArrayView<UEdGraphNode*> InSelectedGraphNodes)
{
	check(InSourcePin);

	USceneStateMachineNode* SourceNode = Cast<USceneStateMachineNode>(InSourcePin->GetOwningNode());
	if (!SourceNode || !SourceNode->HasValidPins())
	{
		return {};
	}

	check(InOldTargetPin);

	USceneStateMachineTransitionNode* OldTransitionNode = Cast<USceneStateMachineTransitionNode>(InOldTargetPin->GetOwningNode());
	if (!OldTransitionNode)
	{
		return {};
	}

	// Collect all transition nodes starting at the source state
	TArray<USceneStateMachineTransitionNode*> TransitionNodes = SourceNode->GatherTransitions();

	// Compare the target states rather than comparing against the transition nodes
	UEdGraphNode* OldTargetNode = OldTransitionNode->GetTargetNode();

	// Remove the transition nodes from the candidates that are linked to a different target state.
	for (TArray<USceneStateMachineTransitionNode*>::TIterator Iter(TransitionNodes); Iter; ++Iter)
	{
		USceneStateMachineTransitionNode* CurrentTransition = *Iter;
		if (!CurrentTransition)
		{
			continue;
		}

		// Get the actual target states from the transition nodes
		UEdGraphNode* TargetNode = CurrentTransition->GetTargetNode();
		if (TargetNode != OldTargetNode)
		{
			Iter.RemoveCurrent();
		}
	}

	// Collect the subset of selected transitions from the list of possible transitions to be relinked
	TSet<USceneStateMachineTransitionNode*> SelectedTransitionNodes;
	SelectedTransitionNodes.Reserve(InSelectedGraphNodes.Num());

	for (UEdGraphNode* GraphNode : InSelectedGraphNodes)
	{
		USceneStateMachineTransitionNode* TransitionNode = Cast<USceneStateMachineTransitionNode>(GraphNode);

		if (TransitionNode && TransitionNodes.Contains(TransitionNode))
		{
			SelectedTransitionNodes.Add(TransitionNode);
		}
	}

	if (!SelectedTransitionNodes.IsEmpty())
	{
		for (TArray<USceneStateMachineTransitionNode*>::TIterator Iter(TransitionNodes); Iter; ++Iter)
		{
			// Only relink the selected transitions. If none are selected, relink them all.
			if (!SelectedTransitionNodes.Contains(*Iter))
			{
				Iter.RemoveCurrent();
			}
		}
	}

	return TransitionNodes;
}

USceneStateMachineNode* USceneStateMachineTransitionNode::GetSourceNode() const
{
	UEdGraphPin* InputPin = GetInputPin();
	if (!InputPin || InputPin->LinkedTo.IsEmpty() || !InputPin->LinkedTo[0])
	{
		return nullptr;
	}
	return Cast<USceneStateMachineNode>(InputPin->LinkedTo[0]->GetOwningNode());
}

USceneStateMachineNode* USceneStateMachineTransitionNode::GetTargetNode() const
{
	UEdGraphPin* OutputPin = GetOutputPin();
	if (!OutputPin || OutputPin->LinkedTo.IsEmpty() || !OutputPin->LinkedTo[0])
	{
		return nullptr;
	}
	return Cast<USceneStateMachineNode>(OutputPin->LinkedTo[0]->GetOwningNode());
}

void USceneStateMachineTransitionNode::CreateConnections(USceneStateMachineNode* InSourceState, USceneStateMachineNode* InTargetState)
{
	check(InSourceState && InTargetState);

	// Source State's Output -> This Input
	{
		UEdGraphPin* InputPin = GetInputPin();
		check(InputPin);

		InputPin->Modify();
		InputPin->LinkedTo.Empty();

		UEdGraphPin* SourceOutputPin = InSourceState->GetOutputPin();
		check(SourceOutputPin);
		InputPin->MakeLinkTo(SourceOutputPin);
	}

	// This Output -> Target State's Input
	{
		UEdGraphPin* OutputPin = GetOutputPin();
		check(OutputPin);

		// This to next
		OutputPin->Modify();
		OutputPin->LinkedTo.Empty();

		UEdGraphPin* TargetInputPin = InTargetState->GetInputPin();
		check(TargetInputPin);
		OutputPin->MakeLinkTo(TargetInputPin);
	}
}

void USceneStateMachineTransitionNode::RelinkHead(USceneStateMachineNode* InNewTargetState)
{
	// Relink the target state of the transition node
	UEdGraphPin* OutputPin = GetOutputPin();
	check(OutputPin);
	OutputPin->Modify();

	if (USceneStateMachineNode* OldTargetState = GetTargetNode())
	{
		OutputPin->BreakLinkTo(OldTargetState->GetInputPin());
	}

	if (InNewTargetState)
	{
		OutputPin->MakeLinkTo(InNewTargetState->GetInputPin());
	}
}

UEdGraph* USceneStateMachineTransitionNode::CreateBoundGraphInternal()
{
	UEdGraph* NewGraph = FBlueprintEditorUtils::CreateNewGraph(this
		, NAME_None
		, USceneStateTransitionGraph::StaticClass()
		, USceneStateTransitionGraphSchema::StaticClass());

	check(NewGraph);

	FEdGraphUtilities::RenameGraphToNameOrCloseToName(NewGraph, TEXT("TransitionGraph"));
	return NewGraph;
}

void USceneStateMachineTransitionNode::AllocateDefaultPins()
{
	UEdGraphPin* InputPin = CreatePin(EGPD_Input, USceneStateMachineGraphSchema::PC_Transition, USceneStateMachineGraphSchema::PN_In);
	UEdGraphPin* OutputPin = CreatePin(EGPD_Output, USceneStateMachineGraphSchema::PC_Transition, USceneStateMachineGraphSchema::PN_Out);

	check(InputPin && OutputPin);
	InputPin->bHidden = true;
	OutputPin->bHidden = true;
}

FText USceneStateMachineTransitionNode::GetNodeTitle(ENodeTitleType::Type InTitleType) const
{
	return GetTitle();
}

FText USceneStateMachineTransitionNode::GetTooltipText() const
{
	return LOCTEXT("Tooltip", "State transition node in a State Machine");
}

void USceneStateMachineTransitionNode::PinConnectionListChanged(UEdGraphPin* InPin)
{
	if (InPin && InPin->LinkedTo.IsEmpty())
	{
		// Destroy this node. Transitions must always have an input and output connection
		Modify();

		// Our parent graph will have our graph in SubGraphs so needs to be modified to record that.
		if (UEdGraph* ParentGraph = GetGraph())
		{
			ParentGraph->Modify();
		}

		DestroyNode();
	}
}

void USceneStateMachineTransitionNode::PostPasteNode()
{
	GenerateNewNodeName();

	// fail-safe, create empty transition graph
	ConditionallyCreateBoundGraph();
	check(GetBoundGraph());

	Super::PostPasteNode();

	// Get rid of nodes that aren't fully linked (transition nodes have fixed pins as they describe connection between two nodes)
	for (UEdGraphPin* Pin : Pins)
	{
		if (!Pin || Pin->LinkedTo.IsEmpty())
		{
			DestroyNode();
			break;
		}
	}
}

void USceneStateMachineTransitionNode::PostPlacedNewNode()
{
	Super::PostPlacedNewNode();
	GenerateNewNodeName();
	ConditionallyCreateBoundGraph();
}

FText USceneStateMachineTransitionNode::GetTitle() const
{
	USceneStateMachineNode* SourceState = GetSourceNode();
	USceneStateMachineNode* TargetState = GetTargetNode();

	if (SourceState && TargetState)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("SourceState"), FText::FromName(SourceState->GetNodeName()));
		Args.Add(TEXT("TargetState"), FText::FromName(TargetState->GetNodeName()));
		return FText::Format(LOCTEXT("SourceTargetTransitionTitle", "{SourceState} to {TargetState}"), Args);
	}

	return FText::Format(LOCTEXT("DefaultTransitionStateTitle", "Transition: {0}"), FText::FromName(GetNodeName()));
}

bool USceneStateMachineTransitionNode::IsBoundToGraphLifetime(UEdGraph& InGraph) const
{
	return &InGraph == GetBoundGraph();
}

UEdGraphNode* USceneStateMachineTransitionNode::AsNode()
{
	return this;
}

void USceneStateMachineTransitionNode::PostInitProperties()
{
	Super::PostInitProperties();

	if (!IsTemplate())
	{
		ParametersId = FGuid::NewGuid();
	}
}

void USceneStateMachineTransitionNode::PostLoad()
{
	Super::PostLoad();

	// Move transition graph to bound graph
	if (TransitionGraph && TransitionGraph->GetOuter() == this)
	{
		BoundGraphs.Empty(0);
		BoundGraphs.Add(TransitionGraph);
	}
	TransitionGraph = nullptr;
}

void USceneStateMachineTransitionNode::PostDuplicate(bool bInDuplicateForPIE)
{
	Super::PostDuplicate(bInDuplicateForPIE);
	GenerateNewParametersId();
}

void USceneStateMachineTransitionNode::PostEditImport()
{
	Super::PostEditImport();
	GenerateNewParametersId();
}

void USceneStateMachineTransitionNode::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	if (InPropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(USceneStateMachineTransitionNode, Parameters))
	{
		NotifyParametersChanged();
	}
}

void USceneStateMachineTransitionNode::GenerateNewParametersId()
{
	const FGuid OldParametersId = ParametersId;
	ParametersId = FGuid::NewGuid();

	UE::SceneState::HandleStructIdChanged(*this, OldParametersId, ParametersId);
}

#undef LOCTEXT_NAMESPACE
