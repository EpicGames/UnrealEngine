// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateMachineGraph.h"
#include "Engine/Blueprint.h"
#include "Nodes/SceneStateMachineEntryNode.h"
#include "Nodes/SceneStateMachineStateNode.h"
#include "SceneStateBindingUtils.h"
#include "Kismet2/BlueprintEditorUtils.h"

USceneStateMachineGraph::FOnParametersChanged USceneStateMachineGraph::OnParametersChangedDelegate;

USceneStateMachineGraph::USceneStateMachineGraph()
{
	bAllowRenaming = true;
	bAllowDeletion = true;
}

USceneStateMachineGraph::FOnParametersChanged::RegistrationType& USceneStateMachineGraph::OnParametersChanged()
{
	return OnParametersChangedDelegate;
}

void USceneStateMachineGraph::NotifyParametersChanged()
{
	OnParametersChangedDelegate.Broadcast(this);
}

USceneStateMachineEntryNode* USceneStateMachineGraph::GetEntryNode() const
{
	USceneStateMachineEntryNode* LastEntryNode = nullptr;

	for (UEdGraphNode* Node : Nodes)
	{
		USceneStateMachineEntryNode* EntryNode = Cast<USceneStateMachineEntryNode>(Node);
		if (!EntryNode)
		{
			continue;
		}

		LastEntryNode = EntryNode;

		// Break immediately if the node connects to a state node
		if (LastEntryNode->GetStateNode())
		{
			break;
		}
	}

	return LastEntryNode;
}

USceneStateMachineStateNode* USceneStateMachineGraph::GetParentStateNode() const
{
	return Cast<USceneStateMachineStateNode>(GetOuter());
}

bool USceneStateMachineGraph::IsRootStateMachine() const
{
	return !!Cast<UBlueprint>(GetOuter());
}

TOptional<uint16> USceneStateMachineGraph::GetCompiledIndex() const
{
	return CompiledIndex;
}

void USceneStateMachineGraph::SetCompiledIndex(uint16 InCompiledIndex)
{
	CompiledIndex = InCompiledIndex;
}

void USceneStateMachineGraph::AddNode(UEdGraphNode* InNodeToAdd, bool bInUserAction, bool bInSelectNewNode)
{
	if (!InNodeToAdd)
	{
		return;
	}

	// Workaround for when 'CanCreateUnderSpecifiedSchema' is not called on situations like in SMyBlueprint::OnActionDragged for functions
	const UEdGraphSchema* GraphSchema = GetSchema();
	if (GraphSchema && InNodeToAdd->CanCreateUnderSpecifiedSchema(GraphSchema))
	{
		Super::AddNode(InNodeToAdd, bInUserAction, bInSelectNewNode);
	}
}

void USceneStateMachineGraph::PostInitProperties()
{
	Super::PostInitProperties();

	if (!IsTemplate())
	{
		ParametersId = FGuid::NewGuid();
	}
}

void USceneStateMachineGraph::PostLoad()
{
	Super::PostLoad();

	for (TArray<TObjectPtr<UEdGraphNode>>::TIterator NodeIter(Nodes); NodeIter; ++NodeIter)
	{
		UEdGraphNode* Node = *NodeIter;
		if (!Node || Node->GetOuter() != this)
		{
			NodeIter.RemoveCurrent();
		}
	}
}

void USceneStateMachineGraph::PostDuplicate(bool bInDuplicateForPIE)
{
	Super::PostDuplicate(bInDuplicateForPIE);
	GenerateNewParametersId();
}

void USceneStateMachineGraph::PostEditImport()
{
	Super::PostEditImport();
	GenerateNewParametersId();
}

void USceneStateMachineGraph::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	if (InPropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(USceneStateMachineGraph, Category))
	{
		if (UBlueprint* const Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(this))
		{
			FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
		}
	}
}

void USceneStateMachineGraph::GenerateNewParametersId()
{
	const FGuid OldParametersId = ParametersId;
	ParametersId = FGuid::NewGuid();

	UE::SceneState::HandleStructIdChanged(*this, OldParametersId, ParametersId);
}
