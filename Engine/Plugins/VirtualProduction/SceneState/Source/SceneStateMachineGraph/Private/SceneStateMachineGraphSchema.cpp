// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateMachineGraphSchema.h"
#include "Actions/SceneStateMachineAction_NewBlueprintTask.h"
#include "Actions/SceneStateMachineAction_NewComment.h"
#include "Actions/SceneStateMachineAction_NewNode.h"
#include "Actions/SceneStateMachineAction_NewTask.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Engine/Blueprint.h"
#include "Framework/Commands/GenericCommands.h"
#include "GraphEditorActions.h"
#include "ISceneStateSchemaProvider.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Nodes/SceneStateMachineConduitNode.h"
#include "Nodes/SceneStateMachineEntryNode.h"
#include "Nodes/SceneStateMachineExitNode.h"
#include "Nodes/SceneStateMachineNode.h"
#include "Nodes/SceneStateMachineStateNode.h"
#include "Nodes/SceneStateMachineTaskNode.h"
#include "Nodes/SceneStateMachineTransitionNode.h"
#include "SceneStateMachineGraph.h"
#include "SceneStateMachineNodeConnectionType.h"
#include "SceneStateSchema.h"
#include "SceneStateUtils.h"
#include "ScopedTransaction.h"
#include "Tasks/SceneStateTask.h"
#include "Tasks/SceneStateTaskMetadata.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"
#include "UObject/UObjectIterator.h"

#define LOCTEXT_NAMESPACE "SceneStateMachineGraphSchema"

namespace UE::SceneState::Graph::Private
{
	template<typename InActionType, typename... InArgTypes>
	TSharedRef<InActionType> AddAction(FGraphContextMenuBuilder& InContextMenuBuilder, const FString& InCategory, InArgTypes&&... InArgs)
	{
		TSharedRef<InActionType> Action = MakeShared<InActionType>(Forward<InArgTypes>(InArgs)...);
		Action->CosmeticUpdateRootCategory(FText::FromString(InCategory));
		InContextMenuBuilder.AddAction(Action);
		return Action;
	}

	/** Allowed Connection Types(Source-->Target) mapped to their type of connection */
	const TMap<FNodeConnectionType, ECanCreateConnectionResponse> GConnectionTypes =
	{
		/** Connections that can directly connect (with optional extra settings like breaking) */
		{ FNodeConnectionType(EStateMachineNodeType::Entry, EStateMachineNodeType::State), CONNECT_RESPONSE_BREAK_OTHERS_A },
		{ FNodeConnectionType(EStateMachineNodeType::Task , EStateMachineNodeType::Task ), CONNECT_RESPONSE_MAKE },

		/** Connections that require custom processing (e.g. adding a transition node in between, or using a different pin) */
		{ FNodeConnectionType(EStateMachineNodeType::State  , EStateMachineNodeType::State  ), CONNECT_RESPONSE_MAKE_WITH_CONVERSION_NODE },
		{ FNodeConnectionType(EStateMachineNodeType::State  , EStateMachineNodeType::Task   ), CONNECT_RESPONSE_MAKE_WITH_CONVERSION_NODE },
        { FNodeConnectionType(EStateMachineNodeType::State  , EStateMachineNodeType::Exit   ), CONNECT_RESPONSE_MAKE_WITH_CONVERSION_NODE },
		{ FNodeConnectionType(EStateMachineNodeType::State  , EStateMachineNodeType::Conduit), CONNECT_RESPONSE_MAKE_WITH_CONVERSION_NODE },
		{ FNodeConnectionType(EStateMachineNodeType::Conduit, EStateMachineNodeType::Conduit), CONNECT_RESPONSE_MAKE_WITH_CONVERSION_NODE },
		{ FNodeConnectionType(EStateMachineNodeType::Conduit, EStateMachineNodeType::State  ), CONNECT_RESPONSE_MAKE_WITH_CONVERSION_NODE },
		{ FNodeConnectionType(EStateMachineNodeType::Conduit, EStateMachineNodeType::Exit   ), CONNECT_RESPONSE_MAKE_WITH_CONVERSION_NODE },
	};

	/** Allowed Relinking Types.  OldTarget --> NewTarget */
	const TSet<FNodeConnectionType> GRelinkingTypes =
	{
		FNodeConnectionType(EStateMachineNodeType::State     , EStateMachineNodeType::State  ),
		FNodeConnectionType(EStateMachineNodeType::Transition, EStateMachineNodeType::Conduit),
		FNodeConnectionType(EStateMachineNodeType::Transition, EStateMachineNodeType::State  ),
		FNodeConnectionType(EStateMachineNodeType::Transition, EStateMachineNodeType::Exit   ),
		FNodeConnectionType(EStateMachineNodeType::Task      , EStateMachineNodeType::Task   ),
	};

	/** Types that require a transition node in between. Source --> Target */
	const TSet<FNodeConnectionType> GTransitionTypes =
	{
		FNodeConnectionType(EStateMachineNodeType::State     , EStateMachineNodeType::State  ),
		FNodeConnectionType(EStateMachineNodeType::State     , EStateMachineNodeType::Exit   ),
		FNodeConnectionType(EStateMachineNodeType::State     , EStateMachineNodeType::Conduit),
		FNodeConnectionType(EStateMachineNodeType::Conduit   , EStateMachineNodeType::Conduit),
		FNodeConnectionType(EStateMachineNodeType::Conduit   , EStateMachineNodeType::State  ),
		FNodeConnectionType(EStateMachineNodeType::Conduit   , EStateMachineNodeType::Exit   ),
	};

} // UE::SceneState::Graph::Private

const FName USceneStateMachineGraphSchema::PN_In(TEXT("In"));
const FName USceneStateMachineGraphSchema::PN_Out(TEXT("Out"));
const FName USceneStateMachineGraphSchema::PN_Task(TEXT("Task"));

const FName USceneStateMachineGraphSchema::PC_Transition(TEXT("Transition"));
const FName USceneStateMachineGraphSchema::PC_Task(TEXT("Task"));

const FLinearColor USceneStateMachineGraphSchema::PCC_Transition(FLinearColor::White);
const FLinearColor USceneStateMachineGraphSchema::PCC_Task(FLinearColor::White);

USceneStateMachineStateNode* USceneStateMachineGraphSchema::FindConnectedStateNode(const UEdGraphNode* InTaskNode)
{
	TArray<const USceneStateMachineTaskNode*> TaskNodesToSearch { Cast<USceneStateMachineTaskNode>(InTaskNode) };

	while (!TaskNodesToSearch.IsEmpty())
	{
		const USceneStateMachineTaskNode* TaskNode = TaskNodesToSearch.Pop();
		if (!TaskNode)
		{
			continue;
		}

		const UEdGraphPin* TaskInputPin = TaskNode->GetInputPin();
		if (!TaskInputPin || TaskInputPin->LinkedTo.IsEmpty())
		{
			continue;
		}

		for (const UEdGraphPin* LinkedPin : TaskInputPin->LinkedTo)
		{
			if (!LinkedPin)
			{
				continue;
			}

			UEdGraphNode* LinkedNode = LinkedPin->GetOwningNodeUnchecked();
			if (!LinkedNode)
			{
				continue;
			}

			if (USceneStateMachineStateNode* LinkedStateNode = Cast<USceneStateMachineStateNode>(LinkedNode))
			{
				return LinkedStateNode;
			}

			if (USceneStateMachineTaskNode* LinkedTaskNode = Cast<USceneStateMachineTaskNode>(LinkedNode))
			{
				TaskNodesToSearch.Add(LinkedTaskNode);
			}
		}
	}

	return nullptr;
}

EGraphType USceneStateMachineGraphSchema::GetGraphType(const UEdGraph* InTestEdGraph) const
{
	return GT_StateMachine;
}

void USceneStateMachineGraphSchema::CreateDefaultNodesForGraph(UEdGraph& InGraph) const
{
	// Create the entry/exit tunnels
	FGraphNodeCreator<USceneStateMachineEntryNode> NodeCreator(InGraph);
	USceneStateMachineEntryNode* EntryNode = NodeCreator.CreateNode();
	NodeCreator.Finalize();

	SetNodeMetaData(EntryNode, FNodeMetadata::DefaultGraphNode);
}

void USceneStateMachineGraphSchema::GetGraphContextActions(FGraphContextMenuBuilder& InContextMenuBuilder) const
{
	GetGraphContextCoreActions(InContextMenuBuilder);
	GetGraphContextTaskActions(InContextMenuBuilder);
}

void USceneStateMachineGraphSchema::GetContextMenuActions(UToolMenu* InMenu, UGraphNodeContextMenuContext* InContext) const
{
	if (!InContext)
	{
		return;
	}

	// Node Actions
	if (!InContext->bIsDebugging && InContext->Node)
	{
		FToolMenuSection& NodeSection = InMenu->AddSection(TEXT("SceneStateMachineNodeActions"), LOCTEXT("NodeActionsTitle", "Node Actions"));

		const FGenericCommands& GenericCommands = FGenericCommands::Get();

		NodeSection.AddMenuEntry(GenericCommands.Delete);
		NodeSection.AddMenuEntry(GenericCommands.Cut);
		NodeSection.AddMenuEntry(GenericCommands.Copy);
		NodeSection.AddMenuEntry(GenericCommands.Duplicate);

		const FGraphEditorCommandsImpl& GraphEditorCommands = FGraphEditorCommands::Get();

		NodeSection.AddMenuEntry(GraphEditorCommands.ReconstructNodes);
		NodeSection.AddMenuEntry(GraphEditorCommands.BreakNodeLinks);

		if (InContext->Node->bCanRenameNode)
		{
			NodeSection.AddMenuEntry(GenericCommands.Rename);
		}
	}
}

const FPinConnectionResponse USceneStateMachineGraphSchema::CanCreateConnection(const UEdGraphPin* InSourcePin, const UEdGraphPin* InTargetPin) const
{
	using namespace UE::SceneState::Graph;

	if (!InSourcePin || !InTargetPin)
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("DisallowConnection_InvalidPins", "Pins are invalid!"));
	}

	if (InSourcePin->LinkedTo.Contains(InTargetPin))
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("DisallowConnection_Redundant", "Pins are already connected"));
	}

	USceneStateMachineNode* const SourceNode = Cast<USceneStateMachineNode>(InSourcePin->GetOwningNode());
	USceneStateMachineNode* const TargetNode = Cast<USceneStateMachineNode>(InTargetPin->GetOwningNode());

	if (!SourceNode || !TargetNode || !SourceNode->HasValidPins() || !TargetNode->HasValidPins())
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("DisallowConnection_InvalidNodes", "Pin nodes are invalid!"));
	}

	// Disallow pin connection on the same node
	if (SourceNode == TargetNode)
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("DisallowConnection_SameNode", "Both pins are on the same node"));
	}

	const EStateMachineNodeType SourceType = SourceNode->GetNodeType();
	const EStateMachineNodeType TargetType = TargetNode->GetNodeType();

	const ECanCreateConnectionResponse* const ConnectionResponse = Private::GConnectionTypes.Find(FNodeConnectionType(SourceType, TargetType));

	// Check if the node connection type is allowed 
	if (!ConnectionResponse)
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("DisallowConnection_IncompatibleType", "Type connection is incompatible"));
	}

	// For Task to Task connection, target can either be isolated or must be connected to the same state as the source
	if (const USceneStateMachineStateNode* const TargetStateNode = FindConnectedStateNode(TargetNode))
	{
		const USceneStateMachineStateNode* SourceStateNode = Cast<USceneStateMachineStateNode>(SourceNode);
		if (!SourceStateNode)
		{
			SourceStateNode = FindConnectedStateNode(SourceNode);
		}

		if (TargetStateNode != SourceStateNode)
		{
			return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("DisallowConnection_CrossStateTasks", "Task is already connected to a State!"));
		}
	}

	return FPinConnectionResponse(*ConnectionResponse, LOCTEXT("AllowConnection", "Connect node"));
}

bool USceneStateMachineGraphSchema::TryCreateConnection(UEdGraphPin* InSourcePin, UEdGraphPin* InTargetPin) const
{
	check(InSourcePin && InTargetPin);

	// Flip the Target pin direction if directions match
	if (InSourcePin->Direction == InTargetPin->Direction)
	{
		if (USceneStateMachineNode* TargetNode = Cast<USceneStateMachineNode>(InTargetPin->GetOwningNode()))
		{
			if (InSourcePin->Direction == EGPD_Input)
			{
				InTargetPin = TargetNode->GetOutputPin();
			}
			else
			{
				InTargetPin = TargetNode->GetInputPin();
			}
		}
	}

	const bool bModified = UEdGraphSchema::TryCreateConnection(InSourcePin, InTargetPin);

	if (bModified)
	{
		UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNodeChecked(InSourcePin->GetOwningNode());
		FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
	}

	return bModified;
}

bool USceneStateMachineGraphSchema::CreateAutomaticConversionNodeAndConnections(UEdGraphPin* InSourcePin, UEdGraphPin* InTargetPin) const
{
	using namespace UE::SceneState::Graph;

	USceneStateMachineNode* const SourceNode = Cast<USceneStateMachineNode>(InSourcePin->GetOwningNode());
	USceneStateMachineNode* const TargetNode = Cast<USceneStateMachineNode>(InTargetPin->GetOwningNode());

	if (!SourceNode || !TargetNode)
	{
		return false;
	}

	const EStateMachineNodeType SourceNodeType = SourceNode->GetNodeType();
	const EStateMachineNodeType TargetNodeType = TargetNode->GetNodeType();

	// Check if the connection type requires a transition in between
	if (Private::GTransitionTypes.Contains(FNodeConnectionType(SourceNodeType, TargetNodeType)))
	{
		FTransitionConnectionParams Params;
		Params.SourceNode = SourceNode;
		Params.TargetNode = TargetNode;
		Params.SourcePin = InSourcePin;
		Params.TargetPin = InTargetPin;

		CreateConnectionWithTransition(Params);
		return true;
	}

	// Special connections: State to Task connections
	if (SourceNodeType == EStateMachineNodeType::State && TargetNodeType == EStateMachineNodeType::Task)
	{
		// Use Task Pin instead of the given source pin (task pin is dedicated to have task connections, but it is hidden)
		UEdGraphPin* StateTaskPin = CastChecked<USceneStateMachineStateNode>(SourceNode)->GetTaskPin();
		check(StateTaskPin);
		StateTaskPin->MakeLinkTo(InTargetPin);
		return true;
	}

	return false;
}

bool USceneStateMachineGraphSchema::TryRelinkConnectionTarget(UEdGraphPin* InSourcePin, UEdGraphPin* InOldTargetPin, UEdGraphPin* InNewTargetPin, const TArray<UEdGraphNode*>& InSelectedGraphNodes) const
{
	using namespace UE::SceneState::Graph;

	const FPinConnectionResponse Response = CanCreateConnection(InSourcePin, InNewTargetPin);
	if (Response.Response == CONNECT_RESPONSE_DISALLOW)
	{
		return false;
	}

	USceneStateMachineNode* const OldTargetNode = Cast<USceneStateMachineNode>(InOldTargetPin->GetOwningNode());
	USceneStateMachineNode* const NewTargetNode = Cast<USceneStateMachineNode>(InNewTargetPin->GetOwningNode());

	if (!OldTargetNode || !OldTargetNode->HasValidPins()|| !NewTargetNode || !NewTargetNode->HasValidPins())
	{
		return false;
	}

	const EStateMachineNodeType OldTargetType = OldTargetNode->GetNodeType();
	const EStateMachineNodeType NewTargetType = NewTargetNode->GetNodeType();
	if (!Private::GRelinkingTypes.Contains(FNodeConnectionType(OldTargetType, NewTargetType)))
	{
		return false;
	}

	// Collect all transition nodes starting at the source state, filter them by the transitions and perform the actual relink operation.
	const TArray<USceneStateMachineTransitionNode*> TransitionNodes = USceneStateMachineTransitionNode::GetTransitionsToRelink(InSourcePin, InOldTargetPin, InSelectedGraphNodes);
	if (!TransitionNodes.IsEmpty())
	{
		for (USceneStateMachineTransitionNode* TransitionNode : TransitionNodes)
		{
			TransitionNode->RelinkHead(NewTargetNode);
		}

		UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNodeChecked(InSourcePin->GetOwningNode());
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);

		InSourcePin->GetOwningNode()->PinConnectionListChanged(InSourcePin);
		InOldTargetPin->GetOwningNode()->PinConnectionListChanged(InOldTargetPin);
		InNewTargetPin->GetOwningNode()->PinConnectionListChanged(InNewTargetPin);
		return true;
	}

	// Fallback default behavior: Break pin links between Source Pin and Target Pin
	// and create a connection between source and target
	InSourcePin->BreakLinkTo(InOldTargetPin);
	return TryCreateConnection(InSourcePin, InNewTargetPin);
}

bool USceneStateMachineGraphSchema::IsConnectionRelinkingAllowed(UEdGraphPin* InPin) const
{
	return InPin && InPin->GetOwningNode()->IsA<USceneStateMachineNode>();
}

const FPinConnectionResponse USceneStateMachineGraphSchema::CanRelinkConnectionToPin(const UEdGraphPin* InOldSourcePin, const UEdGraphPin* InTargetPin) const
{
	FPinConnectionResponse Response = CanCreateConnection(InOldSourcePin, InTargetPin);
	if (Response.Response != CONNECT_RESPONSE_DISALLOW)
	{
		Response.Message = LOCTEXT("AllowConnection_Relink", "Relink");
	}
	return Response;
}

FLinearColor USceneStateMachineGraphSchema::GetPinTypeColor(const FEdGraphPinType& InPinType) const
{
	if (InPinType.PinCategory == PC_Transition)
	{
		return PCC_Transition;
	}

	if (InPinType.PinCategory == PC_Task)
	{
		return PCC_Task;
	}

	return GetDefault<UEdGraphSchema_K2>()->GetPinTypeColor(InPinType);
}

void USceneStateMachineGraphSchema::GetGraphDisplayInformation(const UEdGraph& InGraph, FGraphDisplayInfo& OutDisplayInfo) const
{
	OutDisplayInfo.PlainName = FText::FromName(InGraph.GetFName());
	OutDisplayInfo.DisplayName = OutDisplayInfo.PlainName;
	OutDisplayInfo.Tooltip = InGraph.GetClass()->GetToolTipText();
}

void USceneStateMachineGraphSchema::BreakNodeLinks(UEdGraphNode& InTargetNode) const
{
	const FScopedTransaction Transaction(LOCTEXT("BreakNodeLinks", "Break Node Links"));
	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNodeChecked(&InTargetNode);

	Super::BreakNodeLinks(InTargetNode);

	FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
}

void USceneStateMachineGraphSchema::BreakPinLinks(UEdGraphPin& InTargetPin, bool bInSendsNodeNotification) const
{
	const FScopedTransaction Transaction(LOCTEXT("BreakPinLinks", "Break Pin Links"));
	UBlueprint* const Blueprint = FBlueprintEditorUtils::FindBlueprintForNodeChecked(InTargetPin.GetOwningNode());

	Super::BreakPinLinks(InTargetPin, bInSendsNodeNotification);

	FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
}

void USceneStateMachineGraphSchema::BreakSinglePinLink(UEdGraphPin* InSourcePin, UEdGraphPin* InTargetPin) const
{
	check(InSourcePin && InTargetPin);

	const FScopedTransaction Transaction(LOCTEXT("BreakSinglePinLink", "Break Pin Link"));
	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNodeChecked(InTargetPin->GetOwningNode());

	Super::BreakSinglePinLink(InSourcePin, InTargetPin);

	FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
}

void USceneStateMachineGraphSchema::CreateConnectionWithTransition(const FTransitionConnectionParams& InParams) const
{
	using namespace UE::SceneState::Graph;

	const FVector2D Location = 0.5 * (InParams.SourceNode->GetNodePosition() + InParams.TargetNode->GetNodePosition());

	USceneStateMachineTransitionNode* TransitionNode;
	TransitionNode = NewObject<USceneStateMachineTransitionNode>();
	TransitionNode = FStateMachineAction_NewNode::SpawnNode(InParams.SourceNode->GetGraph(), TransitionNode, /*SourcePin*/nullptr, Location, /*bSelectNewNode*/false);
	check(TransitionNode);

	if (InParams.SourcePin->Direction == EGPD_Output)
	{
		TransitionNode->CreateConnections(InParams.SourceNode, InParams.TargetNode);
	}
	else
	{
		TransitionNode->CreateConnections(InParams.TargetNode, InParams.SourceNode);
	}
}

void USceneStateMachineGraphSchema::GetGraphContextCoreActions(FGraphContextMenuBuilder& InContextMenuBuilder) const
{
	using namespace UE::SceneState;

	const FString CoreCategory; // Category left empty
	constexpr int32 CoreSectionId = 1; 

	// Add State
	Graph::Private::AddAction<Graph::FStateMachineAction_NewNode>(InContextMenuBuilder
		, CoreCategory
		, NewObject<USceneStateMachineStateNode>(InContextMenuBuilder.OwnerOfTemporaries)
		, FText::GetEmpty()
		, LOCTEXT("AddState", "Add State")
		, LOCTEXT("AddStateTooltip", "A new state")
		, CoreSectionId);

	// Add Conduit
	Graph::Private::AddAction<Graph::FStateMachineAction_NewNode>(InContextMenuBuilder
		, CoreCategory
		, NewObject<USceneStateMachineConduitNode>(InContextMenuBuilder.OwnerOfTemporaries)
		, FText::GetEmpty()
		, LOCTEXT("AddConduit", "Add Conduit")
		, LOCTEXT("AddConduitTooltip", "Add new conduit")
		, CoreSectionId);

	// Add Entry Point (only if it doesn't already exist)
	bool bHasEntryNode = InContextMenuBuilder.CurrentGraph->Nodes.ContainsByPredicate(
		[](UEdGraphNode* InNode)
		{
			return InNode && InNode->IsA<USceneStateMachineEntryNode>();
		});

	if (!bHasEntryNode)
	{
		Graph::Private::AddAction<Graph::FStateMachineAction_NewNode>(InContextMenuBuilder
			, CoreCategory
			, NewObject<USceneStateMachineEntryNode>(InContextMenuBuilder.OwnerOfTemporaries)
			, FText::GetEmpty()
			, LOCTEXT("AddEntry", "Add Entry Point")
			, LOCTEXT("AddEntryTooltip", "Define the state machine's entry point")
			, CoreSectionId);
	}

	// Add Exit Point
	Graph::Private::AddAction<Graph::FStateMachineAction_NewNode>(InContextMenuBuilder
		, CoreCategory
		, NewObject<USceneStateMachineExitNode>(InContextMenuBuilder.OwnerOfTemporaries)
		, FText::GetEmpty()
		, LOCTEXT("AddExit", "Add Exit Point")
		, LOCTEXT("AddExitTooltip", "Define a state machine's exit point")
		, CoreSectionId);

	// Add Comment
	if (!InContextMenuBuilder.FromPin)
	{
		Graph::Private::AddAction<Graph::FStateMachineAction_NewComment>(InContextMenuBuilder
			, CoreCategory
			, FText::GetEmpty()
			, LOCTEXT("AddComment", "Add Comment")
			, LOCTEXT("CreateCommentSelectionTooltip", "Create a resizeable comment box around selected nodes.")
			, CoreSectionId);
	}
}

void USceneStateMachineGraphSchema::GetGraphContextTaskActions(FGraphContextMenuBuilder& InContextMenuBuilder) const
{
	using namespace UE::SceneState;

	const ISceneStateSchemaProvider* const SchemaProvider = Cast<ISceneStateSchemaProvider>(FBlueprintEditorUtils::FindBlueprintForGraph(InContextMenuBuilder.CurrentGraph));
	if (!SchemaProvider)
	{
		return;
	}

	const USceneStateSchema* const SceneStateSchema = SchemaProvider->GetSceneStateSchemaClass().GetDefaultObject();
	if (!SceneStateSchema)
	{
		return;
	}

	const FString TaskCategory; // Category left empty
	constexpr int32 TaskSectionId = 0;

	const bool bContextObjectImplementsWorld = 
		[SceneStateSchema] 
		{
			const UObject* const DefaultContextObject = SceneStateSchema->GetContextObjectClass().GetDefaultObject();
			return DefaultContextObject && DefaultContextObject->ImplementsGetWorld();
		}();

	// Buffer for task struct allocation to minimize memory allocations
	TArray<uint8, TInlineAllocator<64>> TaskBuffer;

	// C++ Tasks
	for (UScriptStruct* Struct : TObjectRange<UScriptStruct>())
	{
		constexpr bool bIncludeSuperStructs = true;
		if (Editor::StructHasMetaData(Struct, Editor::MD_Hidden, !bIncludeSuperStructs) || !Struct->IsChildOf<FSceneStateTask>())
		{
			continue;
		}

		if (!SceneStateSchema->IsTaskStructAllowed(Struct))
		{
			continue;
		}

		// If context does not implement world, skip structs that require world context
		if (!bContextObjectImplementsWorld && Editor::StructHasMetaData(Struct, Editor::MD_RequiresContextWorld, bIncludeSuperStructs))
		{
			continue;
		}

		// For structs that implement WithSupportSchema, create an instance of the struct and check for support
		if (Editor::StructHasMetaData(Struct, Editor::MD_WithSupportsSchema, bIncludeSuperStructs))
		{
			const int32 StructSize = Struct->GetStructureSize(); // same as Align(PropertiesSize, MinAlignment)
			const int32 MinAlignment = Struct->GetMinAlignment();

			TaskBuffer.SetNum(StructSize + MinAlignment, EAllowShrinking::No);

			uint8* const StructBytes = Align(TaskBuffer.GetData(), MinAlignment);
			Struct->InitializeStruct(StructBytes);

			const FSceneStateTask* Task = reinterpret_cast<const FSceneStateTask*>(StructBytes);
			const bool bTaskSupportsSchema = Task->SupportsSchema(SceneStateSchema);

			Struct->DestroyStruct(StructBytes);

			if (!bTaskSupportsSchema)
			{
				continue;
			}
		}

		Graph::Private::AddAction<Graph::FStateMachineAction_NewTask>(InContextMenuBuilder, TaskCategory, Struct, TaskSectionId);
	}

	// Blueprint Tasks
	{
		const FTopLevelAssetPath TaskBlueprintPath(TEXT("/Script/SceneStateBlueprint.SceneStateTaskBlueprint"));

		TArray<FAssetData> TaskAssets;
		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(FName("AssetRegistry")).Get();
		AssetRegistry.GetAssetsByClass(TaskBlueprintPath, TaskAssets, /*bSearchSubClasses*/true);

		for (const FAssetData& TaskAsset : TaskAssets)
		{
			if (SceneStateSchema->IsTaskAssetAllowed(TaskAsset))
			{
				Graph::Private::AddAction<Graph::FStateMachineAction_NewBlueprintTask>(InContextMenuBuilder, TaskCategory, TaskAsset, TaskSectionId);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
