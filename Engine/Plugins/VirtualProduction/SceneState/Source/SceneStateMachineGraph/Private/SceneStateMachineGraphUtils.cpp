// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateMachineGraphUtils.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Nodes/SceneStateMachineNode.h"
#include "PropertyBagDetails.h"

namespace UE::SceneState::Graph
{

bool CanDirectlyRemoveGraph(UEdGraph* InGraph)
{
	if (!InGraph || !InGraph->bAllowDeletion)
	{
		return false;
	}

	if (USceneStateMachineNode* const ParentNode = Cast<USceneStateMachineNode>(InGraph->GetOuter()))
	{
		return !ParentNode->GetBoundGraphs().Contains(InGraph);
	}

	return true;
}

void RemoveGraph(UEdGraph* InGraph)
{
	if (!InGraph)
	{
		return;
	}

	if (UEdGraph* const ParentGraph = InGraph->GetTypedOuter<UEdGraph>())
	{
		ParentGraph->SubGraphs.Remove(InGraph);
	}

	if (UBlueprint* const Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(InGraph))
	{
		Blueprint->LastEditedDocuments.RemoveAll(
			[InGraph](const FEditedDocumentInfo& InEditedDocumentInfo)
			{
				UObject* EditedObject = InEditedDocumentInfo.EditedObjectPath.ResolveObject();
				return EditedObject && (EditedObject == InGraph || EditedObject->IsIn(InGraph));
			});
	}
}

FEdGraphPinType GetPropertyDescAsPin(const FPropertyBagPropertyDesc& InPropertyDesc)
{
	FEdGraphPinType PinType = UE::StructUtils::GetPropertyDescAsPin(InPropertyDesc);

	// Change enum category to byte. Even though enum is more accurate, existing code in blueprint API only checks for PC_Byte.
	// Cases like UEdGraphSchema_K2::GetPropertyCategoryInfo also set the pin category to byte for enum properties.
	if (PinType.PinCategory == UEdGraphSchema_K2::PC_Enum)
	{
		PinType.PinCategory = UEdGraphSchema_K2::PC_Byte;
	}

	return PinType;
}

} //UE::SceneState::Graph
