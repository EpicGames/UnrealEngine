// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneStateMachineGraphEnums.h"

namespace UE::SceneState::Graph
{

/** Combines a source node type and a target type to a single keyable (type hashable) struct */
struct FNodeConnectionType
{
	FNodeConnectionType() = default;

	explicit FNodeConnectionType(EStateMachineNodeType InSourceType, EStateMachineNodeType InTargetType)
		: SourceType(InSourceType)
		, TargetType(InTargetType)
	{
	}

	friend uint32 GetTypeHash(const FNodeConnectionType& InNodeConnectionType)
	{
		return InNodeConnectionType.AsNumber();
	}

	bool operator==(const FNodeConnectionType& InOther) const
	{
		return AsNumber() == InOther.AsNumber();
	}

	uint16 AsNumber() const
	{
		return static_cast<uint16>(SourceType) << 8 | static_cast<uint16>(TargetType);
	}

private:
	EStateMachineNodeType SourceType = EStateMachineNodeType::Unspecified;
	EStateMachineNodeType TargetType = EStateMachineNodeType::Unspecified;
};

} // UE::SceneState::Graph
