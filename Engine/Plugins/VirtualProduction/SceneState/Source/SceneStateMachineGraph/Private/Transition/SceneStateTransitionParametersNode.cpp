// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transition/SceneStateTransitionParametersNode.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "Kismet2/KismetDebugUtilities.h"
#include "Nodes/SceneStateMachineTransitionNode.h"
#include "PropertyBagDetails.h"
#include "SceneStateMachineGraphUtils.h"
#include "SceneStateTransitionGraphSchema.h"

#define LOCTEXT_NAMESPACE "SceneStateTransitionParametersNode"

USceneStateTransitionParametersNode::USceneStateTransitionParametersNode()
{
	bIsEditable = false;

	OnParametersChangedHandle = USceneStateMachineTransitionNode::OnParametersChanged().AddUObject(this
		, &USceneStateTransitionParametersNode::OnTransitionParametersChanged);
}

void USceneStateTransitionParametersNode::OnTransitionParametersChanged(USceneStateMachineTransitionNode& InTransitionNode)
{
	if (&InTransitionNode == GetTypedOuter<USceneStateMachineTransitionNode>())
	{
		ReconstructNode();
	}
}

void USceneStateTransitionParametersNode::BuildParameterPins()
{
	const USceneStateMachineTransitionNode* TransitionNode = GetTypedOuter<USceneStateMachineTransitionNode>();
	if (!TransitionNode)
	{
		return;
	}

	TConstArrayView<FPropertyBagPropertyDesc> PropertyDescs;
	if (const UPropertyBag* PropertyBag = TransitionNode->GetParameters().GetPropertyBagStruct())
	{
		PropertyDescs = PropertyBag->GetPropertyDescs();
	}

	// No need to rebuild pins if already matching the provided structure
	if (IsStructurallyEqual(PropertyDescs))
	{
		return;
	}

	ClearPins();

	for (const FPropertyBagPropertyDesc& PropertyDesc : PropertyDescs)
	{
		const FEdGraphPinType PinType = UE::SceneState::Graph::GetPropertyDescAsPin(PropertyDesc);
		CreateUserDefinedPin(PropertyDesc.Name, PinType, EGPD_Output, /*bUseUniqueName*/false);
	}
}

UEdGraphPin* USceneStateTransitionParametersNode::CreatePinFromUserDefinition(const TSharedPtr<FUserPinInfo> InNewPinInfo)
{
	UEdGraphPin* NewPin = CreatePin(EGPD_Output, InNewPinInfo->PinType, InNewPinInfo->PinName);
	check(NewPin);

	const USceneStateTransitionGraphSchema* Schema = GetDefault<USceneStateTransitionGraphSchema>();
	Schema->SetPinAutogeneratedDefaultValue(NewPin, InNewPinInfo->PinDefaultValue);

	return NewPin;
}

bool USceneStateTransitionParametersNode::CanCreateUserDefinedPin(const FEdGraphPinType& InPinType, EEdGraphPinDirection InDesiredDirection, FText& OutErrorMessage)
{
	OutErrorMessage = LOCTEXT("NotEditableError", "Cannot edit this node!");
	return false;
}

bool USceneStateTransitionParametersNode::ModifyUserDefinedPinDefaultValue(TSharedPtr<FUserPinInfo> InPinInfo, const FString& InNewDefaultValue)
{
	if (Super::ModifyUserDefinedPinDefaultValue(InPinInfo, InNewDefaultValue))
	{
		const USceneStateTransitionGraphSchema* Schema = GetDefault<USceneStateTransitionGraphSchema>();
		Schema->HandleParameterDefaultValueChanged(this);
		return true;
	}
	return false;
}

bool USceneStateTransitionParametersNode::IsNodePure() const
{
	return true;
}

void USceneStateTransitionParametersNode::GetMenuActions(FBlueprintActionDatabaseRegistrar& InActionRegistrar) const
{
	UClass* ActionKey = GetClass();

	if (InActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner);
		InActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

FText USceneStateTransitionParametersNode::GetNodeTitle(ENodeTitleType::Type InTitleType) const
{
	return GetClass()->GetDisplayNameText();
}

void USceneStateTransitionParametersNode::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();
	BuildParameterPins();
}

void USceneStateTransitionParametersNode::BeginDestroy()
{
	Super::BeginDestroy();

	USceneStateMachineTransitionNode::OnParametersChanged().Remove(OnParametersChangedHandle);
	OnParametersChangedHandle.Reset();
}

bool USceneStateTransitionParametersNode::IsStructurallyEqual(TConstArrayView<FPropertyBagPropertyDesc> InPropertyDescs) const
{
	if (InPropertyDescs.Num() != Pins.Num())
	{
		return false;
	}

	for (int32 Index = 0; Index < Pins.Num(); ++Index)
	{
		const UEdGraphPin* const Pin = Pins[Index];
		if (!Pin)
		{
			return false;
		}

		const FPropertyBagPropertyDesc& PropertyDesc = InPropertyDescs[Index];
		if (PropertyDesc.Name != Pin->PinName)
		{
			return false;
		}

		FPropertyBagPropertyDesc PinDesc;
		UE::StructUtils::SetPropertyDescFromPin(PinDesc, Pin->PinType);

		if (!PropertyDesc.CompatibleType(PinDesc))
		{
			return false;
		}
	}

	return true;
}

void USceneStateTransitionParametersNode::ClearPins()
{
	const UBlueprint* Blueprint = GetBlueprint();
	for (UEdGraphPin* Pin : Pins)
	{
		Pin->Modify();
		Pin->MarkAsGarbage();

		if (Blueprint)
		{
			FKismetDebugUtilities::RemovePinWatch(Blueprint, Pin);
		}
	}

	Pins.Reset();
	UserDefinedPins.Reset();
}

#undef LOCTEXT_NAMESPACE
