// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraphSchema_K2_Actions.h"
#include "Textures/SlateIcon.h"

#define UE_API SCENESTATEMACHINEGRAPH_API

class USceneStateMachineNode;

namespace UE::SceneState::Graph
{

struct UE_INTERNAL FStateMachineAction_Node : public FEdGraphSchemaAction
{
	FStateMachineAction_Node() = default;

	struct FArguments
	{
		USceneStateMachineNode* Node = nullptr;
		FText Category;
		int32 SectionID;
	};
	UE_API explicit FStateMachineAction_Node(const FArguments& InArgs);

	UE_API static FName StaticGetTypeId();

	USceneStateMachineNode* GetNode() const
	{
		return NodeWeak.Get();
	}

	//~ Begin FEdGraphSchemaAction
	UE_API virtual bool IsParentable() const override;
	UE_API virtual bool CanBeRenamed() const override;
	UE_API virtual FName GetTypeId() const override;
	UE_API virtual FEdGraphSchemaActionDefiningObject GetPersistentItemDefiningObject() const override;
	//~ End FEdGraphSchemaAction

private:
	TWeakObjectPtr<USceneStateMachineNode> NodeWeak;
};

} // UE::SceneState::Graph

#undef UE_API
