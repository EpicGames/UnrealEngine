// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISceneStateTransitionGraphProvider.h"
#include "SceneStateMachineNode.h"
#include "SceneStateMachineConduitNode.generated.h"

UCLASS(MinimalAPI)
class USceneStateMachineConduitNode : public USceneStateMachineNode, public ISceneStateTransitionGraphProvider
{
	GENERATED_BODY()

public:
	USceneStateMachineConduitNode();

	bool ShouldWaitForTasksToFinish() const
	{
		return bWaitForTasksToFinish;
	}

	//~ Begin USceneStateMachineNode
	virtual UEdGraph* CreateBoundGraphInternal() override;
	//~ End USceneStateMachineNode

	//~ Begin UEdGraphNode
	virtual void AllocateDefaultPins() override;
	virtual bool CanDuplicateNode() const override;
	virtual void PostPasteNode() override;
	virtual void PostPlacedNewNode() override;
	//~ End UEdGraphNode

	//~ Begin ISceneStateTransitionGraphProvider
	virtual FText GetTitle() const override;
	virtual bool IsBoundToGraphLifetime(UEdGraph& InGraph) const override;
	virtual UEdGraphNode* AsNode() override;
	//~ End ISceneStateTransitionGraphProvider

private:
	UPROPERTY(EditAnywhere, Category = "Conduit")
	bool bWaitForTasksToFinish = true;
};
