// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneStateMachineNode.h"
#include "SceneStateMachineEntryNode.generated.h"

#define UE_API SCENESTATEMACHINEGRAPH_API

class USceneStateMachineStateNode;

UCLASS(MinimalAPI, meta=(ToolTip="Entry point for state machine"))
class USceneStateMachineEntryNode : public USceneStateMachineNode
{
	GENERATED_BODY()

public:
	USceneStateMachineEntryNode();

	/** Retrieves the first State Node in the State Machine */
	UE_API USceneStateMachineStateNode* GetStateNode() const;

	//~ Begin USceneStateMachineNode
	virtual UEdGraphPin* GetInputPin() const override;
	virtual UEdGraphPin* GetOutputPin() const override;
	virtual bool HasValidPins() const override;
	//~ End USceneStateMachineNode

	//~ Begin UEdGraphNode
	virtual void AllocateDefaultPins() override;
	//~ End UEdGraphNode
};

#undef UE_API
