// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneStateMachineNode.h"
#include "SceneStateMachineExitNode.generated.h"

UCLASS(MinimalAPI, meta=(ToolTip="Exit point for a state machine"))
class USceneStateMachineExitNode : public USceneStateMachineNode
{
	GENERATED_BODY()

public:
	USceneStateMachineExitNode();

	//~ Begin USceneStateMachineNode
	virtual UEdGraphPin* GetInputPin() const override;
	virtual UEdGraphPin* GetOutputPin() const override;
	virtual bool HasValidPins() const override;
	//~ End USceneStateMachineNode

	//~ Begin UEdGraphNode
	virtual void AllocateDefaultPins() override;
	//~ End UEdGraphNode
};
