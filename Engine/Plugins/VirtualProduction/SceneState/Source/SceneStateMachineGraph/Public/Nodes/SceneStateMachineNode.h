// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneStateMachineGraphEnums.h"
#include "EdGraph/EdGraphNode.h"
#include "SceneStateMachineNode.generated.h"

#define UE_API SCENESTATEMACHINEGRAPH_API

class USceneStateMachineTransitionNode;

UCLASS(MinimalAPI, Abstract)
class USceneStateMachineNode : public UEdGraphNode
{
	GENERATED_BODY()

public:
	/** Gathers and returns all the transition nodes connected to this state (including bi-directional transitions) */
    UE_API TArray<USceneStateMachineTransitionNode*> GatherTransitions(bool bInSortList = false) const;

	FName GetNodeName() const
	{
		return NodeName;
	}

	UE::SceneState::Graph::EStateMachineNodeType GetNodeType() const
	{
		return NodeType;
	}

	UE_API UEdGraph* GetBoundGraph() const;

	TConstArrayView<UEdGraph*> GetBoundGraphs() const
	{
		return BoundGraphs;
	}

	FVector2D GetNodePosition() const
	{
		return FVector2D(NodePosX, NodePosY);
	}

	virtual UEdGraphPin* GetInputPin() const
	{
		return Pins[0];
	}

	virtual UEdGraphPin* GetOutputPin() const
	{
		return Pins[1];
	}

	virtual bool HasValidPins() const
	{
		return GetInputPin() && GetOutputPin();
	}

	bool ConditionallyCreateBoundGraph();

	virtual UEdGraph* CreateBoundGraphInternal()
	{
		return nullptr;
	}

	/** Removes null graphs and graphs not outered to this node */
	void CleanInvalidBoundGraphs();

	//~ Begin UEdGraphNode
	virtual void DestroyNode() override;
	virtual void PostPasteNode() override;
	virtual void OnRenameNode(const FString& InNodeName);
	virtual void AutowireNewNode(UEdGraphPin* InSourcePin) override; 
	virtual FText GetNodeTitle(ENodeTitleType::Type InTitleType) const override;
	virtual TArray<UEdGraph*> GetSubGraphs() const override;
	virtual UObject* GetJumpTargetForDoubleClick() const override;
	virtual bool CanJumpToDefinition() const override;
	virtual void JumpToDefinition() const override;
	virtual bool CanCreateUnderSpecifiedSchema(const UEdGraphSchema* InSchema) const override;
	virtual TSharedPtr<INameValidatorInterface> MakeNameValidator() const override;
	//~ End UEdGraphNode

	//~ Begin UObject
	virtual void PostLoad() override;
	//~ End UObject

protected:
	/** Finds the given pin and sets bHidden true to it*/
	void HidePins(TConstArrayView<FName> InPinNames);

	void GenerateNewNodeName();

	UPROPERTY()
	FName NodeName;

	UE::SceneState::Graph::EStateMachineNodeType NodeType = UE::SceneState::Graph::EStateMachineNodeType::Unspecified;

	UPROPERTY(Instanced)
	TArray<TObjectPtr<UEdGraph>> BoundGraphs;
};

#undef UE_API
