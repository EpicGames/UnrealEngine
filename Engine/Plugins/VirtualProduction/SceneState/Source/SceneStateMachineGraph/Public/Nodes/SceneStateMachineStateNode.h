// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISceneStateEventHandlerProvider.h"
#include "SceneStateEventHandler.h"
#include "SceneStateMachineNode.h"
#include "StructUtils/PropertyBag.h"
#include "SceneStateMachineStateNode.generated.h"

#define UE_API SCENESTATEMACHINEGRAPH_API

UCLASS(MinimalAPI, meta=(ToolTip="State node in a State Machine"))
class USceneStateMachineStateNode : public USceneStateMachineNode, public ISceneStateEventHandlerProvider
{
	GENERATED_BODY()

public:
	using FOnParametersChanged = TMulticastDelegate<void(USceneStateMachineStateNode*)>;

	USceneStateMachineStateNode();

	UE_INTERNAL UE_API static FOnParametersChanged::RegistrationType& OnParametersChanged();

	UE_INTERNAL UE_API void NotifyParametersChanged();

	UEdGraphPin* GetTaskPin() const
	{
		return Pins[2];
	}

	TConstArrayView<FSceneStateEventHandler> GetEventHandlers() const
	{
		return EventHandlers;
	}

	/** Gets the index to the state this node compiled to last */
	UE_INTERNAL UE_API TOptional<uint16> GetCompiledIndex() const;

	/** Sets the index to the state this node compiled to last */
	UE_INTERNAL UE_API void SetCompiledIndex(uint16 InCompiledIndex);

	//~ Begin ISceneStateEventHandlerProvider
	virtual bool FindEventHandlerId(const FSceneStateEventSchemaHandle& InEventSchemaHandle, FGuid& OutHandlerId) const override;
	//~ End ISceneStateEventHandlerProvider

	//~ Begin USceneStateMachineNode
	virtual bool HasValidPins() const override;
	virtual UEdGraph* CreateBoundGraphInternal() override;
	//~ End USceneStateMachineNode

	//~ Begin UEdGraphNode
	virtual void AllocateDefaultPins() override;
	virtual bool CanDuplicateNode() const override;
	virtual void PostPasteNode() override;
	virtual void PostPlacedNewNode() override;
	//~ End UEdGraphNode

	//~ Begin UObject
	virtual void PostInitProperties() override;
	virtual void PostLoad() override;
	virtual void PostDuplicate(bool bInDuplicateForPIE) override;
	virtual void PostEditImport() override;
	//~ End UObject

	/** Called to set a new unique id for parameters (e.g. after duplicating) */
	void GenerateNewParametersId();

	/** Identifier for the Parameters Struct Id */
	UPROPERTY(VisibleAnywhere, Category="Parameters")
	FGuid ParametersId;

	/** State parameters */
	UPROPERTY(EditAnywhere, Category="Parameters")
	FInstancedPropertyBag Parameters;

	/** Deprecated: Graphs are now managed in the Node Base class */
	UPROPERTY()
	TObjectPtr<UEdGraph> MainGraph;

	UPROPERTY(EditAnywhere, Category="Events", meta=(NoBinding))
	TArray<FSceneStateEventHandler> EventHandlers;

private:
	/** Index to the state this state node compiled to last. */
	UPROPERTY()
	TOptional<uint16> CompiledIndex;

	static FOnParametersChanged OnParametersChangedDelegate;
};

#undef UE_API
