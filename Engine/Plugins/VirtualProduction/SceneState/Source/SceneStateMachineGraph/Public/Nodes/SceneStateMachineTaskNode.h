// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneStateMachineNode.h"
#include "StructUtils/InstancedStruct.h"
#include "Tasks/SceneStateTask.h"
#include "Templates/SubclassOf.h"
#include "SceneStateMachineTaskNode.generated.h"

#define UE_API SCENESTATEMACHINEGRAPH_API

class USceneStateBlueprintableTask;
struct FSceneStateTaskDescContext;
struct FSceneStateTaskDescMutableContext;

namespace UE::SceneState
{
	enum class ETaskObjectType : uint8;
	struct FStructIdChange;
}

UCLASS(MinimalAPI)
class USceneStateMachineTaskNode : public USceneStateMachineNode
{
	GENERATED_BODY()

public:
	USceneStateMachineTaskNode();

	TConstStructView<FSceneStateTask> GetTask() const
	{
		return Task;
	}

	TStructView<FSceneStateTaskInstance> GetTaskInstance()
	{
		return TaskInstance;
	}

	TMulticastDelegateRegistration<void()>& OnPostEditTask() const
	{
		return OnPostEditTaskDelegate;
	}

	UE_API bool SetTaskStruct(const UScriptStruct* InTaskStruct);

	UE_API void SetTaskBlueprintClass(TSubclassOf<USceneStateBlueprintableTask> InTaskClass);

	UE_API bool FindDataViewById(const FGuid& InStructId, FStructView& OutDataView);

	const FGuid& GetTaskId() const
	{
		return TaskId;
	}

	/** Gets the index to the task this node compiled to last */
	UE_INTERNAL UE_API TOptional<uint16> GetCompiledIndex() const;

	/** Sets the index to the task this node compiled to last */
	UE_INTERNAL UE_API void SetCompiledIndex(uint16 InCompiledIndex);

	//~ Begin UEdGraphNode
	virtual void AllocateDefaultPins() override;
	virtual void OnRenameNode(const FString& InNodeName);
	virtual FText GetNodeTitle(ENodeTitleType::Type InTitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual bool CanDuplicateNode() const override;
	virtual void PostPasteNode() override;
	virtual UObject* GetJumpTargetForDoubleClick() const override;
	virtual TSharedPtr<INameValidatorInterface> MakeNameValidator() const override;
	//~ End UEdGraphNode

	//~ Begin UObject
	virtual void PostLoad() override;
	virtual void BeginDestroy() override;
	virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& InPropertyChangedEvent) override;
	virtual void PostDuplicate(bool bInDuplicateForPIE) override;
	virtual void PostEditImport() override;
	//~ End UObject

	static FName GetTaskPropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(USceneStateMachineTaskNode, Task);
	}

	static FName GetTaskInstancePropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(USceneStateMachineTaskNode, TaskInstance);
	}

private:
	/** Determines whether the changed property comes from the task or task instance object, or none of them */
	static UE::SceneState::ETaskObjectType GetChangedObject(FPropertyChangedChainEvent& InPropertyChangedEvent);

	/** Called on PostEditChangeChainProperty when the property that changed came from a Task or task instance object */
	void PostEditChangeTask(FPropertyChangedChainEvent& InPropertyChangedEvent, UE::SceneState::ETaskObjectType InChangedObject);

	/** Create context for this task node */
	FSceneStateTaskDescContext MakeContext() const;

	/** Creates a mutable context for this task node */
	FSceneStateTaskDescMutableContext MakeMutableContext();

	/** Rebuilds the task instance by recreating any instanced objects / or re-initializing any 'instance' guid property */
	void RebuildTaskInstance();

	/** Called when the task has changed its type, or a property related to the task / task instance has changed. */
	void OnTaskChanged();

	/** Called when a struct id has changed */
	void OnStructIdChanged(const UE::SceneState::FStructIdChange& InChange);

	/** Generates a new task id for this task node, notifying the binding owner of a struct id change */
	void GenerateNewTaskId();

	UPROPERTY(EditAnywhere, Category = "Task", meta=(StructTypeConst))
	TInstancedStruct<FSceneStateTask> Task;

	UPROPERTY(EditAnywhere, Category="Task", meta=(StructTypeConst))
	TInstancedStruct<FSceneStateTaskInstance> TaskInstance;

	/** Index to the task this task node compiled to last. */
	UPROPERTY()
	TOptional<uint16> CompiledIndex;

	UPROPERTY()
	FGuid TaskId;

	mutable TMulticastDelegate<void()> OnPostEditTaskDelegate;

	mutable FText CachedTaskDisplayName;
};

#undef UE_API
