// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISceneStateTransitionGraphProvider.h"
#include "SceneStateMachineNode.h"
#include "StructUtils/PropertyBag.h"
#include "SceneStateMachineTransitionNode.generated.h"

#define UE_API SCENESTATEMACHINEGRAPH_API

UCLASS(MinimalAPI)
class USceneStateMachineTransitionNode : public USceneStateMachineNode, public ISceneStateTransitionGraphProvider
{
	GENERATED_BODY()

public:
	USceneStateMachineTransitionNode();

	using FOnParametersChanged = TMulticastDelegate<void(USceneStateMachineTransitionNode&)>;

	static FOnParametersChanged::RegistrationType& OnParametersChanged();

	void NotifyParametersChanged();

	UE_API static TArray<USceneStateMachineTransitionNode*> GetTransitionsToRelink(UEdGraphPin* InSourcePin
		, UEdGraphPin* InOldTargetPin
		, TConstArrayView<UEdGraphNode*> InSelectedGraphNodes);

	UE_API USceneStateMachineNode* GetSourceNode() const;

	UE_API USceneStateMachineNode* GetTargetNode() const;

	int32 GetPriority() const
	{
		return Priority;
	}

	bool ShouldWaitForTasksToFinish() const
	{
		return bWaitForTasksToFinish;
	}

	const FGuid& GetParametersId() const
	{
		return ParametersId;
	}

	const FInstancedPropertyBag& GetParameters() const
	{
		return Parameters;
	}

	FInstancedPropertyBag& GetParametersMutable()
	{
		return Parameters;
	}

	void CreateConnections(USceneStateMachineNode* InSourceState, USceneStateMachineNode* InTargetState);

	/**
	 * Relink transition head (where the arrow is of a state transition) to a new state.
	 * @param InNewTargetState The new transition target
	 */
	void RelinkHead(USceneStateMachineNode* InNewTargetState);

	//~ Begin USceneStateMachineNode
	virtual UEdGraph* CreateBoundGraphInternal() override;
	//~ End USceneStateMachineNode

	//~ Begin UEdGraphNode
	virtual void AllocateDefaultPins() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type InTitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual void PinConnectionListChanged(UEdGraphPin* InPin) override;
	virtual bool CanDuplicateNode() const override { return true; }
	virtual void PostPasteNode() override;
	virtual void PostPlacedNewNode() override;
	//~ End UEdGraphNode

	//~ Begin ISceneStateTransitionGraphProvider
	virtual FText GetTitle() const override;
	virtual bool IsBoundToGraphLifetime(UEdGraph& InGraph) const override;
	virtual UEdGraphNode* AsNode() override;
	//~ End ISceneStateTransitionGraphProvider

	//~ Begin UObject
	virtual void PostInitProperties() override;
	virtual void PostLoad() override;
	virtual void PostDuplicate(bool bInDuplicateForPIE) override;
	virtual void PostEditImport() override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
	//~ End Uobject

	static FName GetParametersIdName()
	{
		return GET_MEMBER_NAME_CHECKED(USceneStateMachineTransitionNode, ParametersId);
	}

	static FName GetParametersName()
	{
		return GET_MEMBER_NAME_CHECKED(USceneStateMachineTransitionNode, Parameters);
	}

private:
	/** Called to set a new unique id for parameters (e.g. after duplicating) */
	void GenerateNewParametersId();

	static FOnParametersChanged OnParametersChangedDelegate;

	/** Deprecated: Graphs are now managed in the Node Base class */
	UPROPERTY()
	TObjectPtr<UEdGraph> TransitionGraph;

	/**
	 * Priority of Transition
	 * @note the lower the number, the higher the priority
	 */
	UPROPERTY(EditAnywhere, Category = "Transitions")
	int32 Priority = 0;

	UPROPERTY(EditAnywhere, Category = "Transitions")
	bool bWaitForTasksToFinish = true;

	/** Identifier for the Parameters Struct Id */
	UPROPERTY(VisibleAnywhere, Category = "Parameters")
	FGuid ParametersId;

	/** Parameters to feed into the Transition via the "Transition Parameters" node */
	UPROPERTY(EditAnywhere, Category = "Parameters")
	FInstancedPropertyBag Parameters;
};

#undef UE_API
