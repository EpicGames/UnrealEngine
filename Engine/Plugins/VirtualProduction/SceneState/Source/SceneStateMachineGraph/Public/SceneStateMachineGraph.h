// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraph.h"
#include "SceneStateEnums.h"
#include "StructUtils/PropertyBag.h"
#include "SceneStateMachineGraph.generated.h"

#define UE_API SCENESTATEMACHINEGRAPH_API

class USceneStateMachineEntryNode;
class USceneStateMachineStateNode;

UCLASS(MinimalAPI, HideCategories=(Graph))
class USceneStateMachineGraph : public UEdGraph
{
	GENERATED_BODY()

public:
	using FOnParametersChanged = TMulticastDelegate<void(USceneStateMachineGraph*)>;

	USceneStateMachineGraph();

	UE_API static FOnParametersChanged::RegistrationType& OnParametersChanged();

	UE_API void NotifyParametersChanged();

	UE_API USceneStateMachineEntryNode* GetEntryNode() const;

	/** Returns the parent node if it's a state machine nested in a state node*/
	UE_API USceneStateMachineStateNode* GetParentStateNode() const;

	/** Returns whether the state machine is at the root level (i.e. directly under the blueprint and not nested under a state node) */
	UE_API bool IsRootStateMachine() const;

	/** Get the index to the state machine this node compiled to last */
	UE_INTERNAL UE_API TOptional<uint16> GetCompiledIndex() const;

	/** Sets the index to the state machine this node compiled to last */
	UE_INTERNAL UE_API void SetCompiledIndex(uint16 InCompiledIndex);

	//~ Begin UEdGraph
	UE_API virtual void AddNode(UEdGraphNode* InNodeToAdd, bool bInUserAction, bool bInSelectNewNode) override;
	//~ End UEdGraph

	//~ Begin UObject
	UE_API virtual void PostInitProperties() override;
	UE_API virtual void PostLoad() override;
	UE_API virtual void PostDuplicate(bool bInDuplicateForPIE) override;
	UE_API virtual void PostEditImport() override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
	//~ End UObject

	/** Called to set a new unique id for parameters (e.g. after duplicating) */
	void GenerateNewParametersId();

	/** Category of the State Machine */
	UPROPERTY(EditAnywhere, Category="State Machine")
	FText Category;

	/** Identifier for the Parameters Struct Id */
	UPROPERTY(VisibleAnywhere, Category="State Machine")
	FGuid ParametersId;

	UPROPERTY(EditAnywhere, Category="State Machine")
	FInstancedPropertyBag Parameters;

	/** The run-mode for the State Machine. Currently only applies to Top-Level State Machines */
	UPROPERTY(EditAnywhere, Category="State Machine")
	ESceneStateMachineRunMode RunMode = ESceneStateMachineRunMode::Auto;

private:
	/** Index to the state machine this graph compiled to last. */
	UPROPERTY()
	TOptional<uint16> CompiledIndex;

	static FOnParametersChanged OnParametersChangedDelegate;
};

#undef UE_API
