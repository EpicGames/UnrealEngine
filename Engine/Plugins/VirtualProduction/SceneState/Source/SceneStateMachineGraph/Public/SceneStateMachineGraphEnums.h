// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

namespace UE::SceneState::Graph
{

/** Type of state machine node */
enum class EStateMachineNodeType : uint8
{
	Unspecified,
	Entry,
	Exit,
	State,
	Transition,
	Task,
	Conduit,
};

} // UE::SceneState::Graph
