// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraphSchema.h"
#include "SceneStateMachineGraphSchema.generated.h"

#define UE_API SCENESTATEMACHINEGRAPH_API

class USceneStateMachineNode;
class USceneStateMachineStateNode;

UCLASS(MinimalAPI)
class USceneStateMachineGraphSchema : public UEdGraphSchema
{
	GENERATED_BODY()

public:
	/** Pin Names */
	UE_API static const FName PN_In;
	UE_API static const FName PN_Out;
	UE_API static const FName PN_Task;

	/** Pin Categories */
	UE_API static const FName PC_Transition;
	UE_API static const FName PC_Task;

	/** Pin Category Colors */
	UE_API static const FLinearColor PCC_Transition;
	UE_API static const FLinearColor PCC_Task;

	/** Attempts to find the State Node connected to the given Task Node */
	UE_API static USceneStateMachineStateNode* FindConnectedStateNode(const UEdGraphNode* InTaskNode);

	//~ Begin UEdGraphSchema
	virtual EGraphType GetGraphType(const UEdGraph* InGraph) const override;
	virtual void CreateDefaultNodesForGraph(UEdGraph& InGraph) const override;
	virtual void GetGraphContextActions(FGraphContextMenuBuilder& InContextMenuBuilder) const override;
	virtual void GetContextMenuActions(UToolMenu* InMenu, UGraphNodeContextMenuContext* InContext) const override;
	virtual const FPinConnectionResponse CanCreateConnection(const UEdGraphPin* InSourcePin, const UEdGraphPin* InTargetPin) const override;
	virtual bool TryCreateConnection(UEdGraphPin* InSourcePin, UEdGraphPin* InTargetPin) const override;
	virtual bool CreateAutomaticConversionNodeAndConnections(UEdGraphPin* InSourcePin, UEdGraphPin* InTargetPin) const override;
	virtual bool TryRelinkConnectionTarget(UEdGraphPin* InSourcePin, UEdGraphPin* InOldTargetPin, UEdGraphPin* InNewTargetPin, const TArray<UEdGraphNode*>& InSelectedGraphNodes) const override;
	virtual bool IsConnectionRelinkingAllowed(UEdGraphPin* InPin) const override;
	virtual const FPinConnectionResponse CanRelinkConnectionToPin(const UEdGraphPin* InOldSourcePin, const UEdGraphPin* InTargetPin) const override;
	virtual FLinearColor GetPinTypeColor(const FEdGraphPinType& InPinType) const override;
	virtual void GetGraphDisplayInformation(const UEdGraph& InGraph, FGraphDisplayInfo& OutDisplayInfo) const override;
	virtual void BreakNodeLinks(UEdGraphNode& InTargetNode) const override;
	virtual void BreakPinLinks(UEdGraphPin& InTargetPin, bool bInSendsNodeNotification) const override;
	virtual void BreakSinglePinLink(UEdGraphPin* InSourcePin, UEdGraphPin* InTargetPin) const override;
	//~ End UEdGraphSchema

	struct FTransitionConnectionParams
	{
		USceneStateMachineNode* SourceNode;
		USceneStateMachineNode* TargetNode;
		UEdGraphPin* SourcePin;
		UEdGraphPin* TargetPin;
	};
	void CreateConnectionWithTransition(const FTransitionConnectionParams& InParams) const;

private:
	/** Gathers the core actions for the context menu */
	void GetGraphContextCoreActions(FGraphContextMenuBuilder& InContextMenuBuilder) const;

	/** Gathers the task actions for the context menu */
	void GetGraphContextTaskActions(FGraphContextMenuBuilder& InContextMenuBuilder) const;
};

#undef UE_API
