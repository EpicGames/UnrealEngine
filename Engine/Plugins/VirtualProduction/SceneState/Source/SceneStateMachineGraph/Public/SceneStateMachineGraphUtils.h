// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#define UE_API SCENESTATEMACHINEGRAPH_API

class UEdGraph;
struct FEdGraphPinType;
struct FPropertyBagPropertyDesc;

namespace UE::SceneState::Graph
{
	UE_API bool CanDirectlyRemoveGraph(UEdGraph* InGraph);

	UE_API void RemoveGraph(UEdGraph* InGraph);

	/** Gets the given property desc as a pin compatible for blueprints */
	UE_API FEdGraphPinType GetPropertyDescAsPin(const FPropertyBagPropertyDesc& InPropertyDesc);

} // UE::SceneState::Graph

#undef UE_API
