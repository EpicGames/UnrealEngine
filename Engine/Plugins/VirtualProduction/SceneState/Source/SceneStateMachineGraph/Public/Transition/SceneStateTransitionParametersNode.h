// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "K2Node_EditablePinBase.h"
#include "SceneStateTransitionParametersNode.generated.h"

class USceneStateMachineTransitionNode;
struct FPropertyBagPropertyDesc;

UCLASS(MinimalAPI, DisplayName="Transition Parameters")
class USceneStateTransitionParametersNode : public UK2Node_EditablePinBase
{
	GENERATED_BODY()

public:
	USceneStateTransitionParametersNode();

	void OnTransitionParametersChanged(USceneStateMachineTransitionNode& InTransitionNode);

protected:
	/** Builds the pins matching the outer state machine graph's parameters */
	void BuildParameterPins();

	/** Returns whether the Pin structure is equal with the property desc structure (i.e. same count, same names and same types) */
	bool IsStructurallyEqual(TConstArrayView<FPropertyBagPropertyDesc> InPropertyDescs) const;

	/** Removes all the user pin definitions and pins from this node */
	void ClearPins();

	//~ Begin UK2Node_EditablePinBase
	virtual UEdGraphPin* CreatePinFromUserDefinition(const TSharedPtr<FUserPinInfo> InNewPinInfo) override;
	virtual bool CanCreateUserDefinedPin(const FEdGraphPinType& InPinType, EEdGraphPinDirection InDesiredDirection, FText& OutErrorMessage) override;
	virtual bool ModifyUserDefinedPinDefaultValue(TSharedPtr<FUserPinInfo> InPinInfo, const FString& InNewDefaultValue) override;
	//~ End UK2Node_EditablePinBase

	//~ Begin UK2Node
	virtual bool IsNodePure() const override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& InActionRegistrar) const override;
	//~ End UK2Node

	//~ Begin UEdGraphNode
	virtual FText GetNodeTitle(ENodeTitleType::Type InTitleType) const override;
	virtual void AllocateDefaultPins() override;
	//~ End UEdGraphNode

	//~ Begin UObject
	virtual void BeginDestroy() override;
	//~ End UObject

private:
	FDelegateHandle OnParametersChangedHandle;
};
