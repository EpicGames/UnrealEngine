// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SceneStateMachineGraph : ModuleRules
{
    public SceneStateMachineGraph(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "Core",
                "CoreUObject",
                "Engine",
                "SceneState",
                "SceneStateEvent",
                "SceneStateBinding",
                "SceneStateTransitionGraph",
            }
        );

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "BlueprintGraph",
                "Slate",
                "SlateCore",
                "ToolMenus",
            });

        if (Target.Type == TargetRules.TargetType.Editor)
        {
            PrivateDependencyModuleNames.AddRange(
                new string[]
                {
                    "GraphEditor",
                    "SceneStateEditor",
                    "StructUtilsEditor",
                    "UnrealEd",
                });
        }
    }
}
