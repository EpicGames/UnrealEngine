// Copyright Epic Games, Inc. All Rights Reserved.

#include "Functions/SceneStateBooleanFunctions.h"

#if WITH_EDITOR
const UScriptStruct* FSceneStateBooleanFunction::OnGetFunctionDataType() const
{
	return FInstanceDataType::StaticStruct();
}
#endif

void FSceneStateBooleanAndFunction::OnExecute(const FSceneStateExecutionContext& InContext, FStructView InFunctionInstance) const
{
	FInstanceDataType& InstanceData = InFunctionInstance.Get<FInstanceDataType>();
	InstanceData.bOutput = InstanceData.bLeft && InstanceData.bRight;
}

void FSceneStateBooleanOrFunction::OnExecute(const FSceneStateExecutionContext& InContext, FStructView InFunctionInstance) const
{
	FInstanceDataType& InstanceData = InFunctionInstance.Get<FInstanceDataType>();
	InstanceData.bOutput = InstanceData.bLeft || InstanceData.bRight;
}

void FSceneStateBooleanXorFunction::OnExecute(const FSceneStateExecutionContext& InContext, FStructView InFunctionInstance) const
{
	FInstanceDataType& InstanceData = InFunctionInstance.Get<FInstanceDataType>();
	InstanceData.bOutput = InstanceData.bLeft ^ InstanceData.bRight;
}

#if WITH_EDITOR
const UScriptStruct* FSceneStateBooleanNotFunction::OnGetFunctionDataType() const
{
	return FInstanceDataType::StaticStruct();
}
#endif

void FSceneStateBooleanNotFunction::OnExecute(const FSceneStateExecutionContext& InContext, FStructView InFunctionInstance) const
{
	FInstanceDataType& InstanceData = InFunctionInstance.Get<FInstanceDataType>();
	InstanceData.bOutput = !InstanceData.bInput;
}
