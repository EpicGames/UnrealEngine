// Copyright Epic Games, Inc. All Rights Reserved.

#include "Functions/SceneStateFloatFunctions.h"
#include "SceneStateExecutionContext.h"
#include "SceneStateTasksLog.h"

#if WITH_EDITOR
const UScriptStruct* FSceneStateDoubleFunction::OnGetFunctionDataType() const
{
	return FInstanceDataType::StaticStruct();
}
#endif

void FSceneStateAddDoubleFunction::OnExecute(const FSceneStateExecutionContext& InContext, FStructView InFunctionInstance) const
{
	FInstanceDataType& InstanceData = InFunctionInstance.Get<FInstanceDataType>();
	InstanceData.Output = InstanceData.Left + InstanceData.Right;
}

void FSceneStateSubtractDoubleFunction::OnExecute(const FSceneStateExecutionContext& InContext, FStructView InFunctionInstance) const
{
	FInstanceDataType& InstanceData = InFunctionInstance.Get<FInstanceDataType>();
	InstanceData.Output = InstanceData.Left - InstanceData.Right;
}

void FSceneStateMultiplyDoubleFunction::OnExecute(const FSceneStateExecutionContext& InContext, FStructView InFunctionInstance) const
{
	FInstanceDataType& InstanceData = InFunctionInstance.Get<FInstanceDataType>();
	InstanceData.Output = InstanceData.Left * InstanceData.Right;
}

void FSceneStateDivideDoubleFunction::OnExecute(const FSceneStateExecutionContext& InContext, FStructView InFunctionInstance) const
{
	FInstanceDataType& InstanceData = InFunctionInstance.Get<FInstanceDataType>();
	if (!FMath::IsNearlyZero(InstanceData.Right))
	{
		InstanceData.Output = InstanceData.Left / InstanceData.Right;
	}
	else
	{
		UE_LOGF(LogSceneStateTasks, Warning, "[%ls] Attempting to divide %f by zero", *InContext.GetExecutionContextName(), InstanceData.Left);
		InstanceData.Output = 0.0;
	}
}
