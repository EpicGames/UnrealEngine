// Copyright Epic Games, Inc. All Rights Reserved.

#include "Functions/SceneStateFromStringFunctions.h"

void FSceneStateTextFromStringFunction::OnExecute(const FSceneStateExecutionContext& InContext, FStructView InFunctionInstance) const
{
	FInstanceDataType& InstanceData = InFunctionInstance.Get<FInstanceDataType>();
	InstanceData.Output = FText::FromString(InstanceData.String);
}

void FSceneStateNameFromStringFunction::OnExecute(const FSceneStateExecutionContext& InContext, FStructView InFunctionInstance) const
{
	FInstanceDataType& InstanceData = InFunctionInstance.Get<FInstanceDataType>();
	InstanceData.Output = FName(InstanceData.String);
}
