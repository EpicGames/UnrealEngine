// Copyright Epic Games, Inc. All Rights Reserved.

#include "Functions/SceneStateIntegerFunctions.h"
#include "SceneStateExecutionContext.h"
#include "SceneStateTasksLog.h"

#if WITH_EDITOR
const UScriptStruct* FSceneStateIntegerFunction::OnGetFunctionDataType() const
{
	return FInstanceDataType::StaticStruct();
}
#endif

void FSceneStateAddIntegerFunction::OnExecute(const FSceneStateExecutionContext& InContext, FStructView InFunctionInstance) const
{
	FInstanceDataType& InstanceData = InFunctionInstance.Get<FInstanceDataType>();
	InstanceData.Output = InstanceData.Left + InstanceData.Right;
}

void FSceneStateSubtractIntegerFunction::OnExecute(const FSceneStateExecutionContext& InContext, FStructView InFunctionInstance) const
{
	FInstanceDataType& InstanceData = InFunctionInstance.Get<FInstanceDataType>();
	InstanceData.Output = InstanceData.Left - InstanceData.Right;
}

void FSceneStateMultiplyIntegerFunction::OnExecute(const FSceneStateExecutionContext& InContext, FStructView InFunctionInstance) const
{
	FInstanceDataType& InstanceData = InFunctionInstance.Get<FInstanceDataType>();
	InstanceData.Output = InstanceData.Left * InstanceData.Right;
}

void FSceneStateDivideIntegerFunction::OnExecute(const FSceneStateExecutionContext& InContext, FStructView InFunctionInstance) const
{
	FInstanceDataType& InstanceData = InFunctionInstance.Get<FInstanceDataType>();
	if (InstanceData.Right != 0)
	{
		InstanceData.Output = InstanceData.Left / InstanceData.Right;
	}
	else
	{
		UE_LOGF(LogSceneStateTasks, Warning, "[%ls] Attempting to divide %d by zero", *InContext.GetExecutionContextName(), InstanceData.Left);
		InstanceData.Output = 0;
	}
}
