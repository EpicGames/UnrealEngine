// Copyright Epic Games, Inc. All Rights Reserved.

#include "Functions/SceneStateStringFunctions.h"

#if WITH_EDITOR
const UScriptStruct* FSceneStateConcatenateStringFunction::OnGetFunctionDataType() const
{
	return FInstanceDataType::StaticStruct();
}
#endif

void FSceneStateConcatenateStringFunction::OnExecute(const FSceneStateExecutionContext& InContext, FStructView InFunctionInstance) const
{
	FInstanceDataType& InstanceData = InFunctionInstance.Get<FInstanceDataType>();
	InstanceData.Output = InstanceData.Left + InstanceData.Right;
}
