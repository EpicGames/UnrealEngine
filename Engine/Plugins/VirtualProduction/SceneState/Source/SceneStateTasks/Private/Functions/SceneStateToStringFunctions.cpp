// Copyright Epic Games, Inc. All Rights Reserved.

#include "Functions/SceneStateToStringFunctions.h"

void FSceneStateTextToStringFunction::OnExecute(const FSceneStateExecutionContext& InContext, FStructView InFunctionInstance) const
{
	FInstanceDataType& InstanceData = InFunctionInstance.Get<FInstanceDataType>();
	InstanceData.Output = InstanceData.Value.ToString();
}

void FSceneStateNameToStringFunction::OnExecute(const FSceneStateExecutionContext& InContext, FStructView InFunctionInstance) const
{
	FInstanceDataType& InstanceData = InFunctionInstance.Get<FInstanceDataType>();
	InstanceData.Output = InstanceData.Value.ToString();
}

void FSceneStateIntegerToStringFunction::OnExecute(const FSceneStateExecutionContext& InContext, FStructView InFunctionInstance) const
{
	FInstanceDataType& InstanceData = InFunctionInstance.Get<FInstanceDataType>();
	InstanceData.Output = FString::FromInt(InstanceData.Value);
}

void FSceneStateDoubleToStringFunction::OnExecute(const FSceneStateExecutionContext& InContext, FStructView InFunctionInstance) const
{
	FInstanceDataType& InstanceData = InFunctionInstance.Get<FInstanceDataType>();
	InstanceData.Output = FString::SanitizeFloat(InstanceData.Value);
}
