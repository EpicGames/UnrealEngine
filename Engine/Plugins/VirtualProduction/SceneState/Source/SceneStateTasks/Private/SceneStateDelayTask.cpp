// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateDelayTask.h"
#include "Tasks/SceneStateTaskExecutionContext.h"

FSceneStateDelayTask::FSceneStateDelayTask()
{
	SetFlags(ESceneStateTaskFlags::Ticks);
}

#if WITH_EDITOR
const UScriptStruct* FSceneStateDelayTask::OnGetTaskInstanceType() const
{
	return FInstanceDataType::StaticStruct();
}
#endif

void FSceneStateDelayTask::OnStart(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance) const
{
	FInstanceDataType& Instance = InTaskInstance.Get<FInstanceDataType>();
	Instance.ElapsedTime = 0.f;

	if (Instance.Delay < UE_SMALL_NUMBER)
	{
		Finish(InContext, InTaskInstance);
	}
}

void FSceneStateDelayTask::OnTick(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance, float InDeltaSeconds) const
{
	FInstanceDataType& Instance = InTaskInstance.Get<FInstanceDataType>();
	Instance.ElapsedTime += InDeltaSeconds;

	if (Instance.ElapsedTime >= Instance.Delay)
	{
		Finish(InContext, InTaskInstance);
	}
}
