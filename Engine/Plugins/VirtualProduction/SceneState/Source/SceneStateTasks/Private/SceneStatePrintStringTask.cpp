// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStatePrintStringTask.h"
#include "Kismet/KismetSystemLibrary.h"
#include "SceneStateExecutionContext.h"

#if WITH_EDITOR
const UScriptStruct* FSceneStatePrintStringTask::OnGetTaskInstanceType() const
{
	return FInstanceDataType::StaticStruct();
}
#endif

void FSceneStatePrintStringTask::OnStart(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance) const
{
#if !NO_LOGGING
	FInstanceDataType& Instance = InTaskInstance.Get<FInstanceDataType>();

	UKismetSystemLibrary::PrintString(InContext.GetContextObject()
		, Instance.Message
		, Instance.PrintSettings.bPrintToScreen
		, Instance.PrintSettings.bPrintToLog
		, Instance.PrintSettings.TextColor
		, Instance.PrintSettings.Duration
		, Instance.PrintSettings.Key);
#endif

	Finish(InContext, InTaskInstance);
}
