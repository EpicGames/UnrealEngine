// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateSpawnActorTask.h"
#include "Engine/Engine.h"
#include "GameFramework/Actor.h"
#include "SceneStateExecutionContext.h"
#include "SceneStateTasksLog.h"
#include "Setters/SceneStateSetterUtils.h"

#if WITH_EDITOR
#include "ActorFactories/ActorFactory.h"
#include "AssetSelection.h"
#include "Tasks/SceneStateTaskEditChange.h"
#endif

#if WITH_EDITOR
const UScriptStruct* FSceneStateSpawnActorTask::OnGetTaskInstanceType() const
{
	return FInstanceDataType::StaticStruct();
}

void FSceneStateSpawnActorTask::OnPostTaskEditChange(UE::SceneState::FTaskEditChange& InEditChange, FStructView InTaskInstance)
{
	using namespace UE::SceneState;

	if (InEditChange.IsTaskChange() && InEditChange.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(FSceneStateSpawnActorTask, ActorClass))
	{
		UpdateActorTemplate(InEditChange.Outer, InTaskInstance);
	}
}
#endif

void FSceneStateSpawnActorTask::OnStart(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance) const
{
	FInstanceDataType& Instance = InTaskInstance.Get<FInstanceDataType>();

	AActor* SpawnedActor = nullptr;

	ON_SCOPE_EXIT
	{
		SetSpawnedActor(InContext, SpawnedActor, Instance.SpawnedActor);
		Finish(InContext, InTaskInstance);
	};

	AActor* const TemplateActor = Instance.ActorTemplate.Template;
	if (!TemplateActor)
	{
		UE_LOGF(LogSceneStateTasks, Error, "[%ls] Template Actor was not spawned. Template Actor is null.", *InContext.GetExecutionContextName());
		return;
	}

	FText ErrorMessage;
	if (!ShouldSpawnActor(InTaskInstance, ErrorMessage))
	{
		UE_LOGF(LogSceneStateTasks, Error, "[%ls] Template Actor was not spawned. Reason: %ls"
			, *InContext.GetExecutionContextName()
			, *ErrorMessage.ToString());
		return;
	}

	if (!GEngine)
	{
		UE_LOGF(LogSceneStateTasks, Error, "[%ls] Template Actor was not spawned. GEngine unexpectedly null.", *InContext.GetExecutionContextName());
		return;
	}

	UWorld* const World = GEngine->GetWorldFromContextObject(InContext.GetContextObject(), EGetWorldErrorMode::LogAndReturnNull);
	if (!World)
	{
		UE_LOGF(LogSceneStateTasks, Error, "[%ls] Template Actor was not spawned. Could not find a valid world for context object '%ls'."
			, *InContext.GetExecutionContextName()
			, *GetNameSafe(InContext.GetContextObject()));
		return;
	}

	FActorSpawnParameters SpawnParameters;
	SpawnParameters.Template = TemplateActor;
	SpawnParameters.SpawnCollisionHandlingOverride = Instance.SpawnCollisionHandling;
	SpawnParameters.ObjectFlags = RF_Transient | RF_Transactional;

	SpawnedActor = World->SpawnActor(TemplateActor->GetClass(), &Instance.SpawnTransform, SpawnParameters);
	if (SpawnedActor)
	{
		OnActorSpawned(SpawnedActor, InTaskInstance);
	}
}

#if WITH_EDITOR
void FSceneStateSpawnActorTask::UpdateActorTemplate(UObject* InOuter, FStructView InTaskInstance) const
{
	FInstanceDataType& Instance = InTaskInstance.Get<FInstanceDataType>();

	TObjectPtr<AActor>& TemplateActor = Instance.ActorTemplate.Template;
	if (TemplateActor && TemplateActor->GetClass() == ActorClass && TemplateActor->GetOuter() == InOuter)
	{
		return;
	}

	if (!ActorClass)
	{
		TemplateActor = nullptr;
		return;
	}

	// Use an existing world to create a temporary actor instance
	UWorld* const World = GWorld;
	if (!World)
	{
		TemplateActor = nullptr;
		UE_LOGF(LogSceneStateTasks, Error, "Unable to create an actor of class %ls. No valid world found.", *ActorClass->GetName());
		return;
	}

	const FAssetData ActorClassAssetData(ActorClass);

	// FAssetData::GetAsset loads the asset synchronously if not already loaded. 
	// This editor-only function is called only in PostEditChange when the ActorClass has changed, so no need to load this async for now.
	UActorFactory* ActorFactory = FActorFactoryAssetProxy::GetFactoryForAssetObject(ActorClassAssetData.GetAsset());
	if (!ActorFactory)
	{
		TemplateActor = nullptr;
		UE_LOGF(LogSceneStateTasks, Error, "Unable to find an actor factory for class %ls.", *ActorClass->GetName());
		return;
	}

	FText ErrorText;
	if (!ActorFactory->CanCreateActorFrom(ActorClassAssetData, ErrorText))
	{
		TemplateActor = nullptr;
		UE_LOGF(LogSceneStateTasks, Error, "Unable to create actor of class %ls. Reason: %ls", *ActorClass->GetName(), *ErrorText.ToString());
		return;
	}

	FActorSpawnParameters SpawnParams;
	SpawnParams.ObjectFlags = RF_Transient | RF_Transactional;

	AActor* const SpawnedActor = ActorFactory->CreateActor(ActorClassAssetData.GetAsset(), World->PersistentLevel, FTransform::Identity, SpawnParams);
	if (!SpawnedActor)
	{
		TemplateActor = nullptr;
		UE_LOGF(LogSceneStateTasks, Error, "Unable to create actor of class %ls. Actor Factory failed to create actor.", *ActorClass->GetName());
		return;
	}

	constexpr bool bCallModify = false;
	TemplateActor = Cast<AActor>(StaticDuplicateObject(SpawnedActor, InOuter, NAME_None, RF_AllFlags & ~RF_Transient));
	TemplateActor->DetachFromActor(FDetachmentTransformRules(EDetachmentRule::KeepRelative, bCallModify));
	TemplateActor->bIsEditorPreviewActor = false;

	// Tasks and its instances containing objects are all outered to the Scene State Generated Class.
	// BP Generated Classes are, by design, renamed so that they're outered to the package of the object owning the blueprint.
	// See: UBlueprint::RenameGeneratedClasses.
	// This means that actors outered to scene state generated classes will not have a valid ULevel
	// as AActor::GetLevel() returns the first level object in the outer chain, which in scene state blueprints won't exist.
	//
	// UEditorEngine::Map_Load deletes all actors that are in the same package as a world but mismatch their Actor->GetWorld() with the given world.
	// In the case of embedded scene state blueprints, the template actors would live on the same package, but not have a level in its outer chain.
	// AActor::GetWorld relies on AActor::GetLevel to return a valid level. Because of this, template actors in embedded scene state bps would get deleted,
	// and the only other condition that stops Map_Load from deleting these actors is if the actor is an archetype object.
	TemplateActor->SetFlags(RF_ArchetypeObject);

	constexpr bool bNetForce = false;
	World->DestroyActor(SpawnedActor, bNetForce, bCallModify);
}
#endif

void FSceneStateSpawnActorTask::SetSpawnedActor(const FSceneStateExecutionContext& InContext, AActor* InSpawnedActor, const FSceneStatePropertyReference& InSpawnedActorReference) const
{
	// Spawned actor reference is optional so it's ok if it is unset.
	if (!InSpawnedActorReference.IsValidIndex())
	{
		return;
	}

	UE::SceneState::FResolvePropertyResult Result;
	if (ResolveProperty(InContext, InSpawnedActorReference, Result))
	{
		if (!UE::SceneState::SetValue<AActor*>(Result.ValuePtr, *Result.ResolvedReference, InSpawnedActor))
		{
			UE_LOGF(LogSceneStateTasks, Warning, "[%ls] Spawned Actor Reference was not set for spawned actor '%ls' (Class: %ls). Failed to set value."
				, *InContext.GetExecutionContextName()
				, *InSpawnedActor->GetActorNameOrLabel()
				, *GetNameSafe(InSpawnedActor->GetClass()));
		}
	}
	else
	{
		UE_LOGF(LogSceneStateTasks, Warning, "[%ls] Spawned Actor Reference was not set for spawned actor '%ls' (Class: %ls). Failed to resolve property."
			, *InContext.GetExecutionContextName()
			, *InSpawnedActor->GetActorNameOrLabel()
			, *GetNameSafe(InSpawnedActor->GetClass()));
	}
}
