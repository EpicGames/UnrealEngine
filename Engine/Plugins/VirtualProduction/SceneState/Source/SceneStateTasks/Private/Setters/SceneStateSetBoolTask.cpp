// Copyright Epic Games, Inc. All Rights Reserved.

#include "Setters/SceneStateSetBoolTask.h"
#include "Setters/SceneStateSetterUtils.h"

#if WITH_EDITOR
const UScriptStruct* FSceneStateSetBoolTask::OnGetTaskInstanceType() const
{
	return FInstanceDataType::StaticStruct();
}
#endif

void FSceneStateSetBoolTask::OnStart(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance) const
{
	UE::SceneState::SetValue<bool>(*this, InContext, InTaskInstance);
	Finish(InContext, InTaskInstance);
}
