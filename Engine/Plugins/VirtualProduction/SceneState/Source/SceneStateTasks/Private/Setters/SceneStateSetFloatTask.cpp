// Copyright Epic Games, Inc. All Rights Reserved.

#include "Setters/SceneStateSetFloatTask.h"
#include "Setters/SceneStateSetterUtils.h"

#if WITH_EDITOR
const UScriptStruct* FSceneStateSetFloatTask::OnGetTaskInstanceType() const
{
	return FInstanceDataType::StaticStruct();
}
#endif

void FSceneStateSetFloatTask::OnStart(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance) const
{
	UE::SceneState::SetValue<double, float>(*this, InContext, InTaskInstance);
	Finish(InContext, InTaskInstance);
}
