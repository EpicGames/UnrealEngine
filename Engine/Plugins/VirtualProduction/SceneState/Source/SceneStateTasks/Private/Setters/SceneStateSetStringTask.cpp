// Copyright Epic Games, Inc. All Rights Reserved.

#include "Setters/SceneStateSetStringTask.h"
#include "Setters/SceneStateSetterUtils.h"

#if WITH_EDITOR
const UScriptStruct* FSceneStateSetStringTask::OnGetTaskInstanceType() const
{
	return FInstanceDataType::StaticStruct();
}
#endif

void FSceneStateSetStringTask::OnStart(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance) const
{
	UE::SceneState::SetValue<FString>(*this, InContext, InTaskInstance);
	Finish(InContext, InTaskInstance);
}
