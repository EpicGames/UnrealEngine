// Copyright Epic Games, Inc. All Rights Reserved.

#include "Setters/SceneStateSetTextTask.h"
#include "Setters/SceneStateSetterUtils.h"

#if WITH_EDITOR
const UScriptStruct* FSceneStateSetTextTask::OnGetTaskInstanceType() const
{
	return FInstanceDataType::StaticStruct();
}
#endif

void FSceneStateSetTextTask::OnStart(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance) const
{
	UE::SceneState::SetValue<FText>(*this, InContext, InTaskInstance);
	Finish(InContext, InTaskInstance);
}
