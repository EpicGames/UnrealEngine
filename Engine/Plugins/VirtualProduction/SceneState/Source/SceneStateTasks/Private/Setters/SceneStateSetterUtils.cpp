// Copyright Epic Games, Inc. All Rights Reserved.

#include "Setters/SceneStateSetterUtils.h"

namespace UE::SceneState
{

bool CallValueSetter(uint8* InDestValuePtr, const FSceneStateBindingResolvedReference& InPropertyReference, const uint8* InSourceValuePtr, int32 InArrayIndex)
{
	const FProperty* const DestProperty = InPropertyReference.SourceLeafProperty;
	if (!ensureMsgf(InArrayIndex >= 0 && InArrayIndex < DestProperty->ArrayDim, TEXT("Index %d outside property array dim bounds %d!"), InArrayIndex, DestProperty->ArrayDim))
	{
		return false;
	}

	// No setter, can't call via this method
	if (!DestProperty->HasSetter())
	{
		return false;
	}

	// Reverse logic of ContainerPtrToValuePtr (i.e. ValuePtrToContainerPtr)
	uint8* const DestContainerPtr = InDestValuePtr - (DestProperty->GetOffset_ForInternal() + DestProperty->GetElementSize() * InArrayIndex);

	// check that this reverse logic remains 'up-to-date' with the latest ContainerPtrToValuePtr logic
	checkf(DestProperty->ContainerPtrToValuePtr<uint8>(DestContainerPtr) == InDestValuePtr, TEXT("ValuePtrToContainerPtr logic outdated. Verify against ContainerPtrToValuePtr."));

	DestProperty->CallSetter(DestContainerPtr, InSourceValuePtr);
	return true;
}

} // UE::SceneState
