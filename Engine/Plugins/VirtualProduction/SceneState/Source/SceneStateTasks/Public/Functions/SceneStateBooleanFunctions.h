// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Functions/SceneStateFunction.h"
#include "SceneStateBooleanFunctions.generated.h"

/** Function instance for all boolean operation functions involving two operands, left and right, and a result */
USTRUCT()
struct FSceneStateBooleanOpFunctionInstance
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Math")
	bool bLeft = false;

	UPROPERTY(EditAnywhere, Category="Math")
	bool bRight = false;

	UPROPERTY(EditAnywhere, Category="Math", meta=(Output))
	bool bOutput = false;
};

/** Base class for boolean operations where there are two operands, left and right, and a result */
USTRUCT(meta=(Hidden))
struct FSceneStateBooleanFunction : public FSceneStateFunction
{
	GENERATED_BODY()

	using FInstanceDataType = FSceneStateBooleanOpFunctionInstance;

protected:
	//~ Begin FSceneStateFunction
#if WITH_EDITOR
	virtual const UScriptStruct* OnGetFunctionDataType() const override;
#endif
	//~ End FSceneStateFunction
};

/** Performs boolean AND (Left AND Right) */
USTRUCT(DisplayName="And", Category="Math")
struct FSceneStateBooleanAndFunction : public FSceneStateBooleanFunction
{
	GENERATED_BODY()

protected:
	//~ Begin FSceneStateFunction
	virtual void OnExecute(const FSceneStateExecutionContext& InContext, FStructView InFunctionInstance) const override;
	//~ End FSceneStateFunction
};

/** Performs boolean OR (Left AND Right) */
USTRUCT(DisplayName="Or", Category="Math")
struct FSceneStateBooleanOrFunction : public FSceneStateBooleanFunction
{
	GENERATED_BODY()

protected:
	//~ Begin FSceneStateFunction
	virtual void OnExecute(const FSceneStateExecutionContext& InContext, FStructView InFunctionInstance) const override;
	//~ End FSceneStateFunction
};

/** Performs boolean XOR (Left XOR Right) */
USTRUCT(DisplayName="XOr", Category="Math")
struct FSceneStateBooleanXorFunction : public FSceneStateBooleanFunction
{
	GENERATED_BODY()

protected:
	//~ Begin FSceneStateFunction
	virtual void OnExecute(const FSceneStateExecutionContext& InContext, FStructView InFunctionInstance) const override;
	//~ End FSceneStateFunction
};

/** Function instance the boolean NOT operation where only one input is required */
USTRUCT()
struct FSceneStateBooleanNotFunctionInstance
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Math")
	bool bInput = false;

	UPROPERTY(EditAnywhere, Category="Math", meta=(Output))
	bool bOutput = false;
};

/** Performs boolean NOT */
USTRUCT(DisplayName="Not", Category="Math")
struct FSceneStateBooleanNotFunction : public FSceneStateBooleanFunction
{
	GENERATED_BODY()

	using FInstanceDataType = FSceneStateBooleanNotFunctionInstance;

protected:
	//~ Begin FSceneStateFunction
#if WITH_EDITOR
	virtual const UScriptStruct* OnGetFunctionDataType() const override;
#endif
	virtual void OnExecute(const FSceneStateExecutionContext& InContext, FStructView InFunctionInstance) const override;
	//~ End FSceneStateFunction
};
