// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Functions/SceneStateFunction.h"
#include "SceneStateFloatFunctions.generated.h"

/** Function instance for all double-precision float operation functions involving two operands, left and right, and a result */
USTRUCT()
struct FSceneStateDoubleOpFunctionInstance
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Math")
	double Left = 0.0;

	UPROPERTY(EditAnywhere, Category="Math")
	double Right = 0.0;

	UPROPERTY(EditAnywhere, Category="Math", meta=(Output))
	double Output = 0.0;
};

/** Base class for double-precision float operations where there are two operands, left and right, and a result */
USTRUCT(meta=(Hidden))
struct FSceneStateDoubleFunction : public FSceneStateFunction
{
	GENERATED_BODY()

	using FInstanceDataType = FSceneStateDoubleOpFunctionInstance;

protected:
	//~ Begin FSceneStateFunction
#if WITH_EDITOR
	virtual const UScriptStruct* OnGetFunctionDataType() const override;
#endif
	//~ End FSceneStateFunction
};

/** Performs addition (Left + Right) */
USTRUCT(DisplayName="Add", Category="Math")
struct FSceneStateAddDoubleFunction : public FSceneStateDoubleFunction
{
	GENERATED_BODY()

protected:
	//~ Begin FSceneStateFunction
	virtual void OnExecute(const FSceneStateExecutionContext& InContext, FStructView InFunctionInstance) const override;
	//~ End FSceneStateFunction
};

/** Performs subtraction (Left - Right) */
USTRUCT(DisplayName="Subtract", Category="Math")
struct FSceneStateSubtractDoubleFunction : public FSceneStateDoubleFunction
{
	GENERATED_BODY()

protected:
	//~ Begin FSceneStateFunction
	virtual void OnExecute(const FSceneStateExecutionContext& InContext, FStructView InFunctionInstance) const override;
	//~ End FSceneStateFunction
};

/** Performs multiplication (Left * Right) */
USTRUCT(DisplayName="Multiply", Category="Math")
struct FSceneStateMultiplyDoubleFunction : public FSceneStateDoubleFunction
{
	GENERATED_BODY()

protected:
	//~ Begin FSceneStateFunction
	virtual void OnExecute(const FSceneStateExecutionContext& InContext, FStructView InFunctionInstance) const override;
	//~ End FSceneStateFunction
};

/** Performs division (Left / Right) */
USTRUCT(DisplayName="Divide", Category="Math")
struct FSceneStateDivideDoubleFunction : public FSceneStateDoubleFunction
{
	GENERATED_BODY()

protected:
	//~ Begin FSceneStateFunction
	virtual void OnExecute(const FSceneStateExecutionContext& InContext, FStructView InFunctionInstance) const override;
	//~ End FSceneStateFunction
};
