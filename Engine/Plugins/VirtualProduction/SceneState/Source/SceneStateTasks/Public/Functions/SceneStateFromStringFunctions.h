// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Functions/SceneStateFunction.h"
#include "SceneStateFromStringFunctions.generated.h"

/** Base function instance for all "FromString" functions */
USTRUCT()
struct FSceneStateFromStringFunctionInstance
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="String")
	FString String;
};

//////////////////////////////////////////////////////////////////////////////////////////////
// Text from String
//////////////////////////////////////////////////////////////////////////////////////////////
USTRUCT()
struct FSceneStateTextFromStringFunctionInstance : public FSceneStateFromStringFunctionInstance
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="String", meta=(Output))
	FText Output;
};

USTRUCT(DisplayName="String to Text", Category="String")
struct FSceneStateTextFromStringFunction : public FSceneStateFunction
{
	GENERATED_BODY()

	using FInstanceDataType = FSceneStateTextFromStringFunctionInstance;

protected:
	//~ Begin FSceneStateFunction
#if WITH_EDITOR
	virtual const UScriptStruct* OnGetFunctionDataType() const override
	{
		return FInstanceDataType::StaticStruct();
	}
#endif
	virtual void OnExecute(const FSceneStateExecutionContext& InContext, FStructView InFunctionInstance) const override;
	//~ End FSceneStateFunction
};

//////////////////////////////////////////////////////////////////////////////////////////////
// Name from String
//////////////////////////////////////////////////////////////////////////////////////////////
USTRUCT()
struct FSceneStateNameFromStringFunctionInstance : public FSceneStateFromStringFunctionInstance
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="String", meta=(Output))
	FName Output;
};

USTRUCT(DisplayName="String to Name", Category="String")
struct FSceneStateNameFromStringFunction : public FSceneStateFunction
{
	GENERATED_BODY()

	using FInstanceDataType = FSceneStateNameFromStringFunctionInstance;

protected:
	//~ Begin FSceneStateFunction
#if WITH_EDITOR
	virtual const UScriptStruct* OnGetFunctionDataType() const override
	{
		return FInstanceDataType::StaticStruct();
	}
#endif
	virtual void OnExecute(const FSceneStateExecutionContext& InContext, FStructView InFunctionInstance) const override;
	//~ End FSceneStateFunction
};
