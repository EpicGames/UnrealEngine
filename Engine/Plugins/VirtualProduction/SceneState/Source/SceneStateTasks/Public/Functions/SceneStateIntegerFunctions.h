// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Functions/SceneStateFunction.h"
#include "SceneStateIntegerFunctions.generated.h"

/** Function instance for all integer operation functions involving two operands, left and right, and an integer result */
USTRUCT()
struct FSceneStateIntegerOpFunctionInstance
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Math")
	int32 Left = 0;

	UPROPERTY(EditAnywhere, Category="Math")
	int32 Right = 0;

	UPROPERTY(EditAnywhere, Category="Math", meta=(Output))
	int32 Output = 0;
};

/** Base class for integer operations where there are two operands, left and right, and an integer result */
USTRUCT(meta=(Hidden))
struct FSceneStateIntegerFunction : public FSceneStateFunction
{
	GENERATED_BODY()

	using FInstanceDataType = FSceneStateIntegerOpFunctionInstance;

protected:
	//~ Begin FSceneStateFunction
#if WITH_EDITOR
	virtual const UScriptStruct* OnGetFunctionDataType() const override;
#endif
	//~ End FSceneStateFunction
};

/** Performs addition (Left + Right) */
USTRUCT(DisplayName="Add", Category="Math")
struct FSceneStateAddIntegerFunction : public FSceneStateIntegerFunction
{
	GENERATED_BODY()

protected:
	//~ Begin FSceneStateFunction
	virtual void OnExecute(const FSceneStateExecutionContext& InContext, FStructView InFunctionInstance) const override;
	//~ End FSceneStateFunction
};

/** Performs subtraction (Left - Right) */
USTRUCT(DisplayName="Subtract", Category="Math")
struct FSceneStateSubtractIntegerFunction : public FSceneStateIntegerFunction
{
	GENERATED_BODY()

protected:
	//~ Begin FSceneStateFunction
	virtual void OnExecute(const FSceneStateExecutionContext& InContext, FStructView InFunctionInstance) const override;
	//~ End FSceneStateFunction
};

/** Performs multiplication (Left * Right) */
USTRUCT(DisplayName="Multiply", Category="Math")
struct FSceneStateMultiplyIntegerFunction : public FSceneStateIntegerFunction
{
	GENERATED_BODY()

protected:
	//~ Begin FSceneStateFunction
	virtual void OnExecute(const FSceneStateExecutionContext& InContext, FStructView InFunctionInstance) const override;
	//~ End FSceneStateFunction
};

/** Performs division (Left / Right) */
USTRUCT(DisplayName="Divide", Category="Math")
struct FSceneStateDivideIntegerFunction : public FSceneStateIntegerFunction
{
	GENERATED_BODY()

protected:
	//~ Begin FSceneStateFunction
	virtual void OnExecute(const FSceneStateExecutionContext& InContext, FStructView InFunctionInstance) const override;
	//~ End FSceneStateFunction
};
