// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Functions/SceneStateFunction.h"
#include "SceneStateStringFunctions.generated.h"

USTRUCT()
struct FSceneStateConcatenateStringFunctionInstance
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="String")
	FString Left;

	UPROPERTY(EditAnywhere, Category="String")
	FString Right;

	UPROPERTY(EditAnywhere, Category="String", meta=(Output))
	FString Output;
};

/** Concatenates two strings (Left + Right) together */
USTRUCT(DisplayName="Concatenate String", Category="String")
struct FSceneStateConcatenateStringFunction : public FSceneStateFunction
{
	GENERATED_BODY()

	using FInstanceDataType = FSceneStateConcatenateStringFunctionInstance;

protected:
	//~ Begin FSceneStateFunction
#if WITH_EDITOR
	virtual const UScriptStruct* OnGetFunctionDataType() const override;
#endif
	virtual void OnExecute(const FSceneStateExecutionContext& InContext, FStructView InFunctionInstance) const override;
	//~ End FSceneStateFunction
};
