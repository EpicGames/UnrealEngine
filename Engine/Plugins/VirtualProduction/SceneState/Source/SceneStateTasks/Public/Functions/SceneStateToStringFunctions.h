// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Functions/SceneStateFunction.h"
#include "SceneStateToStringFunctions.generated.h"

/** Base function instance for all "ToString" functions */
USTRUCT()
struct FSceneStateToStringFunctionInstance
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="String", meta=(Output))
	FString Output;
};

//////////////////////////////////////////////////////////////////////////////////////////////
// Text to String
//////////////////////////////////////////////////////////////////////////////////////////////
USTRUCT()
struct FSceneStateTextToStringFunctionInstance : public FSceneStateToStringFunctionInstance
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="String")
	FText Value;
};

USTRUCT(DisplayName="Text to String", Category="String")
struct FSceneStateTextToStringFunction : public FSceneStateFunction
{
	GENERATED_BODY()

	using FInstanceDataType = FSceneStateTextToStringFunctionInstance;

protected:
	//~ Begin FSceneStateFunction
#if WITH_EDITOR
	virtual const UScriptStruct* OnGetFunctionDataType() const override
	{
		return FInstanceDataType::StaticStruct();
	}
#endif
	virtual void OnExecute(const FSceneStateExecutionContext& InContext, FStructView InFunctionInstance) const override;
	//~ End FSceneStateFunction
};

//////////////////////////////////////////////////////////////////////////////////////////////
// Name to String
//////////////////////////////////////////////////////////////////////////////////////////////
USTRUCT()
struct FSceneStateNameToStringFunctionInstance : public FSceneStateToStringFunctionInstance
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="String")
	FName Value;
};

USTRUCT(DisplayName="Name to String", Category="String")
struct FSceneStateNameToStringFunction : public FSceneStateFunction
{
	GENERATED_BODY()

	using FInstanceDataType = FSceneStateNameToStringFunctionInstance;

protected:
	//~ Begin FSceneStateFunction
#if WITH_EDITOR
	virtual const UScriptStruct* OnGetFunctionDataType() const override
	{
		return FInstanceDataType::StaticStruct();
	}
#endif
	virtual void OnExecute(const FSceneStateExecutionContext& InContext, FStructView InFunctionInstance) const override;
	//~ End FSceneStateFunction
};

//////////////////////////////////////////////////////////////////////////////////////////////
// Integer to String
//////////////////////////////////////////////////////////////////////////////////////////////
USTRUCT()
struct FSceneStateIntegerToStringFunctionInstance : public FSceneStateToStringFunctionInstance
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="String")
	int32 Value = 0;
};

USTRUCT(DisplayName="Integer to String", Category="String")
struct FSceneStateIntegerToStringFunction : public FSceneStateFunction
{
	GENERATED_BODY()

	using FInstanceDataType = FSceneStateIntegerToStringFunctionInstance;

protected:
	//~ Begin FSceneStateFunction
#if WITH_EDITOR
	virtual const UScriptStruct* OnGetFunctionDataType() const override
	{
		return FInstanceDataType::StaticStruct();
	}
#endif
	virtual void OnExecute(const FSceneStateExecutionContext& InContext, FStructView InFunctionInstance) const override;
	//~ End FSceneStateFunction
};

//////////////////////////////////////////////////////////////////////////////////////////////
// Double-precision Float to String
//////////////////////////////////////////////////////////////////////////////////////////////
USTRUCT()
struct FSceneStateDoubleToStringFunctionInstance : public FSceneStateToStringFunctionInstance
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="String")
	double Value = 0.0;
};

USTRUCT(DisplayName="Float to String", Category="String")
struct FSceneStateDoubleToStringFunction : public FSceneStateFunction
{
	GENERATED_BODY()

	using FInstanceDataType = FSceneStateDoubleToStringFunctionInstance;

protected:
	//~ Begin FSceneStateFunction
#if WITH_EDITOR
	virtual const UScriptStruct* OnGetFunctionDataType() const override
	{
		return FInstanceDataType::StaticStruct();
	}
#endif
	virtual void OnExecute(const FSceneStateExecutionContext& InContext, FStructView InFunctionInstance) const override;
	//~ End FSceneStateFunction
};
