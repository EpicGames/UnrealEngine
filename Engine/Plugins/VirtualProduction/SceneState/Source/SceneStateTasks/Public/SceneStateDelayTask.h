// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tasks/SceneStateTask.h"
#include "Tasks/SceneStateTaskInstance.h"
#include "SceneStateDelayTask.generated.h"

USTRUCT()
struct FSceneStateDelayTaskInstance : public FSceneStateTaskInstance
{
	GENERATED_BODY()

	/** The amount of time elapsed */
	float ElapsedTime = 0.f;

	/** The amount to wait for, in seconds */
	UPROPERTY(EditAnywhere, Category="Delay", meta=(ClampMin="0"))
	float Delay = 0.5f;
};

/** Waits for a set amount of seconds */
USTRUCT(DisplayName="Delay", Category="Utility", meta=(UtilityTask))
struct FSceneStateDelayTask : public FSceneStateTask
{
	GENERATED_BODY()

	using FInstanceDataType = FSceneStateDelayTaskInstance;

	FSceneStateDelayTask();

protected:
	//~ Begin FSceneStateTask
#if WITH_EDITOR
	virtual const UScriptStruct* OnGetTaskInstanceType() const override;
#endif
	virtual void OnStart(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance) const override;
	virtual void OnTick(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance, float InDeltaSeconds) const override;
	//~ End FSceneStateTask
};
