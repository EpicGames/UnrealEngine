// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tasks/SceneStateTask.h"
#include "Tasks/SceneStateTaskInstance.h"
#include "SceneStatePrintStringTask.generated.h"

USTRUCT()
struct FSceneStatePrintSettings
{
	GENERATED_BODY()

	/** Whether to print the message to the console */
	UPROPERTY(EditAnywhere, Category="Print Settings")
	bool bPrintToLog = true;

	/** Whether to print the message to the screen */
	UPROPERTY(EditAnywhere, Category="Print Settings")
	bool bPrintToScreen = true;

	/** The color of the text to display */
	UPROPERTY(EditAnywhere, Category="Print Settings", meta=(EditCondition="bPrintToScreen", EditConditionHides))
	FLinearColor TextColor = FLinearColor(0.0f, 0.66f, 1.0f);

	/** the display duration. Using negative number will result in loading the duration time from the config */
	UPROPERTY(EditAnywhere, Category="Print Settings", meta=(EditCondition="bPrintToScreen", EditConditionHides))
	float Duration = 2.f;

	/** If a non-empty key is provided, the message will replace any existing on-screen messages with the same key */
	UPROPERTY(EditAnywhere, Category="Print Settings", meta=(EditCondition="bPrintToScreen", EditConditionHides))
	FName Key = NAME_None;
};

USTRUCT()
struct FSceneStatePrintStringTaskInstance : public FSceneStateTaskInstance
{
	GENERATED_BODY()

	/** The message to print to screen/log */
	UPROPERTY(EditAnywhere, Category="Print Task")
	FString Message;

	/** Additional settings for how to print the string */
	UPROPERTY(EditAnywhere, Category="Print Task")
	FSceneStatePrintSettings PrintSettings;
};

USTRUCT(DisplayName="Print String", Category="Utility", meta=(UtilityTask))
struct FSceneStatePrintStringTask : public FSceneStateTask
{
	GENERATED_BODY()

	using FInstanceDataType = FSceneStatePrintStringTaskInstance;

protected:
	//~ Begin FSceneStateTask
#if WITH_EDITOR
	virtual const UScriptStruct* OnGetTaskInstanceType() const override;
#endif
	virtual void OnStart(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance) const override;
	//~ End FSceneStateTask
};
