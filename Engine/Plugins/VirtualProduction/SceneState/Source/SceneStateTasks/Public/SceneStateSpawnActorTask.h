// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineTypes.h"
#include "SceneStatePropertyReference.h"
#include "Tasks/SceneStateTask.h"
#include "Tasks/SceneStateTaskInstance.h"
#include "Templates/SubclassOf.h"
#include "SceneStateSpawnActorTask.generated.h"

#define UE_API SCENESTATETASKS_API

struct FActorSpawnParameters;

/** Represents the actor template to use */
USTRUCT()
struct FSceneStateActorTemplate
{
	GENERATED_BODY()

	/** The actor template to use */
	UPROPERTY(VisibleAnywhere, Instanced, Category="Scene State")
	TObjectPtr<AActor> Template;
};

USTRUCT()
struct FSceneStateSpawnActorTaskInstance : public FSceneStateTaskInstance
{
	GENERATED_BODY()

	/** The actor template to use */
	UPROPERTY(EditAnywhere, Category="Scene State", meta=(NoBindingSelfOnly))
	FSceneStateActorTemplate ActorTemplate;

	/** The spawn transform for the actor */
	UPROPERTY(EditAnywhere, Category="Scene State")
	FTransform SpawnTransform = FTransform::Identity;

	/** Method for resolving collisions at the spawn point. Default (unspecified) means no override, use the actor's setting */
	UPROPERTY(EditAnywhere, Category="Scene State")
	ESpawnActorCollisionHandlingMethod SpawnCollisionHandling = ESpawnActorCollisionHandlingMethod::Undefined;

	/** Optional: Sets the actor that was spawned from this task */
	UPROPERTY(EditAnywhere, Category="Scene State", meta=(RefType="/Script/Engine.Actor"))
	FSceneStatePropertyReference SpawnedActor;
};

/** Spawns an actor of a given class */
USTRUCT(DisplayName="Spawn Actor", Category="Utility", meta=(UtilityTask, RequiresContextWorld))
struct FSceneStateSpawnActorTask : public FSceneStateTask
{
	GENERATED_BODY()

	using FInstanceDataType = FSceneStateSpawnActorTaskInstance;

protected:
	//~ Begin FSceneStateTask
#if WITH_EDITOR
	UE_API virtual const UScriptStruct* OnGetTaskInstanceType() const override;
	UE_API virtual void OnPostTaskEditChange(UE::SceneState::FTaskEditChange& InEditChange, FStructView InTaskInstance) override;
#endif
	UE_API virtual void OnStart(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance) const override;
	//~ End FSceneStateTask

#if WITH_EDITOR
	/** Called to update the actor template from the task instance to match the current actor class */
	void UpdateActorTemplate(UObject* InOuter, FStructView InTaskInstance) const;
#endif

	/**
	 * Updates the spawned actor reference value (if set) to the newly spawned actor
	 * @param InContext the execution context that is running this task
	 * @param InSpawnedActor the value to set the property reference to
	 * @param InSpawnedActorReference the actor property reference to set
	 */
	void SetSpawnedActor(const FSceneStateExecutionContext& InContext, AActor* InSpawnedActor, const FSceneStatePropertyReference& InSpawnedActorReference) const;

	/**
	 * Determines whether the actor should be spawned
	 * @param InTaskInstance data view to the task instance of the execution.
	 * @param OutErrorMessage the error message indicating why an actor should not be spawned.
	 * @return True if the actor should be spawned, false otherwise ideally accompanied by an error message. 
	 */
	virtual bool ShouldSpawnActor(FStructView InTaskInstance, FText& OutErrorMessage) const
	{
		return true;
	}

	/**
	 * Called after the actor has spawned for additional handling
	 * @param InActorChecked the spawned actor, already checked to be valid
	 * @param InTaskInstance data view to the task instance of the execution.
	 */
	virtual void OnActorSpawned(AActor* InActorChecked, FStructView InTaskInstance) const
	{
	}

	/** Actor class to spawn */
	UPROPERTY(EditAnywhere, Category="Scene State")
	TSubclassOf<AActor> ActorClass;
};

#undef UE_API
