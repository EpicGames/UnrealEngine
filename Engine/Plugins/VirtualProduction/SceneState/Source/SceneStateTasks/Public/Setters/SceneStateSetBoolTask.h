// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneStatePropertyReference.h"
#include "Tasks/SceneStateTask.h"
#include "Tasks/SceneStateTaskInstance.h"
#include "SceneStateSetBoolTask.generated.h"

USTRUCT()
struct FSceneStateSetBoolTaskInstance : public FSceneStateTaskInstance
{
	GENERATED_BODY()

	/** Target bool property to set */
	UPROPERTY(EditAnywhere, Category="Setter", meta=(RefType="bool"))
	FSceneStatePropertyReference Target;

	/** The boolean value to set. */
	UPROPERTY(EditAnywhere, Category="Setter")
	bool Value = false;
};

/** Sets a boolean value to the bound property reference */
USTRUCT(DisplayName="Set Boolean", Category="Setter", meta=(CoreTask))
struct FSceneStateSetBoolTask : public FSceneStateTask
{
	GENERATED_BODY()

	using FInstanceDataType = FSceneStateSetBoolTaskInstance;

protected:
	//~ Begin FSceneStateTask
#if WITH_EDITOR
	virtual const UScriptStruct* OnGetTaskInstanceType() const override;
#endif
	virtual void OnStart(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance) const override;
	//~ End FSceneStateTask
};
