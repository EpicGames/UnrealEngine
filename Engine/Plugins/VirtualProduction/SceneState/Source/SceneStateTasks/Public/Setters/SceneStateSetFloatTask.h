// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneStatePropertyReference.h"
#include "Tasks/SceneStateTask.h"
#include "Tasks/SceneStateTaskInstance.h"
#include "SceneStateSetFloatTask.generated.h"

USTRUCT()
struct FSceneStateSetFloatTaskInstance : public FSceneStateTaskInstance
{
	GENERATED_BODY()

	/** Target float property to set */
	UPROPERTY(EditAnywhere, Category="Setter", meta=(RefType="double,float"))
	FSceneStatePropertyReference Target;

	/** The float value to set */
	UPROPERTY(EditAnywhere, Category="Setter")
	double Value = 0.0;
};

/** Sets a float value to the bound property reference */
USTRUCT(DisplayName="Set Float", Category="Setter", meta=(CoreTask))
struct FSceneStateSetFloatTask : public FSceneStateTask
{
	GENERATED_BODY()

	using FInstanceDataType = FSceneStateSetFloatTaskInstance;

protected:
	//~ Begin FSceneStateTask
#if WITH_EDITOR
	virtual const UScriptStruct* OnGetTaskInstanceType() const override;
#endif
	virtual void OnStart(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance) const override;
	//~ End FSceneStateTask
};
