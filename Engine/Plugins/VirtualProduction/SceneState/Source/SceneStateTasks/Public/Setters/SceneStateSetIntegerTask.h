// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneStatePropertyReference.h"
#include "Tasks/SceneStateTask.h"
#include "Tasks/SceneStateTaskInstance.h"
#include "SceneStateSetIntegerTask.generated.h"

USTRUCT()
struct FSceneStateSetIntegerTaskInstance : public FSceneStateTaskInstance
{
	GENERATED_BODY()

	/** Target integer property to set */
	UPROPERTY(EditAnywhere, Category="Setter", meta=(RefType="int32"))
	FSceneStatePropertyReference Target;

	/** The integer value to set. */
	UPROPERTY(EditAnywhere, Category="Setter")
	int32 Value = 0;
};

/** Sets a integer value to the bound property reference */
USTRUCT(DisplayName="Set Integer", Category="Setter", meta=(CoreTask))
struct FSceneStateSetIntegerTask : public FSceneStateTask
{
	GENERATED_BODY()

	using FInstanceDataType = FSceneStateSetIntegerTaskInstance;

protected:
	//~ Begin FSceneStateTask
#if WITH_EDITOR
	virtual const UScriptStruct* OnGetTaskInstanceType() const override;
#endif
	virtual void OnStart(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance) const override;
	//~ End FSceneStateTask
};
