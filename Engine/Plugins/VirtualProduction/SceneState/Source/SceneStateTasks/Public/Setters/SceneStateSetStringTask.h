// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneStatePropertyReference.h"
#include "Tasks/SceneStateTask.h"
#include "Tasks/SceneStateTaskInstance.h"
#include "SceneStateSetStringTask.generated.h"

USTRUCT()
struct FSceneStateSetStringTaskInstance : public FSceneStateTaskInstance
{
	GENERATED_BODY()

	/** Target string property to set */
	UPROPERTY(EditAnywhere, Category="Setter", meta=(RefType="String"))
	FSceneStatePropertyReference Target;

	/** The string value to set. */
	UPROPERTY(EditAnywhere, Category="Setter")
	FString Value;
};

/** Sets a string value to the bound property reference */
USTRUCT(DisplayName="Set String", Category="Setter", meta=(CoreTask))
struct FSceneStateSetStringTask : public FSceneStateTask
{
	GENERATED_BODY()

	using FInstanceDataType = FSceneStateSetStringTaskInstance;

protected:
	//~ Begin FSceneStateTask
#if WITH_EDITOR
	virtual const UScriptStruct* OnGetTaskInstanceType() const override;
#endif
	virtual void OnStart(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance) const override;
	//~ End FSceneStateTask
};
