// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneStatePropertyReference.h"
#include "Tasks/SceneStateTask.h"
#include "Tasks/SceneStateTaskInstance.h"
#include "SceneStateSetTextTask.generated.h"

USTRUCT()
struct FSceneStateSetTextTaskInstance : public FSceneStateTaskInstance
{
	GENERATED_BODY()

	/** Target text property to set */
	UPROPERTY(EditAnywhere, Category="Setter", meta=(RefType="Text"))
	FSceneStatePropertyReference Target;

	/** The text value to set. */
	UPROPERTY(EditAnywhere, Category="Setter")
	FText Value;
};

/** Sets a text value to the bound property reference */
USTRUCT(DisplayName="Set Text", Category="Setter", meta=(CoreTask))
struct FSceneStateSetTextTask : public FSceneStateTask
{
	GENERATED_BODY()

	using FInstanceDataType = FSceneStateSetTextTaskInstance;

protected:
	//~ Begin FSceneStateTask
#if WITH_EDITOR
	virtual const UScriptStruct* OnGetTaskInstanceType() const override;
#endif
	virtual void OnStart(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance) const override;
	//~ End FSceneStateTask
};
