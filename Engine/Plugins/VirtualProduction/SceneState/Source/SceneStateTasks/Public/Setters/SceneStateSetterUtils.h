// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Concepts/DecaysTo.h"
#include "PropertyBindingDataView.h"
#include "SceneStateExecutionContext.h"
#include "SceneStatePropertyUtils.h"

#define UE_API SCENESTATETASKS_API

struct FSceneStatePropertyReference;

namespace UE::SceneState
{
	/**
	 * Attempts to call the Dest Property's setter with a provided source value
	 * @param InDestValuePtr the value pointer to get the container ptr from (to pass back into FProperty::CallSetter)
	 * @param InPropertyReference the property reference describing the destination value ptr
	 * @param InSourceValuePtr pointer to the source data to copy from
	 * @param InArrayIndex the array index to set within the destination property (necessary if property has an Array Dim > 1, 0 otherwise)
	 * @return true if the setter call was made
	 */
	UE_API bool CallValueSetter(uint8* InDestValuePtr, const FSceneStateBindingResolvedReference& InPropertyReference, const uint8* InSourceValuePtr, int32 InArrayIndex = 0);

	/** A Setter Task Instance is valid if it has a valid 'Value' member, as well as a 'Target' FSceneStatePropertyReference member */
	template<typename InTaskInstanceType>
	concept CSetterTaskInstanceTypeable = requires(const InTaskInstanceType& InTaskInstance)
	{
		InTaskInstance.Value;
		{ InTaskInstance.Target }->UE::CDecaysTo<FSceneStatePropertyReference>;
	};

	/** A Setter Task is valid if it has the FInstanceDataType for its Task Instance type, and this Task Instance meets the requirements to be a Setter Task Instance*/
	template<typename InTaskType>
	concept CSetterTaskTypeable = CSetterTaskInstanceTypeable<typename InTaskType::FInstanceDataType>;

	/**
	 * Copies the provided value to the given destination address, calling a Setter if available
	 * @param InDestValuePtr pointer of the destination address to set
	 * @param InPropertyReference the property reference describing the destination value ptr
	 * @param InValue the value to copy from
	 */
	template<typename InTargetType, typename InValueType>
	bool SetValue(uint8* InDestValuePtr, const FSceneStateBindingResolvedReference& InPropertyReference, const InValueType& InValue)
	{
		if (TDataTypeHelper<InTargetType>::IsValid(InPropertyReference.SourceLeafProperty))
		{
			// Attempt to call setter if available
			if (!CallValueSetter(InDestValuePtr, InPropertyReference, reinterpret_cast<const uint8*>(&InValue)))
			{
				// The setter wasn't called, set the value of the property directly
				*reinterpret_cast<InTargetType*>(InDestValuePtr) = InValue;
			}
			return true;
		}
		return false;
	}

	/**
	 * Sets the Task Instance's value to the Task Instance's property reference
	 * @param InTask the FSceneStateTask object
	 * @param InContext the current execution context
	 * @param InTaskInstance view of the task instance data
	 */
	template<typename... InTargetTypes, CSetterTaskTypeable InTaskType>
	bool SetValue(const InTaskType& InTask, const FSceneStateExecutionContext& InContext, FStructView InTaskInstance)
	{
		typename InTaskType::FInstanceDataType& Instance = InTaskInstance.Get<typename InTaskType::FInstanceDataType>();

		FResolvePropertyResult Result;
		if (ResolveProperty(InContext, Instance.Target, Result))
		{
			return (SetValue<InTargetTypes>(Result.ValuePtr, *Result.ResolvedReference, Instance.Value) || ...);
		}
		return false;
	}

} // UE::SceneState

#undef UE_API
