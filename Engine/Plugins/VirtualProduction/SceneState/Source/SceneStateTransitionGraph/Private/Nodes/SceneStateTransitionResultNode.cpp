// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/SceneStateTransitionResultNode.h"
#include "SceneStateTransitionGraphSchema.h"
#include "SceneStateTransitionOptionalPinManager.h"
#include "GraphEditorSettings.h"

#define LOCTEXT_NAMESPACE "SceneStateTransitionResultNode"

FString USceneStateTransitionResultNode::GetDescriptiveCompiledName() const
{
	return FString::Printf(TEXT("Result_%d"), GetFName().GetNumber());
}

bool USceneStateTransitionResultNode::CanCreateUnderSpecifiedSchema(const UEdGraphSchema* InSchema) const
{
	return InSchema && InSchema->IsA<USceneStateTransitionGraphSchema>();
}

void USceneStateTransitionResultNode::AllocateDefaultPins()
{
	using namespace UE::SceneState::Graph;

	Super::AllocateDefaultPins();

	FStructProperty* StructProperty = FindFProperty<FStructProperty>(GetClass(), GET_MEMBER_NAME_CHECKED(USceneStateTransitionResultNode, Result));

	TArray<FOptionalPinFromProperty> ShowPinForProperties;

	UObject* NodeDefaults = GetArchetype();

	// FOptionalPinManager by default exposes all pins
	FTransitionOptionalPinManager OptionalPinManager;
	OptionalPinManager.RebuildPropertyList(ShowPinForProperties, StructProperty->Struct);
	OptionalPinManager.CreateVisiblePins(ShowPinForProperties
		, StructProperty->Struct
		, EGPD_Input
		, this
		, StructProperty->ContainerPtrToValuePtr<uint8>(this)
		, NodeDefaults ? StructProperty->ContainerPtrToValuePtr<uint8>(NodeDefaults) : nullptr);
}

bool USceneStateTransitionResultNode::CanUserDeleteNode() const
{
	return false;
}

bool USceneStateTransitionResultNode::CanDuplicateNode() const
{
	return false;
}

FLinearColor USceneStateTransitionResultNode::GetNodeTitleColor() const
{
	return GetDefault<UGraphEditorSettings>()->ResultNodeTitleColor;
}

FText USceneStateTransitionResultNode::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Title", "Result");
}

FText USceneStateTransitionResultNode::GetTooltipText() const
{
	return LOCTEXT("Tooltip", "This expression is evaluated to determine if the state transition can be taken");
}

bool USceneStateTransitionResultNode::IsNodeRootSet() const
{
	return true;
}

#undef LOCTEXT_NAMESPACE
