// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateTransitionGraphSchema.h"
#include "ISceneStateTransitionGraphProvider.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Nodes/SceneStateTransitionResultNode.h"
#include "SceneStateTransitionGraph.h"

#define LOCTEXT_NAMESPACE "SceneStateTransitionGraphSchema"

EGraphType USceneStateTransitionGraphSchema::GetGraphType(const UEdGraph* InGraph) const
{
	return GT_StateMachine;
}

void USceneStateTransitionGraphSchema::CreateDefaultNodesForGraph(UEdGraph& InGraph) const
{
	FGraphNodeCreator<USceneStateTransitionResultNode> NodeCreator(InGraph);
	USceneStateTransitionResultNode* ResultNode = NodeCreator.CreateNode();
	NodeCreator.Finalize();

	SetNodeMetaData(ResultNode, FNodeMetadata::DefaultGraphNode);

	USceneStateTransitionGraph* Graph = CastChecked<USceneStateTransitionGraph>(&InGraph);
	Graph->ResultNode = ResultNode;
}

bool USceneStateTransitionGraphSchema::CanDuplicateGraph(UEdGraph* InSourceGraph) const
{
	return false;
}

void USceneStateTransitionGraphSchema::GetGraphDisplayInformation(const UEdGraph& InGraph, FGraphDisplayInfo& OutDisplayInfo) const
{
	OutDisplayInfo.PlainName = FText::FromName(InGraph.GetFName());

	if (ISceneStateTransitionGraphProvider* Provider = Cast<ISceneStateTransitionGraphProvider>(InGraph.GetOuter()))
	{
		OutDisplayInfo.PlainName = FText::Format(LOCTEXT("TransitionRuleGraphTitle", "{0} (rule)"), Provider->GetTitle());
		OutDisplayInfo.Tooltip = LOCTEXT("GraphTooltip", "Transitions contain rules that define when to move between states");
	}

	OutDisplayInfo.DisplayName = OutDisplayInfo.PlainName;
}

bool USceneStateTransitionGraphSchema::ShouldAlwaysPurgeOnModification() const
{
	return true;
}

void USceneStateTransitionGraphSchema::HandleGraphBeingDeleted(UEdGraph& InGraphBeingRemoved) const
{
	Super::HandleGraphBeingDeleted(InGraphBeingRemoved);

	if (UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(&InGraphBeingRemoved))
	{
		TArray<ISceneStateTransitionGraphProvider*> Providers;
		FBlueprintEditorUtils::GetAllNodesOfClassEx<ISceneStateTransitionGraphProvider>(Blueprint, Providers);

		TSet<ISceneStateTransitionGraphProvider*> ProvidersToDelete;
		for (ISceneStateTransitionGraphProvider* Provider : Providers)
		{
			check(Provider);
			if (Provider->IsBoundToGraphLifetime(InGraphBeingRemoved))
			{
				ProvidersToDelete.Add(Provider);
			}
		}

		// Delete the providers that are bound to the lifetime of this graph
		ensure(ProvidersToDelete.Num() <= 1);

		for (ISceneStateTransitionGraphProvider* ProviderToDelete : ProvidersToDelete)
		{
			if (UEdGraphNode* Node = ProviderToDelete->AsNode())
			{
				FBlueprintEditorUtils::RemoveNode(Blueprint, Node, true);
			}
		}
	}
}

bool USceneStateTransitionGraphSchema::DoesSupportCollapsedNodes() const
{
	return false;
}

bool USceneStateTransitionGraphSchema::DoesSupportEventDispatcher() const
{
	return false;
}

#undef LOCTEXT_NAMESPACE
