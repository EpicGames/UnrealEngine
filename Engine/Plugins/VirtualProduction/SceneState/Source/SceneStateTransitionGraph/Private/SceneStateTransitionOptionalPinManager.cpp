// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneStateTransitionOptionalPinManager.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "SceneStateTransitionGraphSchema.h"

namespace UE::SceneState::Graph
{

void FTransitionOptionalPinManager::PostInitNewPin(UEdGraphPin* InPin
	, FOptionalPinFromProperty& InRecord
	, int32 InArrayIndex
	, FProperty* InProperty
	, uint8* InPropertyAddress
	, uint8* InDefaultPropertyAddress) const
{
	check(InPropertyAddress);

	const USceneStateTransitionGraphSchema* Schema = GetDefault<USceneStateTransitionGraphSchema>();
	check(Schema);

	// Initial construction of a visible pin; copy values from the struct
	FString StringValue;
	FBlueprintEditorUtils::PropertyValueToString_Direct(InProperty, InPropertyAddress, StringValue);
	Schema->SetPinDefaultValueAtConstruction(InPin, StringValue);
}

} // UE::SceneState::Graph
