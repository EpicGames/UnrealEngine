// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "K2Node.h"

namespace UE::SceneState::Graph
{

struct FTransitionOptionalPinManager : FOptionalPinManager
{
	//~ Begin FOptionalPinManager
	virtual void PostInitNewPin(UEdGraphPin* InPin
		, FOptionalPinFromProperty& InRecord
		, int32 InArrayIndex
		, FProperty* InProperty
		, uint8* InPropertyAddress
		, uint8* InDefaultPropertyAddress) const override;
	//~ End FOptionalPinManager
};

} // UE::SceneState::Graph
