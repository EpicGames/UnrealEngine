// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "ISceneStateTransitionGraphProvider.generated.h"

class FText;
class UEdGraph;
class UEdGraphNode;

UINTERFACE(MinimalAPI, meta = (CannotImplementInterfaceInBlueprint))
class USceneStateTransitionGraphProvider : public UInterface
{
	GENERATED_BODY()
};

class ISceneStateTransitionGraphProvider
{
	GENERATED_BODY()

public:
	/** Gets the Title of this Provider, to use as the Display Title of the Bound Graph */
	virtual FText GetTitle() const = 0;

	/**
	 * Determines whether the provider is bound to the given Ed Graph.
	 * In other words, whether the provider (as an ed node) should be destroyed when the bound graph is being deleted
	 */
	virtual bool IsBoundToGraphLifetime(UEdGraph& InGraph) const = 0;

	/** The provider casted as an ed graph node */
	virtual UEdGraphNode* AsNode() = 0;
};
