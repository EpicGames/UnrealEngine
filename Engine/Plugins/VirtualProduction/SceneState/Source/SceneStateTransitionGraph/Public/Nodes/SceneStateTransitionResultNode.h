// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "K2Node.h"
#include "Transition/SceneStateTransitionResult.h"
#include "SceneStateTransitionResultNode.generated.h"

UCLASS(MinimalAPI)
class USceneStateTransitionResultNode : public UK2Node
{
	GENERATED_BODY()

public:
	//~ Begin UEdGraphNode
	virtual FString GetDescriptiveCompiledName() const override;
	virtual bool CanCreateUnderSpecifiedSchema(const UEdGraphSchema* InSchema) const override;
	virtual void AllocateDefaultPins() override;
	virtual bool CanUserDeleteNode() const override;
	virtual bool CanDuplicateNode() const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type InTitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual bool IsNodeRootSet() const override;
	//~ End UEdGraphNode

	UPROPERTY(EditAnywhere, Category="Scene State Transition")
	FSceneStateTransitionResult Result;
};
