// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneStateTransitionGraph.h"
#include "SceneStateConduitGraph.generated.h"

UCLASS(MinimalAPI)
class USceneStateConduitGraph : public USceneStateTransitionGraph
{
	GENERATED_BODY()
};
