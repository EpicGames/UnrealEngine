// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraph.h"
#include "SceneStateTransitionGraph.generated.h"

class USceneStateTransitionResultNode;

UCLASS(MinimalAPI)
class USceneStateTransitionGraph : public UEdGraph
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TObjectPtr<USceneStateTransitionResultNode> ResultNode;
};
