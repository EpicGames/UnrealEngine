// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraphSchema_K2.h"
#include "SceneStateTransitionGraphSchema.generated.h"

UCLASS(MinimalAPI)
class USceneStateTransitionGraphSchema : public UEdGraphSchema_K2
{
	GENERATED_BODY()

public:
	//~ Begin UEdGraphSchema
	virtual EGraphType GetGraphType(const UEdGraph* InGraph) const override;
	virtual void CreateDefaultNodesForGraph(UEdGraph& InGraph) const override;
	virtual bool CanDuplicateGraph(UEdGraph* InSourceGraph) const override;
	virtual void GetGraphDisplayInformation(const UEdGraph& InGraph, FGraphDisplayInfo& OutDisplayInfo) const override;
	virtual bool ShouldAlwaysPurgeOnModification() const override;
	virtual void HandleGraphBeingDeleted(UEdGraph& InGraphBeingRemoved) const override;
	//~ End UEdGraphSchema

	//~ Begin UEdGraphSchema_K2
	virtual bool DoesSupportCollapsedNodes() const override;
	virtual bool DoesSupportEventDispatcher() const override;
	//~ End UEdGraphSchema_K2
};
